// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/GammaCorrectionCommon.ush"

// Same definitions as MediaShaders.usf
#define UE_Color_EEncoding_Linear		1u
#define UE_Color_EEncoding_sRGB			2u
#define UE_Color_EEncoding_ST2084		3u
#define UE_Color_EEncoding_SLog3		12u
#define UE_Color_EEncoding_HLG				17u

// Simplified version of ApplyColorProcessingRGB, removed tonemapping.
float3 ApplyColorTransformRGB(float3 InColor, float4x4 InColorSpaceMatrix, uint InEOTF)
{
	// Go linear
	switch(InEOTF)
	{
	case UE_Color_EEncoding_sRGB:				InColor = sRGBToLinear(InColor); break;
	case UE_Color_EEncoding_ST2084:				InColor = ST2084ToLinear(InColor); break;
	case UE_Color_EEncoding_SLog3:				InColor = SLog3ToLinear(InColor); break;
	case UE_Color_EEncoding_HLG:				InColor = HLGToLinear(InColor); break;
	default: break;
	}

	// Adjust colorspace
	float3 OutColor;
	float4 Color = float4(InColor, 1);
	OutColor.x = dot(InColorSpaceMatrix[0], Color);
	OutColor.y = dot(InColorSpaceMatrix[1], Color);
	OutColor.z = dot(InColorSpaceMatrix[2], Color);
	return OutColor;
}

float3 YuvToRgb(float3 InYUV, float4x4 InYUVMatrix)
{
	float3 Color;
	float4 YUV4 = float4(InYUV, 1);
	Color.x = dot(InYUVMatrix[0], YUV4);
	Color.y = dot(InYUVMatrix[1], YUV4);
	Color.z = dot(InYUVMatrix[2], YUV4);

	// Make sure we safely stay above zero
	return max(Color, 0.0f);
}

// Simplified version of ApplyColorProcessing, removed tonemapping.
float3 ApplyColorTransform(float3 InYUV, float4x4 InYUVMatrix, float4x4 InColorSpaceMatrix, uint InEOTF)
{
	return ApplyColorTransformRGB(YuvToRgb(InYUV, InYUVMatrix), InColorSpaceMatrix, InEOTF);
}