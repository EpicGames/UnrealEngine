// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Plugin/TmvMedia/Private/TmvColorProcessing.ush"

#ifndef PERMUTATION_CHANNELS
#define PERMUTATION_CHANNELS 0
#endif

#define NUM_CHANNELS (PERMUTATION_CHANNELS + 1)

/** The unswizzled buffer packed into 4 byte buffer stride. */
StructuredBuffer<uint> UnswizzledBuffer;

// --- Partial Tiled Addressing mode ---

/**
 * Describes the memory mapping of a tile "sub" buffer in the unswizzled buffer.
 */
struct FTileDesc
{
	/** Offset within the tile buffer for the start of each plane. */
	uint4 Offsets;
	/** Offset within the tile buffer for the start of each plane. */
	uint4 Strides;
};

/** For partial tiles, describes the mapping of each tile to its location in the unswizzled buffer. */
StructuredBuffer<FTileDesc> TileMappingBuffer;
// -----------------------------

/** Texture size that the buffer is supposed to be unpacked to. */
int2 DestTextureSize;

/** Size (dimensions), in pixels, of tiles that represent this texture. */
int2 TileSize;

/** Total number of tiles in X and Y direction for the current mip level. */
int2 NumTiles;

// --- Tiled Addressing mode ---

/** Total size of a single tile buffer (i.e. sum of all planes and padding). */
int TileBufferFullStride;

/** Offset within the tile buffer for the start of each plane. */
int4 TileBufferOffsets;

/** Stride of each planes in the tile buffer. */
int4 TileBufferStrides;
// -----------------------------

// --- Planar Addressing mode ---
int4 PlaneBufferOffsets;
int4 PlaneBufferStrides;
// -----------------------------

/** Ratio of width for the chroma samples */ 
int4 PlaneWidthRatios;	// todo: check if bit shifting is more efficient.

/** Ratio of height for the chroma samples */ 
int4 PlaneHeightRatios; // todo: check if bit shifting is more efficient.

/** Flag to enable color space transformations. */
uint bApplyColorTransform;

/** Source encoding to linearize. */
uint EOTF;

/** Color space transformation matrix, from source to working. */
float4x4 ColorSpaceMatrix;

uint bConvertYUV;

float4x4 YUVMatrix;

/** Alpha is not part of the color correction. */
float AlphaScale;

float4 MipTint;

/** Vertex Shader. */
void MainVS(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	out noperspective float4 OutUVAndScreenPos : TEXCOORD0,
	out float4 OutPosition : SV_POSITION)
{
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutUVAndScreenPos);
}

/** Unpacks data from the provided structured buffer. */
float GetBufferValue(int BufferOffset)
{
	// todo: need option for int8 vs int16.

	// the location as if it would've been if the buffer was unpacked.
	int UnpackedLocation = BufferOffset;
	// Byte offset within the 4 byte packed value.
	int Remainder = UnpackedLocation % 2;

	// Position in a packed buffer.
	int PackedPosition = (UnpackedLocation - Remainder) >> 1;

	uint PackedValue = (UnswizzledBuffer[PackedPosition] >> (16 * Remainder)) & 0xffff;

#if ELEMENT_FORMAT == 0
	// Convert from uint32 to f16
	return (half)f16tof32(PackedValue);
#else
	return (float)PackedValue;	// Value normalized by color transformation
#endif
}

/** Calculate the buffer "position" of the given pixel for all planes (channels). */
void GetBufferPositions_PartialTiles(in int2 InPixelPos, out int4 OutBufferPositions)
{
	int2 TileCoord = int2(ceil(InPixelPos.x / TileSize.x), ceil(InPixelPos.y / TileSize.y));
	FTileDesc TileDesc = TileMappingBuffer[TileCoord.y * NumTiles.x + TileCoord.x];

	int xTileCoord = (InPixelPos.x) % TileSize.x;
	int yTileCoord = (InPixelPos.y) % TileSize.y;

	// Always default initialize `out` parameters to avoid error X3508 in FXC
	OutBufferPositions = 0;

	// Sentinel for "unmapped tile": when virtualization is on and the visible
	// tile count exceeds the per-mip budget, overflow tiles get no slot and
	// their TileMappingBuffer entry stays at its zero-initialized default.
	// Strides[0] is the Y plane's per-row byte stride - always > 0 for a
	// valid tile, so Strides[0] == 0 is an unambiguous "not populated" flag.
	// MainPS catches the negative sentinel position below and discards the
	// pixel so the lower-mip upscale pass remains visible (instead of
	// sampling from whatever data happens to be at buffer offset 0, which
	// produced magenta and green artifacts pre-fix).
	if (TileDesc.Strides[0] == 0)
	{
		OutBufferPositions = int4(-1, -1, -1, -1);
		return;
	}

	// Luminance is always full size.
	OutBufferPositions[0] = TileDesc.Offsets[0] + yTileCoord * TileDesc.Strides[0] + xTileCoord;

	// Other components may be subsampled.
	for (int i=1; i < NUM_CHANNELS; ++i)
	{
		// Position of the pixel in the tile buffer.
		OutBufferPositions[i] = TileDesc.Offsets[i]
			+ (yTileCoord / PlaneHeightRatios[i]) * TileDesc.Strides[i]
			+ xTileCoord / PlaneWidthRatios[i];
	}
}

/** Calculate the buffer "position" of the given pixel for all planes (channels). */
void GetBufferPositions_RenderTiles(in int2 InPixelPos, out int4 OutBufferPositions)
{
	// Note: difference with _PartialTiles, ceil is used instead.
	int TileX = floor((InPixelPos.x) / TileSize.x);
	int TileY = floor((InPixelPos.y) / TileSize.y);

	int xTileCoord = (InPixelPos.x) % TileSize.x;
	int yTileCoord = (InPixelPos.y) % TileSize.y;

	int TileBufferOffset = TileY * NumTiles.x + TileX * TileBufferFullStride;

	// Always default initialize `out` parameters to avoid error X3508 in FXC
	OutBufferPositions = 0;
	
	// Luminance is always full size.
	OutBufferPositions[0] = TileBufferOffset + TileBufferOffsets[0] + yTileCoord * TileBufferStrides[0] + xTileCoord;

	// Other components may be subsampled.
	for (int i=1; i < NUM_CHANNELS; ++i)
	{
		// Position of the pixel in the tile buffer.
		OutBufferPositions[i] = TileBufferOffset + TileBufferOffsets[i]
			+ (yTileCoord / PlaneHeightRatios[i]) * TileBufferStrides[i]
			+ xTileCoord / PlaneWidthRatios[i];
	}
}

/** Calculate the buffer "position" of the given pixel for all planes (channels). */
void GetBufferPositions_Planar(in int2 InPixelPos, out int4 OutBufferPositions)
{
	// Always default initialize `out` parameters to avoid error X3508 in FXC
	OutBufferPositions = 0;
	
	// Luminance is always full size.
	OutBufferPositions[0] = PlaneBufferOffsets[0] + InPixelPos.y * PlaneBufferStrides[0] + InPixelPos.x;
	
	// Other components may be subsampled.
	for (int i=1; i < NUM_CHANNELS; ++i)
	{
		// Position of the pixel in the tile buffer.
		OutBufferPositions[i] = PlaneBufferOffsets[i]
			+ (InPixelPos.y / PlaneHeightRatios[i]) * PlaneBufferStrides[i]
			+ InPixelPos.x / PlaneWidthRatios[i];
	}
}

/** Pixel Shader. */
half4 MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0) : SV_Target0
{
	float2 UV = UVAndScreenPos.xy;
	int2 PixelPos = UV * DestTextureSize;

	int4 BufferPositions;

#if BUFFER_LAYOUT == 0
	GetBufferPositions_PartialTiles(PixelPos, BufferPositions);
#elif BUFFER_LAYOUT == 1
	GetBufferPositions_RenderTiles(PixelPos, BufferPositions);
#else
	GetBufferPositions_Planar(PixelPos, BufferPositions);
#endif

	// Unmapped-tile fallback (PartialTiles + virtualization only). The
	// PartialTiles getter sets BufferPositions to all -1 when the tile
	// descriptor was never populated (visibility exceeded the slot budget).
	// Discard so the destination keeps whatever the lower-mip upscale pass
	// (which is drawn first for the same screen region) wrote there. Without
	// this discard, reading from buffer offset 0 + a tiny xTileCoord lands
	// inside slot 0's bytes - typically a bright-Y tile, which YCbCr-converts
	// to magenta when Cb/Cr are also read from offset 0.
	if (BufferPositions[0] < 0)
	{
		discard;
	}

	float4 Color = float4(1.0f, 1.0f, 1.0f, 1.0f);
	for (int i=0; i < NUM_CHANNELS; ++i)
	{
		Color[i] = GetBufferValue(BufferPositions[i]);
	}

#if NUM_CHANNELS == 4
	Color.a *= AlphaScale;
#endif

	// Single channel video frames are converted to grayscale by duplicating the red channel.
#if NUM_CHANNELS == 1
	Color.rgb = Color.r;
#endif
	
	if (bConvertYUV)
	{
		Color.rgb = YuvToRgb(Color.rgb, YUVMatrix);
	}

	Color.rgb = lerp(Color.rgb, MipTint.rgb, MipTint.a);

	if(bApplyColorTransform)
	{
		Color.rgb = ApplyColorTransformRGB(Color.rgb, ColorSpaceMatrix, EOTF);
	}
	
	return Color;
}