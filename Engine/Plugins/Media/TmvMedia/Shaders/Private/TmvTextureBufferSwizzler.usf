// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Plugin/TmvMedia/Private/TmvColorProcessing.ush"

#ifndef PERMUTATION_CHANNELS
#define PERMUTATION_CHANNELS 0
#endif

#define NUM_CHANNELS (PERMUTATION_CHANNELS + 1)

Texture2D InPlaneTexture0;
Texture2D InPlaneTexture1;
Texture2D InPlaneTexture2;
Texture2D InPlaneTexture3;
SamplerState InPlaneTextureSampler;

/** Flag to enable color space transformations. */
uint bApplyColorTransform;

/** Source encoding to linearize. */
uint EOTF;

/** Color space transformation matrix, from source to working. */
float4x4 ColorSpaceMatrix;

uint bConvertYUV;

float4x4 YUVMatrix;

/** Alpha is not part of the color correction. */
float AlphaScale;

float4 MipTint;

float SampleComponent(Texture2D InTexture, float2 UV)
{
	return Texture2DSample(InTexture, InPlaneTextureSampler, UV).r;
}

/** Pixel Shader. */
half4 MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0) : SV_Target0
{
	float2 UV = UVAndScreenPos.xy;
	float4 Color = float4(1.0f, 1.0f, 1.0f, 1.0f);

#if NUM_CHANNELS == 4
	Color.a = SampleComponent(InPlaneTexture3, UV) * AlphaScale;
#endif
#if NUM_CHANNELS >= 3
	Color.b = SampleComponent(InPlaneTexture2, UV);
#endif
#if NUM_CHANNELS >= 2
	Color.g = SampleComponent(InPlaneTexture1, UV);
#endif
#if NUM_CHANNELS >= 1
	Color.r = SampleComponent(InPlaneTexture0, UV);
#endif

	// Special case for single channel: we convert to grayscale by duplicating the red channel.
#if NUM_CHANNELS == 1
	Color.rgb = Color.r;
#endif

	if (bConvertYUV)
	{
		Color.rgb = YuvToRgb(Color.rgb, YUVMatrix);
	}

	Color.rgb = lerp(Color.rgb, MipTint.rgb, MipTint.a);

	if(bApplyColorTransform)
	{
		Color.rgb = ApplyColorTransformRGB(Color.rgb, ColorSpaceMatrix, EOTF);
	}

	return Color;
}