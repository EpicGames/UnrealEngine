// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Plugin/TmvMedia/Private/TmvColorProcessing.ush"

/** Input media texture to convert. */ 
Texture2D InTexture;

/** Sampler state for the media texture. */
SamplerState InTextureSampler;

float4 UVScaleAndOffset;

/** Flag to enable color space transformations. */
uint bApplyColorTransform;

/** Destination encoding (transfer function) to apply. Enum value of UE::Color::EEncoding. */
uint EOTF;

/** Color space transformation matrix from "working" to destination. */
float4x4 ColorSpaceMatrix;

/** If true (non-zero), perform the rgb to yuv conversion. */
uint bConvertYUV;

/** RGB to YUV conversion matrix. */
float4x4 YUVMatrix;

/** Final Scale applied to values before being stored. */
float4 DataScale;

float4 MipTint;

/** Transforms the given color with the given matrix. */
float3 GenericColorTransform(float3 InColor, float4x4 InMatrix)
{
	float3 OutColor;
	float4 Color = float4(InColor, 1);
	OutColor.x = dot(InMatrix[0], Color);
	OutColor.y = dot(InMatrix[1], Color);
	OutColor.z = dot(InMatrix[2], Color);
	return OutColor;
}

//
// Sony S-Log3 (todo: test)
//
float3 LinearToSLog3(float3 Value)
{
	return select(Value >= 0.01125000f,
		(420.0f + log10((Value + 0.01f) / 0.19f) * 261.5f) / 1023.0f,
		(Value * 76.2102946929f / 0.01125f + 95.0f) / 1023.0f);
}

float3 EncodeTransferFunction(float3 InColor, uint InEOTF)
{
	switch(InEOTF)
	{
	case UE_Color_EEncoding_sRGB:	
		InColor = LinearToSrgb(InColor); 
		break;

	case UE_Color_EEncoding_ST2084:
		InColor = LinearToST2084(InColor);
		break;

	case UE_Color_EEncoding_SLog3:
		InColor = LinearToSLog3(InColor);
		break;

	case UE_Color_EEncoding_HLG:
		// HLG signal is spec-bounded to [0, 1]; clamp to peak so per-channel saturation is
		// consistent regardless of which YUV plane the max magnitude lands on.
		InColor = min(LinearToHLG(InColor), 1.0f);
		break;

	default:
		break;
	}
	return InColor;
}

/** Converts from screen position (FScreenVertexShaderVS output) to texture UV according to draw info. */
float2 ScreenPosToUV(float2 InScreenPos)
{
	return InScreenPos * UVScaleAndOffset.xy + UVScaleAndOffset.zw;
}

/** Pixel Shader. */
half4 MainPS(float4 SvPosition : SV_POSITION) : SV_Target0
{
	float4 Color = InTexture.Sample(InTextureSampler, ScreenPosToUV(SvPosition.xy));

	// Color space transform
	Color.rgb = GenericColorTransform(Color.rgb, ColorSpaceMatrix);
	Color.rgb = max(EncodeTransferFunction(Color.rgb, EOTF), 0.0f);	// Make sure we safely stay above zero.

	if (bConvertYUV)
	{
		Color.rgb = max(GenericColorTransform(Color.rgb, YUVMatrix), 0.0f);
	}
	return Color * DataScale;
}
