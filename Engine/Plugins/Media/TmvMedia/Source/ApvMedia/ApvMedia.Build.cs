// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ApvMedia : ModuleRules
	{
		public ApvMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"Media",
					"Projects", 
					"RHI",
					"RenderCore",
					"Renderer",
					"TmvMedia",
				});
			
			// Todo: other platforms
			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				PrivateDependencyModuleNames.Add("UEOpenAPV");
			}
		}
	}
}
