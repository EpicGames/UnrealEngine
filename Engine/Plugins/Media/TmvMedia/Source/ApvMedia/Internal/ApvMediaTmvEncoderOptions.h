// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ApvMediaTypes.h"
#include "Encoder/TmvMediaEncoderOptions.h"
#include "Misc/FrameRate.h"

#include "ApvMediaTmvEncoderOptions.generated.h"

#define UE_API APVMEDIA_API

/**
 * OpenApv Tmv Encoder options.
 */
USTRUCT(DisplayName = "OpenApv Tmv Encoder Options")
struct FApvMediaTmvEncoderOptions : public FTmvMediaEncoderOptions
{
	GENERATED_BODY()

	FApvMediaTmvEncoderOptions()
	{
		// Select commonly used color settings by default.
		bEnableColorManagement = true;
		DestinationColorSpace = ETextureColorSpace::TCS_sRGB;
		DestinationEncoding = ETmvMediaEncoderEncoding::sRGB;
		YuvMatrix = ETmvMediaEncoderColorMatrix::Rec709;
		YuvMatrixRange = ETmvMediaEncoderColorMatrixRange::Full;
	}

	virtual ~FApvMediaTmvEncoderOptions() override = default;

	//~ Begin FTmvMediaEncoderOptions
	UE_API virtual FName GetEncoderName() const override;
	virtual FString GetFileSequenceExtension() const override { return TEXT("apv1"); }
	UE_API virtual uint32 GetCodecFourCC() const override;

	virtual void GetSupportedDestinationColorSpaces(TArray<ETextureColorSpace>& OutSupportedColorSpaces) const override
	{
		// The APV encoder maps UE color spaces to OpenApv color primaries in GetApvColorSpace (ApvMipBufferUtils.h);
		// only sRGB/Rec709 and Rec2020 have a match.
		OutSupportedColorSpaces = { ETextureColorSpace::TCS_sRGB, ETextureColorSpace::TCS_Rec2020 };
	}

	virtual void GetSupportedDestinationEncodings(TArray<ETmvMediaEncoderEncoding>& OutSupportedEncodings) const override
	{
		OutSupportedEncodings = {
			ETmvMediaEncoderEncoding::Linear,
			ETmvMediaEncoderEncoding::sRGB,
			ETmvMediaEncoderEncoding::ST2084,
			ETmvMediaEncoderEncoding::SLog3,
			ETmvMediaEncoderEncoding::HLG,
		};
	}
	//~ End FTmvMediaEncoderOptions

	/** Get currently configured color/chromat format. */
	EApvMediaChromaFormat GetChromaFormat() const
	{
		return  UE::ApvMedia::GetChromaFormat(Profile);
	}

	/** Get currently configured bit depth */
	int8 GetBitDepth() const
	{
		return  UE::ApvMedia::GetBitDepth(Profile);
	}

	/** 
	 * Dimensions of a tile in pixels.
	 * The tile size determines parallelism and performance. Frames are divided into rectangular tiles that 
	 * can be encoded and decoded simultaneously, enabling high throughput on multicore processors.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0.0"))
	FIntPoint TileSize = FIntPoint(256, 256);

	/**
	 * Specify the color/chroma profile, includes bitdepth.
	 * Input video will be converted to the given profile.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings")
	EApvMediaProfile Profile = EApvMediaProfile::YCbCr422_10;

	/**
	 * Coded data rate band Setting.
	 * If the bitrate is not specified otherwise, this determines the max coded data rate (higher band leads to higher bit rate).
	 */
	UPROPERTY(EditAnywhere, Category = "Settings")
	EApvMediaBand Band = EApvMediaBand::Band2;

	// todo: bitrate/quality specification: low, standard, high, ultra. (Family parameter)

	/**
	 * Specify the encoding optimization level.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings")
	EApvMediaPreset Preset = EApvMediaPreset::Medium;

	/**
	 * Number of worker thread to encode tiles (0 = automatically use the available number of cores).
	 */
	UPROPERTY(EditAnywhere, Category = "Advanced", meta = (ClampMin = "0.0"))
	int32 NumThreads = 0;
};

#undef UE_API
