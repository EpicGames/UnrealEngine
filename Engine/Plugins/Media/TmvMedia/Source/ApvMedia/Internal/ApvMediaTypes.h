// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ApvMediaTypes.generated.h"

/**
 * List of color/chroma image buffer formats used by OpenApv.
 */
UENUM()
enum class EApvMediaChromaFormat : uint8
{
	/** 1 component (monochrome) */
	YCbCr400 = 0,
	/** 3 components YUV 422 */
	YCbCr422 = 2,
	/** 3 components YUV 444 */
	YCbCr444 = 3,
	/** 4 components YUV 444 + alpha */
	YCbCrA4444 = 4,
	/** 3 components YUV 422 with UV in the same memory plane. */
	YCbCr422_P2 = 5
};

/** Optimization level control presets */
UENUM()
enum class EApvMediaPreset : uint8
{
	Fastest = 0,
	Fast = 1,
	Medium = 2,
	Slow = 3,
	Placebo = 4
};

/**
 * Each profile specifies a subset of algorithmic features and limits that must be supported
 * by all decoders conforming to that profile. */
UENUM()
enum class EApvMediaProfile : uint8
{
	YCbCr422_10 UMETA(DisplayName = "4:2:2 10 bits", ToolTip = "422-10: YCbCr422 10 bits"),
	YCbCr422_12 UMETA(DisplayName = "4:2:2 12 bits", ToolTip = "422-12: YCbCr422 12 bits"),
	YCbCr444_10 UMETA(DisplayName = "4:4:4 10 bits", ToolTip = "444-10: YCbCr444 10 bits"),
	YCbCr444_12 UMETA(DisplayName = "4:4:4 12 bits", ToolTip = "444-12: YCbCr444 12 bits"),
	YCbCr4444_10 UMETA(DisplayName = "4:4:4:4 10 bits", ToolTip = "4444-10: YCbCrA4444 10 bits"),
	YCbCr4444_12 UMETA(DisplayName = "4:4:4:4 12 bits", ToolTip = "4444-12: YCbCrA4444 12 bits"),
	YCbCr400_10 UMETA(DisplayName = "4:0:0 10 bits Monochrome", ToolTip = "400-10: YCbCr400 (monochrome) 10 bits"),
};

/**
 * Supported bit depths.
 */
UENUM()
enum class EApvMediaBitDepth : uint8
{
	/** 10 bits per component */
	BitDepth_10 = 10,
	/** 12 bits per component */
	BitDepth_12 = 12,
};

/**
 * 	 Determines the max coded data rate (higher band leads to higher bit rate).
 */
UENUM()
enum class EApvMediaBand : uint8
{
	/** Lowest coded bit rate */
	Band0 = 0 UMETA(DisplayName = "0"),
	/** Low coded bit rate (~150% of lowest) */
	Band1 = 1 UMETA(DisplayName = "1"),
	/** Medium coded bit rate (~200% of lowest) */
	Band2 = 2 UMETA(DisplayName = "2"),
	/** Highest coded bit rate (~300% of lowest) */
	Band3 = 3 UMETA(DisplayName = "3"),
};

/**
 * Color transfer functions defined in OpenApv.
 * Reference: Rec H.273 https://www.itu.int/rec/T-REC-H.273
 */
UENUM()
enum class EApvMediaColorTransfer : uint8
{
	Reserved0 = 0 UMETA(Hidden),
	Bt709 = 1,
	Unspecified = 2,
	Reserved3 = 3 UMETA(Hidden),
	Bt470M = 4,
	Bt470BG = 5,
	Smpte170m = 6,
	Smpte240m = 7,
	Linear = 8,
	Log100 = 9,
	Log316 = 10,
	Iec61966_2_4 = 11,
	Bt1361e = 12,
	Iec61966_2_1 = 13,
	Bt2020_10 = 14,
	Bt2020_12 = 15,
	Smpte2084 = 16,
	Smpte428 = 17,
	Arib_std_b67 = 18,
};

/**
 * Color primaries defined in OpenApv.
 * Reference: Rec H.273 https://www.itu.int/rec/T-REC-H.273
 */
UENUM()
enum class EApvMediaColorSpace : uint8
{
	Reserved0 = 0 UMETA(Hidden),
	Bt709 = 1,
	Unspecified = 2,
	Reserved3 = 3 UMETA(Hidden),
	Bt470m = 4,
	Bt470bg = 5,
	Smpte170m = 6,
	Smpte240m = 7,
	Film = 8,
	Bt2020 = 9,
	Smpte428 = 10,
	Smpte431 = 11,
	Smpte432 = 12,
};

/**
 * YUV color matrices defined in OpenApv.
 * Reference: Rec H.273 https://www.itu.int/rec/T-REC-H.273
 */
UENUM()
enum class EApvMediaColorMatrix : uint8
{
	Gbr = 0,
	Bt709 = 1,
	Unspecified = 2,
	Reserved3 = 3 UMETA(Hidden),
	Fcc = 4,
	Bt470bg = 5,
	Smpte170m = 6,
	Smpte240m = 7,
	Ycgco = 8,
	Bt2020nc = 9,
	Bt2020c = 10,
	Smpte2085 = 11,
	Chroma_derived_nc = 12,
	Chroma_derived_c = 13,
	Ictcp = 14,
};

/**
 * YUV color matrix ranges defined in OpenApv.
 * Specifies the scaling and offset applied in association with the YUV matrix coefficients.
 * Reference: Rec H.273 https://www.itu.int/rec/T-REC-H.273
 */
UENUM()
enum class EApvMediaColorMatrixRange : uint8
{
	/** Standard for television, broadcast and video content. Typically defines black at 16 and white at 235 (standard dependent). */
	Limited = 0,
	/** Standard for PC graphics and image processing. Defines black at 0 and white at 255. */
	Full = 1
};

namespace UE::ApvMedia
{
	/** Get the color/chroma format from the given apv profile. */
	inline EApvMediaChromaFormat GetChromaFormat(const EApvMediaProfile InProfile)
	{
		switch (InProfile)
		{
		case EApvMediaProfile::YCbCr400_10:
			return EApvMediaChromaFormat::YCbCr400;
		case EApvMediaProfile::YCbCr422_10:
			return EApvMediaChromaFormat::YCbCr422;
		case EApvMediaProfile::YCbCr422_12:
			return EApvMediaChromaFormat::YCbCr422;
		case EApvMediaProfile::YCbCr444_10:
			return EApvMediaChromaFormat::YCbCr444;
		case EApvMediaProfile::YCbCr444_12:
			return EApvMediaChromaFormat::YCbCr444;
		case EApvMediaProfile::YCbCr4444_10:
			return EApvMediaChromaFormat::YCbCrA4444;
		case EApvMediaProfile::YCbCr4444_12:
			return EApvMediaChromaFormat::YCbCrA4444;
		default:
			return EApvMediaChromaFormat::YCbCr422;
		}
	}

	/** Get the bit depth from the given apv profile. */
	inline int8 GetBitDepth(const EApvMediaProfile InProfile)
	{
		switch (InProfile)
		{
		case EApvMediaProfile::YCbCr400_10:
			return 10;
		case EApvMediaProfile::YCbCr422_10:
			return 10;
		case EApvMediaProfile::YCbCr422_12:
			return 12;
		case EApvMediaProfile::YCbCr444_10:
			return 10;
		case EApvMediaProfile::YCbCr444_12:
			return 12;
		case EApvMediaProfile::YCbCr4444_10:
			return 10;
		case EApvMediaProfile::YCbCr4444_12:
			return 12;
		default:
			return 10;
		}
	}

	/** Get the number of components from the given apv chroma format. */
	inline int32 GetNumComponents(const EApvMediaChromaFormat InFormat)
	{
		switch (InFormat)
		{
		case EApvMediaChromaFormat::YCbCr400:
			return 1;
		case EApvMediaChromaFormat::YCbCr422:
		case EApvMediaChromaFormat::YCbCr422_P2:
		case EApvMediaChromaFormat::YCbCr444:
			return 3;
		case EApvMediaChromaFormat::YCbCrA4444:
			return 4;
		default:
			return 3;
		}
	}

	/** Get the number of memory planes for the given color/chroma format. */
	inline int32 GetNumPlanes(const EApvMediaChromaFormat InFormat)
	{
		switch (InFormat)
		{
		case EApvMediaChromaFormat::YCbCr400:
			return 1;
		case EApvMediaChromaFormat::YCbCr422_P2:
			return 2; // 2 planes (Y and UV)
		case EApvMediaChromaFormat::YCbCr422:
		case EApvMediaChromaFormat::YCbCr444:
			return 3;
		case EApvMediaChromaFormat::YCbCrA4444:
			return 4;
		default:
			return 3;
		}
	}

	/** Get the number of components packed in the given plane for the given format. */
	inline int32 GetPlaneNumComponents(const int32 InPlaneIndex, const EApvMediaChromaFormat InFormat)
	{
		if (InFormat == EApvMediaChromaFormat::YCbCr422_P2)
		{
			return InPlaneIndex == 0 ? 1 : 2;
		}
		return 1;
	}

	/** Returns the width factor for the chroma samples of the given format. */
	inline int32 GetSubWidthC(const EApvMediaChromaFormat InFormat)
	{
		// Chroma 420 is not supported.
		if (InFormat == EApvMediaChromaFormat::YCbCr422_P2 || InFormat == EApvMediaChromaFormat::YCbCr422)
		{
			return 2;
		}
		return 1;
	}

	/** Returns the height factor for the chroma samples of the given format. */
	inline int32 GetSubHeightC(const EApvMediaChromaFormat InFormat)
	{
		// Only 420 has a sub sampling for chroma.
		// But, it doesn't seem supported by the encoder though.
		// todo: double check.
		return 1;
	}
}
