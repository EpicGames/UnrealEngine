// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START

#define OAPV_STATIC_DEFINE
#include "oapv/oapv.h"
#undef OAPV_STATIC_DEFINE

THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

// Extension for selective multi mips tile decoding.
#ifndef OAPV_HAS_SELECTIVE_MULTI_MIPS_DECODE_API
#define OAPV_HAS_SELECTIVE_MULTI_MIPS_DECODE_API 0
#endif

// Extension for selective tile decoding.
#ifndef OAPV_HAS_SELECTIVE_DECODE_API
#define OAPV_HAS_SELECTIVE_DECODE_API   0
#endif

// Extension for frame decoding
#ifndef OAPV_HAS_FRAME_DECODE_API
#define OAPV_HAS_FRAME_DECODE_API 0
#endif

// Extension for logging
#ifndef OAPV_HAS_LOGGING_API
#define OAPV_HAS_LOGGING_API 0
#endif

// Extension for memory management
#ifndef OAPV_HAS_MEMORY_API
#define OAPV_HAS_MEMORY_API 0
#endif

#ifndef OAPV_HAS_CPU_TRACE_API
#define OAPV_HAS_CPU_TRACE_API 0
#endif