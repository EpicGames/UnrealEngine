// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApvDecoder.h"
#include "ApvMediaLog.h"
#include "Templates/UniquePtr.h"

namespace UE::ApvMedia
{
	const TCHAR* GetApvErrorString(int32 InError)
	{
		switch (InError)
		{
		case OAPV_ERR:
			return TEXT("Generic Error");
		case OAPV_ERR_INVALID_ARGUMENT:
			return TEXT("Invalid argument");
		case OAPV_ERR_OUT_OF_MEMORY:
			return TEXT("Out of memory");
		case OAPV_ERR_REACHED_MAX:
			return TEXT("reached max");
		case OAPV_ERR_UNSUPPORTED:
			return TEXT("Unsupported operation");
		case OAPV_ERR_UNEXPECTED:
			return TEXT("Unexpected error");
		case OAPV_ERR_UNSUPPORTED_COLORSPACE:
			return TEXT("Unsupported color space");
		case OAPV_ERR_MALFORMED_BITSTREAM:
			return TEXT("Malformed bitstream");
		case OAPV_ERR_OUT_OF_BS_BUF:
			return TEXT("Bitstream buffer too small");
		case OAPV_ERR_NOT_FOUND:
			return TEXT("Not found");
		case OAPV_ERR_FAILED_SYSCALL:
			return TEXT("System call failed");
		case OAPV_ERR_INVALID_PROFILE:
			return TEXT("Invalid profile");
		case OAPV_ERR_INVALID_LEVEL:
			return TEXT("Invalid level");
		case OAPV_ERR_INVALID_WIDTH:
			return TEXT("Invalid width");
		case OAPV_ERR_INVALID_HEIGHT:
			return TEXT("Invalid height");
		case OAPV_ERR_INVALID_QP:
			return TEXT("Invalid QP");
		case OAPV_ERR_INVALID_FAMILY:
			return TEXT("Invalid Family Number");
		default:
			return TEXT("Unknown error");
		}
	}

	FApvImageBitmap::FApvImageBitmap()
		: ReferenceCount(0)
	{
		FMemory::Memset(this, 0, sizeof(oapv_imgb_t));
	}

	// Adapted from imgb_create in oapv_app_util.
	oapv_imgb_t* FApvImageBitmap::Create(int InWidth, int InHeight, int InColorSpace, bool bInAllocate)
	{
		TUniquePtr<FApvImageBitmap> ImageBitmap = MakeUnique<FApvImageBitmap>();		
		if (ImageBitmap->Init(InWidth, InHeight, InColorSpace, bInAllocate))
		{
			return ImageBitmap.Release();
		}
		return nullptr;
	}

	bool FApvImageBitmap::Init(int InWidth, int InHeight, int InColorSpace, bool bInAllocate)
	{
		int bd = OAPV_CS_GET_BYTE_DEPTH(InColorSpace); /* byte unit */

		w[0] = InWidth;
		h[0] = InHeight;
		switch(OAPV_CS_GET_FORMAT(InColorSpace))
		{
		case OAPV_CF_YCBCR400:
			w[1] = w[2] = InWidth;
			h[1] = h[2] = InHeight;
			np = 1;
			break;
		case OAPV_CF_YCBCR420:
			w[1] = w[2] = (InWidth + 1) >> 1;
			h[1] = h[2] = (InHeight + 1) >> 1;
			np = 3;
			break;
		case OAPV_CF_YCBCR422:
			w[1] = w[2] = (InWidth + 1) >> 1;
			h[1] = h[2] = InHeight;
			np = 3;
			break;
		case OAPV_CF_YCBCR444:
			w[1] = w[2] = InWidth;
			h[1] = h[2] = InHeight;
			np = 3;
			break;
		case OAPV_CF_YCBCR4444:
			w[1] = w[2] = w[3] = InWidth;
			h[1] = h[2] = h[3] = InHeight;
			np = 4;
			break;
		case OAPV_CF_PLANAR2:
			w[1] = InWidth;
			h[1] = InHeight;
			np = 2;
			break;
		default:
			UE_LOGF(LogApvMedia, Error, "FApvImageBitmap: unsupported color format");
			return false;
		}

		for(int i = 0; i < np; i++)
		{
			// width and height need to be aligned to macroblock size
			aw[i] = Align(w[i], OAPV_MB_W);
			s[i] = aw[i] * bd;
			ah[i] = Align(h[i], OAPV_MB_H);
			e[i] = ah[i];

			bsize[i] = s[i] * e[i];

			if (bInAllocate)
			{
				a[i] = baddr[i] = FMemory::Malloc(bsize[i]);
				FMemory::Memset(a[i], 0, bsize[i]);	// TODO: check if we can remove this memset.
			}
		}

		cs = InColorSpace;
		addref = FApvImageBitmap::AddRef;
		getref = FApvImageBitmap::GetRef;
		release = FApvImageBitmap::Release;
		addref(this); /* increase reference count */

		return true;
	}

	
	int FApvImageBitmap::ReleaseImpl()
	{
		const int RefCount = DecRefImpl();
		if(RefCount == 0)
		{
			for(int i = 0; i < OAPV_MAX_CC; i++)
			{
				if(baddr[i])
				{
					FMemory::Free(baddr[i]);
				}
			}
				
			delete this;
		}
		return RefCount;
	}

#if OAPV_HAS_MEMORY_API
	void* ApvMallocHandler(size_t InSize)
	{
		return FMemory::Malloc(InSize);
	}

	void* ApvCallocHandler(size_t InCount, size_t InSize)
	{
		return FMemory::MallocZeroed(InCount * InSize);
	}

	void* ApvReallocHandler(void* InBlock, size_t InSize)
	{
		return FMemory::Realloc(InBlock, InSize);
	}

	void ApvFreeHandler(void* InBlock)
	{
		return FMemory::Free(InBlock);
	}

	void ApvInstallMemoryHooks() 
	{
		static bool bIsInstalled = false;	// todo: manage this more globally.
		if (!bIsInstalled)
		{
			oapv_memory_callbacks_t ApvMemoryCallbacks;
			FMemory::Memzero(&ApvMemoryCallbacks, sizeof(ApvMemoryCallbacks));
			ApvMemoryCallbacks.malloc = ApvMallocHandler;
			ApvMemoryCallbacks.calloc = ApvCallocHandler;
			ApvMemoryCallbacks.realloc = ApvReallocHandler;
			ApvMemoryCallbacks.free = ApvFreeHandler;
			if (oapv_set_memory_callbacks(&ApvMemoryCallbacks) != OAPV_OK)
			{
				UE_LOGF(LogApvMedia, Error, "ApvDecoder: Failed to install APV memory callback handlers.");
			}
			bIsInstalled = true;	// don't call this again.
		}
	}
#endif

#if OAPV_HAS_LOGGING_API
	void ApvLogHandler(const char* message, int verbosity, void* userdata)
	{
		switch (verbosity)
		{
		case OAPV_LOG_ERROR:
			UE_LOGF(LogApvMedia, Error, "OpenApv: %ls", StringCast<TCHAR>(message).Get());
			break;
		case OAPV_LOG_WARNING:
			UE_LOGF(LogApvMedia, Warning, "OpenApv: %ls", StringCast<TCHAR>(message).Get());
			break;
		case OAPV_LOG_INFO:
			UE_LOGF(LogApvMedia, Log, "OpenApv: %ls", StringCast<TCHAR>(message).Get());
			break;
		case OAPV_LOG_DEBUG:
			UE_LOGF(LogApvMedia, Verbose, "OpenApv: %ls", StringCast<TCHAR>(message).Get());
			break;
		}
	}

	void ApvInstallLogHandler()
	{
		static bool bIsInstalled = false;
		if (!bIsInstalled)
		{
			oapv_set_logging_callback(ApvLogHandler, nullptr);
			oapv_set_logging_verbosity(OAPV_LOG_WARNING);	// Don't output stat info unless profiling or troublehsooting.
			bIsInstalled = true;	// don't call this again.

		}
	}
#endif

#if OAPV_HAS_CPU_TRACE_API && CPUPROFILERTRACE_ENABLED
	void ApvCpuTraceBegin(const char* InName, const char* InFile, int InLine)
	{
		FCpuProfilerTrace::OutputBeginDynamicEvent(InName, InFile, InLine);
	}
	
	void ApvCpuTraceEnd()
	{
		FCpuProfilerTrace::OutputEndEvent();
	}
	
	void ApvInstallCpuTraceHandlers()
	{
		static bool bIsInstalled = false;	// todo: manage this more globally.
		if (!bIsInstalled)
		{
			oapv_cputrace_callbacks_t ApvCpuTraceCallbacks;
			FMemory::Memzero(&ApvCpuTraceCallbacks, sizeof(ApvCpuTraceCallbacks));
			ApvCpuTraceCallbacks.begin_event = ApvCpuTraceBegin;
			ApvCpuTraceCallbacks.end_event = ApvCpuTraceEnd;
			if (oapv_set_cputrace_callbacks(&ApvCpuTraceCallbacks) != OAPV_OK)
			{
				UE_LOGF(LogApvMedia, Error, "ApvDecoder: Failed to install APV cpu trace callback handlers.");
			}
			bIsInstalled = true;	// Don't call this again.
		}
	}
#endif
	
	FApvDecoderContext::FApvDecoderContext(int32 InNumDecodeThreads)
	{
#if OAPV_HAS_MEMORY_API
		ApvInstallMemoryHooks();
#endif
#if	OAPV_HAS_LOGGING_API
		ApvInstallLogHandler();
#endif

#if OAPV_HAS_CPU_TRACE_API && CPUPROFILERTRACE_ENABLED
		ApvInstallCpuTraceHandlers();
#endif
		
		oapvd_cdesc_t DecoderDescriptor;
		FMemory::Memzero(&DecoderDescriptor, sizeof(oapvd_cdesc_t));
		DecoderDescriptor.threads = InNumDecodeThreads;

		int Err = 0;
		did = oapvd_create(&DecoderDescriptor, &Err);
		if (did == nullptr)
		{
			UE_LOGF(LogApvMedia, Error, "ApvDecoderContext: cannot create OpenApv decoder (err=%d)", Err);
		}
		else
		{
			UE_LOGF(LogApvMedia, Log,
				"ApvDecoderContext: created OpenApv decoder (v%ls) with %d worker threads",
				StringCast<TCHAR>(oapv_version(nullptr)).Get(), DecoderDescriptor.threads);
		}
		
		mid = oapvm_create(&Err);
		if(OAPV_FAILED(Err))
		{
			UE_LOGF(LogApvMedia, Error, "ApvDecoderContext: cannot create OpenApv metadata container (err=%d)", Err);
		}
	}
		
	FApvDecoderContext::~FApvDecoderContext()
	{
		if (did != nullptr)
		{
			oapvd_delete(did);
			did = nullptr;
		}
		if (mid != nullptr)
		{
			oapvm_delete(mid);
			mid = nullptr;
		}
	}
}
