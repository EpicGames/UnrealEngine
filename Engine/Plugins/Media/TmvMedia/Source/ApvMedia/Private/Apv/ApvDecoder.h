// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ApvCommon.h"
#include "HAL/UnrealMemory.h"

namespace UE::ApvMedia
{
	/** Returns the error string corresponding to the error code (OAPV_ERR_...) to use for error reporting. */
	const TCHAR* GetApvErrorString(int32 InError);
	
	/**
	 * Utility wrapper for the decoding image buffer. 
	 */
	struct FApvImageBitmap : public oapv_imgb_t
	{
		FApvImageBitmap();
		
		/** Creates an image buffer of the given dimensions and color space. */
		static oapv_imgb_t* Create(int InWidth, int InHeight, int InColorSpace, bool bInAllocate = true);

		/** Initialise */ 
		bool Init(int InWidth, int InHeight, int InColorSpace, bool bInAllocate = true);

	private:
		// Adding an actual atomic ref count here (may not be necessary though).
		std::atomic<int> ReferenceCount;

		int AddRefImpl()
		{
			int OldRef = ReferenceCount.fetch_add(1); 
			return OldRef + 1; // original oapv code (atomic_inc) returns incremented value.
		}

		int DecRefImpl()
		{
			int OldRef = ReferenceCount.fetch_sub(1);
			return OldRef - 1; // original oapv code (atomic_dec) returns decremented value.
		}

		int GetRefImpl() const
		{
			return ReferenceCount.load();
		}

		int ReleaseImpl();
		
		static int AddRef(oapv_imgb_t *imgb)
		{
			return static_cast<FApvImageBitmap*>(imgb)->AddRefImpl();
		}

		static int GetRef(oapv_imgb_t *imgb)
		{
			return static_cast<FApvImageBitmap*>(imgb)->GetRefImpl();
		}

		static int Release(oapv_imgb_t *imgb)
		{
			return static_cast<FApvImageBitmap*>(imgb)->ReleaseImpl();
		}
	};

	// Wrapper utility for oapv frames
	struct FApvFrames : public oapv_frms_t
	{
		FApvFrames()
		{
			FMemory::Memset(this, 0, sizeof(oapv_frms_t));
		}

		~FApvFrames()
		{
			for(int i = 0; i < num_frms; i++)
			{
				if (frm[i].imgb != nullptr)
				{
					frm[i].imgb->release(frm[i].imgb);
				}
			}
		}
	};

	/**
	 * Recyclable Context for a "ReadFrame" function call, which is on a worker thread.
	 * Not meant to be thread safe directly (but must be acquired and released by a thread safe pool container).
	 */
	struct FApvDecoderContext
	{
		FApvDecoderContext(int32 InNumDecodeThreads);
		
		~FApvDecoderContext();

		bool IsValid() const
		{
			return did != nullptr && mid != nullptr;
		}
		
		// oapv decoder context
		oapvd_t did = nullptr;

		// oapv metadata container
		oapvm_t mid = nullptr;
		
		// oapv frame buffers
		// Contains the destination frame buffers the decoder will write to.
		FApvFrames DecodedFrames;
	};
}