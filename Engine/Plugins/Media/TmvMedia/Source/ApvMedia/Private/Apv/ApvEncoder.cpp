// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApvEncoder.h"
#include "ApvMediaLog.h"
#include "ApvMediaSettings.h"
#include "ApvMediaTmvEncoderOptions.h"
#include "ApvMipBufferUtils.h"
#include "Encoder/ITmvMediaEncoder.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Utils/TmvMediaMessageContext.h"
#include "Utils/TmvMediaTimeUtils.h"
#include "Utils/TmvMediaUtils.h"

#define LOCTEXT_NAMESPACE "ApvEncoderContext"

namespace UE::ApvMedia
{
	int32 GetProfileIdc(EApvMediaProfile InProfile)
	{
		switch (InProfile)
		{
		case EApvMediaProfile::YCbCr400_10:
			return OAPV_PROFILE_400_10;
		case EApvMediaProfile::YCbCr422_10:
			return OAPV_PROFILE_422_10;
		case EApvMediaProfile::YCbCr422_12:
			return OAPV_PROFILE_422_12;
		case EApvMediaProfile::YCbCr444_10:
			return OAPV_PROFILE_444_10;
		case EApvMediaProfile::YCbCr444_12:
			return OAPV_PROFILE_444_12;
		case EApvMediaProfile::YCbCr4444_10:
			return OAPV_PROFILE_4444_10;
		case EApvMediaProfile::YCbCr4444_12:
			return OAPV_PROFILE_4444_12;
		default:
			return OAPV_PROFILE_422_10;
		}
	}

	int32 GetPreset(EApvMediaPreset InPreset)
	{
		switch (InPreset)
		{
		case EApvMediaPreset::Fastest:
			return OAPV_PRESET_FASTEST;
		case EApvMediaPreset::Fast:
			return OAPV_PRESET_FAST;
		case EApvMediaPreset::Medium:
			return OAPV_PRESET_MEDIUM;
		case EApvMediaPreset::Slow:
			return OAPV_PRESET_SLOW;
		case EApvMediaPreset::Placebo:
			return OAPV_PRESET_PLACEBO;
		}
		return OAPV_PRESET_DEFAULT;
	}

	int32 GetValidatedNumThreads(int32 InNumThreads)
	{
		int32 NumThreads = InNumThreads > 0 ? InNumThreads : FMath::Min(FPlatformMisc::NumberOfCores(), OAPV_MAX_THREADS);
		if (NumThreads > OAPV_MAX_THREADS)
		{
			UE_LOGF(LogApvMedia, Warning, "Explicitly requested number of threads (%d) exceeds maximum OpenAPV threads (%d).", NumThreads, OAPV_MAX_THREADS);
		}
		return FMath::Clamp(NumThreads, 1, OAPV_MAX_THREADS);
	}

	/** Log helper for displaying the name of an enum value as string. */
	template <typename InEnumType>
	FString GetEnumNameStringByValue(const int64 InValue)
	{
		FString ValueName = StaticEnum<InEnumType>()->GetNameStringByValue(InValue);
		return !ValueName.IsEmpty() ? ValueName : TEXT("<invalid_enum_value>");
	}

	FApvEncoderContext::FApvEncoderContext(
		const FApvMediaTmvEncoderOptions& InOptions,
		const FTmvMediaEncoderMipInfo& InMipInfo,
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		FTmvMediaMessageContext* OutMessageContext)
	{
		// Remark:
		// we need to have all the encoder options when creating the encoder.
		// This includes the frame dimensions and number of mips.
		oapve_cdesc_t EncoderDescriptor;
		FMemory::Memzero(&EncoderDescriptor, sizeof(oapve_cdesc_t));

		// Configure Primary Frame Parameters (other frames will have the same config)
		oapve_param_t& PrimaryFrameParams = EncoderDescriptor.param[0];
		oapve_param_default(&PrimaryFrameParams);
		EncoderDescriptor.max_num_frms = 1;	// Keep track of the access unit frames we are using.

		// Report invalid frame dimensions.
		if (InMipInfo.Width <= 0 || InMipInfo.Height <= 0)
		{
			FText Message = FText::Format(
				LOCTEXT("InvalidFrameDim", "Invalid frame dimensions ({0} x {1})"), 
				FText::AsNumber(InMipInfo.Width), FText::AsNumber(InMipInfo.Height));
			UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("ApvEncoderContext"), Message);
			return;
		}

		// Setup encoder options here.
		PrimaryFrameParams.w = InMipInfo.Width;
		PrimaryFrameParams.h = InMipInfo.Height;
		PrimaryFrameParams.tile_w = Align(InOptions.TileSize.X, OAPV_MB_W);
		PrimaryFrameParams.tile_h = Align(InOptions.TileSize.Y, OAPV_MB_H);
		PrimaryFrameParams.profile_idc = GetProfileIdc(InOptions.Profile);
		PrimaryFrameParams.band_idc = FMath::Clamp(static_cast<std::underlying_type_t<EApvMediaBand>>(InOptions.Band), 0, 3);
		PrimaryFrameParams.preset = GetPreset(InOptions.Preset);

		// Note: frame rate is not stored by OpenApv frame header.
		// todo: Could use metadata for this as an option (if no media container is used).
		const FFrameRate FrameRate = UE::TmvMedia::TimeUtils::ComputeFrameRateFromDuration(InTimeInfo.FrameDuration, FFrameRate(24, 1));
		PrimaryFrameParams.fps_den = FrameRate.Denominator;
		PrimaryFrameParams.fps_num = FrameRate.Numerator;

		// Rate control - Average bitrate vs constant qp.
		// Average bitrate: uses the frame rate to determine a target bit rate, see oapve_family_bitrate.
		PrimaryFrameParams.rc_type = OAPV_RC_ABR;	// rate control type - average bit rate
		// If target bitrate is not specified, the max coding rate of the specified band (and level) is used.

		if (InMipInfo.ColorInfo.Encoding != UE::Color::EEncoding::None 
			|| InMipInfo.ColorInfo.ColorSpace != UE::Color::EColorSpace::None
			|| InMipInfo.ColorInfo.YuvMatrix != ETmvMediaFrameColorMatrix::None)
		{
			PrimaryFrameParams.color_description_present_flag = 1;	// enable writing color description in header.

			const TOptional<EApvMediaColorTransfer> ApvColorTransfer = GetApvColorTransfer(InMipInfo.ColorInfo.Encoding);
			if (ApvColorTransfer.IsSet())
			{
				PrimaryFrameParams.transfer_characteristics = ToUnderlyingType(ApvColorTransfer.GetValue());
			}
			else
			{
				UE_LOGF(LogApvMedia, Warning,
					"Encoder: Requested transfer function \"%ls\" doesn't exist in Apv, will remain \"unspecified\".",
					*GetEnumNameStringByValue<ETextureSourceEncoding>(ToUnderlyingType(InMipInfo.ColorInfo.Encoding)));
			}

			const TOptional<EApvMediaColorSpace> ApvColorSpace = GetApvColorSpace(InMipInfo.ColorInfo.ColorSpace);
			if (ApvColorSpace.IsSet())
			{
				PrimaryFrameParams.color_primaries = ToUnderlyingType(ApvColorSpace.GetValue());
			}
			else
			{
				UE_LOGF(LogApvMedia, Warning,
					"Encoder: Requested color space \"%ls\" doesn't exist in Apv, will remain \"unspecified\".",
					*GetEnumNameStringByValue<ETextureColorSpace>(ToUnderlyingType(InMipInfo.ColorInfo.ColorSpace)));
			}

			PrimaryFrameParams.matrix_coefficients = ToUnderlyingType(GetApvColorMatrix(InMipInfo.ColorInfo.YuvMatrix));
			PrimaryFrameParams.full_range_flag = ToUnderlyingType(GetApvColorMatrixRange(InMipInfo.ColorInfo.YuvMatrixRange));
		}

		// If the primary frame is already at the smallest possible size, don't add any extra mip frames.
		if (InMipInfo.bEnableMips && (PrimaryFrameParams.w > 1 || PrimaryFrameParams.h > 1))
		{
			const EApvMediaChromaFormat ChromaFormat = InOptions.GetChromaFormat();
			
			const int32 MaxNumMips = FTmvMediaUtils::GetMaxMipCountFromDimensions(PrimaryFrameParams.w, PrimaryFrameParams.h);

			// Note: start at frame index 1 because we have already done frame 0 as PrimaryFrame.
			for (int32 MipIndex = 1; MipIndex < MaxNumMips; ++MipIndex)
			{
				const int32 MipWidth = FMath::Max(1, PrimaryFrameParams.w >> MipIndex);
				const int32 MipHeight = FMath::Max(1, PrimaryFrameParams.h >> MipIndex);

				if (GetSubWidthC(ChromaFormat) == 2 && MipWidth & 0x1)
				{
					UE_LOGF(LogApvMedia, Warning, "Can't generate mip of width %d. Not a multiple of two (YUC 422/420 constraint)", MipWidth);
					break;
				}

				if (GetSubHeightC(ChromaFormat) == 2 && MipHeight & 0x1)
				{
					UE_LOGF(LogApvMedia, Warning, "Can't generate mip of height %d. Not a multiple of two (YUC 420 constraint)", MipHeight);
					break;
				}

				const int32 FrameIndex = EncoderDescriptor.max_num_frms;

				if (FrameIndex >= OAPV_MAX_NUM_FRAMES)
				{
					UE_LOGF(LogApvMedia, Warning, "Reached OpenApv encoder maximum number of frames (%d); stopping mip generation.", OAPV_MAX_NUM_FRAMES); 
					break;
				}

				EncoderDescriptor.param[FrameIndex] = PrimaryFrameParams;	// Copy primary frame parameters
				EncoderDescriptor.param[FrameIndex].w = MipWidth;
				EncoderDescriptor.param[FrameIndex].h = MipHeight;
				++EncoderDescriptor.max_num_frms;
			}
		}

		EncoderDescriptor.threads = GetValidatedNumThreads(InOptions.NumThreads);
		EncoderDescriptor.max_bs_buf_size = GetDefault<UApvMediaSettings>()->MaxEncoderBitStreamBufferSize;

		// Report invalid bitstream buffer size.
		if (EncoderDescriptor.max_bs_buf_size <= 0)
		{
			FText Message = FText::Format(
				LOCTEXT("InvalidBitstreamSize", "Invalid MaxEncoderBitStreamBufferSize {0}. Recommended 128 MB"), FText::AsNumber(EncoderDescriptor.max_bs_buf_size));
			UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("ApvEncoderContext"), Message);
			return;
		}

		int Err = 0;
		eid = oapve_create(&EncoderDescriptor, &Err);
		if (eid == nullptr)
		{
			FText Message = FText::Format(
				LOCTEXT("FailedCreateEncoder", "Cannot create OpenApv encoder, error: {0} ({1})"), 
				FText::FromString(GetApvErrorString(Err)), FText::AsNumber(Err));
			UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("ApvEncoderContext"), Message);
		}
		else
		{
			UE_LOGF(LogApvMedia, Log,
				"ApvEncoderContext: created OpenApv encoder (v%ls) with %d worker threads",
				StringCast<TCHAR>(oapv_version(nullptr)).Get(), EncoderDescriptor.threads);
		}

		mid = oapvm_create(&Err);
		if(OAPV_FAILED(Err))
		{
			FText Message = FText::Format(
				LOCTEXT("FailedCreateMetadata", "Cannot create OpenApv metadata container, error: {0} ({1})"), 
				FText::FromString(GetApvErrorString(Err)), FText::AsNumber(Err));
			UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("ApvEncoderContext"), Message);
		}
	}

	FApvEncoderContext::~FApvEncoderContext()
	{
		if (eid != nullptr)
		{
			oapve_delete(eid);
			eid = nullptr;
		}
		if (mid != nullptr)
		{
			oapvm_delete(mid);
			mid = nullptr;
		}
	}
}

#undef LOCTEXT_NAMESPACE