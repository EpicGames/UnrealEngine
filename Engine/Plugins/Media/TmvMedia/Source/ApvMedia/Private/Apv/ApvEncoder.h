// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ApvCommon.h"
#include "ApvDecoder.h"

class FTmvMediaMessageContext;
struct FApvMediaTmvEncoderOptions;
struct FTmvMediaEncoderMipInfo;
struct FTmvMediaFrameTimeInfo;

namespace UE::ApvMedia
{
	/**
	 * Utility wrapper for OpenApv Encoder context.
	 */
	struct FApvEncoderContext
	{
		/**
		 * Construct the OpenApv encoder context.
		 * @param InOptions Encoder options (user specified)
		 * @param InMipInfo Primary Mip (mip 0) descriptor
		 * @param InTimeInfo Only Time info's frame rate is used as a hint for bit rate calculations.
		 * @param OutMessageContext Optional message context for the given operation.
		 * @remark Sequence's frame rate can't be stored by OpenApv frame header (not supported). 
		 */
		FApvEncoderContext(
			const FApvMediaTmvEncoderOptions& InOptions,
			const FTmvMediaEncoderMipInfo& InMipInfo,
			const FTmvMediaFrameTimeInfo& InTimeInfo,
			FTmvMediaMessageContext* OutMessageContext);

		// Non-copiable
		FApvEncoderContext(const FApvEncoderContext& InOther) = delete;
		FApvEncoderContext& operator=(const FApvEncoderContext& InOther) = delete;
		
		~FApvEncoderContext();

		/** returns true if the context is valid. */
		bool IsValid() const
		{
			return eid != nullptr && mid != nullptr;
		}

		/** OpenApv encoder context */
		oapve_t eid = nullptr;

		/** OpenApv metadata container */
		oapvm_t mid = nullptr;
		
		/** Contains the frame buffers the encoder will read from. */
		FApvFrames Frames;
	};
}