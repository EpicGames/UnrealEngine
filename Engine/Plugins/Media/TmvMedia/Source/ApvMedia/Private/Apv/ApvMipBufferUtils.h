// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ApvDecoder.h"
#include "ApvMediaLog.h"
#include "ApvMediaTmvEncoderOptions.h"
#include "ColorManagement/ColorManagementDefines.h"
#include "Misc/Optional.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include <type_traits>

namespace UE::ApvMedia
{
	/**
	 * Determines the Apv original color format from the setup of the mip color planes. 
	 */
	inline int32 GetApvColorFormat(const FTmvMediaFrameMipInfo& InMipInfo)
	{
		switch (InMipInfo.Planes.Num())
		{
		case 1:
			return OAPV_CF_YCBCR400;
		case 2:
			return OAPV_CF_PLANAR2;
		case 3:
			// Infer subsampling: 420, 422 0r 444
			if (InMipInfo.Planes[1].Height == InMipInfo.Height) // 422, 444
			{
				if (InMipInfo.Planes[1].Width == InMipInfo.Width)
				{
					return OAPV_CF_YCBCR444;
				}
				return OAPV_CF_YCBCR422;	
			}
			return OAPV_CF_YCBCR420;
		case 4:
			return OAPV_CF_YCBCR4444;
		default:
			UE_LOGF(LogApvMedia, Error,
				"GetApvColorFormat: Unsupported number of planes (%d).", InMipInfo.Planes.Num());
			return OAPV_CF_UNKNOWN;
		}
	}

	/** Safely gets the bit depth from plane 0. */
	inline uint8 GetBitDepth(const FTmvMediaFrameMipInfo& InMipInfo)
	{
		return !InMipInfo.Planes.IsEmpty() ? InMipInfo.Planes[0].BitDepth : 0;
	}

	/** Determines the original color space from the setup of the mip color planes. */
	inline int32 GetApvColorSpace(const FTmvMediaFrameMipInfo& InMipInfo)
	{
		return OAPV_CS_SET(GetApvColorFormat(InMipInfo), GetBitDepth(InMipInfo), 0);
	}

	/** Wrap a FrameMipBuffer in an Apv Image buffer structure. */
	inline oapv_imgb_t* ApvSetupImageBuffer(const FTmvMediaFrameMipInfo& InMipInfo, FTmvMediaFrameMipBuffer& InResourceBuffer)
	{
		TUniquePtr<FApvImageBitmap> OutFrame = MakeUnique<FApvImageBitmap>();
		if (OutFrame->Init(InMipInfo.Width, InMipInfo.Height, GetApvColorSpace(InMipInfo), /*bAllocate*/ false))
		{
			for (int i = 0; i < OutFrame->np; ++i)
			{
				// GetPlaneBufferForComponent returns:
				//   - scanline: pointer to the start of plane i in the mip buffer
				//   - tiled:    pointer to the intra-tile offset of plane i in tile 0
				//               (Planes[] describes per-tile layout in tiled mode, so
				//               GetPlaneBufferOffset() yields the intra-tile offset)
				// Either pointer is the correct base for the decoder's per-row addressing.
				OutFrame->a[i] = InResourceBuffer.GetPlaneBufferForComponent(i);
				if (!OutFrame->a[i])
				{
					UE_LOGF(LogApvMedia, Warning,
						"ApvSetupImageBuffer: GetPlaneBufferForComponent(%d) returned null for mip=%d - decode will be skipped (this frame's tile bytes won't reach the buffer; shader will read stale/uninit slot bytes if rendered before next decode lands).",
						i, InMipInfo.MipLevel);
					return nullptr;	// don't let an invalid memory frame be used.
				}
			}

			if (InMipInfo.Layout == ETmvMediaFrameBufferLayout::Tiled)
			{
				// Tile-major layout: tell the decoder to write tiles contiguously.
				// `tile_size` is the per-tile byte stride (== all-plane size of a single
				// tile, since planes are interleaved inside each tile); `tile_stride[c]`
				// is the per-row byte advance within plane c of a tile.
				OutFrame->tiled_layout  = 1;
				OutFrame->num_tile_cols = InMipInfo.NumTiles.X;
				OutFrame->num_tile_rows = InMipInfo.NumTiles.Y;
				OutFrame->tile_size     = static_cast<int>(InMipInfo.GetAllPlaneMemorySizeInBytes());
				for (int c = 0; c < OutFrame->np && c < OAPV_MAX_CC && c < InMipInfo.Planes.Num(); ++c)
				{
					const FTmvMediaFramePlaneInfo& PlaneInfo = InMipInfo.Planes[c];
					// PlaneInfo.NumLines holds the tile height in samples (set in
					// ApvMediaTmvDecoder::CreateMipInfos), PlaneInfo.Stride is the
					// per-row byte advance within a tile.
					OutFrame->tile_h[c]      = PlaneInfo.NumLines;
					OutFrame->tile_stride[c] = static_cast<int>(PlaneInfo.Stride);
					// Derive tile width in component samples from the row stride.
					const int32 BytesPerComp = (PlaneInfo.BitDepth + 7) / 8;
					OutFrame->tile_w[c]      = (BytesPerComp > 0)
						? (static_cast<int>(PlaneInfo.Stride) / BytesPerComp) : 0;
				}
			}
			return OutFrame.Release();
		}
		return nullptr;
	}

	/**
	 * Enum conversion utility:
	 * Converts an enum value to it's underlying type for OpenApv struct fields. 
	 */
	template<typename Enum>
	constexpr auto ToUnderlyingType(Enum Val)
	{
		return static_cast<std::underlying_type_t<Enum>>(Val);
	}

	/**
	 * Convert UE encoding to OpenApv color transfer if possible.
	 * @remark potential match only. (wip)
	 */
	inline TOptional<EApvMediaColorTransfer> GetApvColorTransfer(UE::Color::EEncoding InEncoding)
	{
		// Note: MediaShaders.usf only support sRGB, Bt2084 and SLog3 for now.
		using namespace Color;
		switch (InEncoding)
		{
		case EEncoding::None:
			return EApvMediaColorTransfer::Unspecified;

		case EEncoding::Linear:
			return EApvMediaColorTransfer::Linear;

		case EEncoding::sRGB:
			return EApvMediaColorTransfer::Bt709;

		// SMPTE ST 2084/PQ
		case EEncoding::ST2084:
			return EApvMediaColorTransfer::Smpte2084;

		case EEncoding::Gamma22:
			return EApvMediaColorTransfer::Bt709;

		// BT1886/Gamma 2.4
		//case EEncoding::BT1886: // unmatched

		//case EEncoding::Gamma26: // unmatched

		//case EEncoding::Cineon: // unmatched

		//case EEncoding::REDLog: // unmatched

		//case EEncoding::REDLog3G10: // unmatched

		// Sony SLog1
		//case EEncoding::SLog1: // unmatched

		// Sony SLog2
		//case EEncoding::SLog2: // unmatched

		// Sony SLog3
		case EEncoding::SLog3:
			return EApvMediaColorTransfer::Log316; // todo: verify the equation, could be 9, 10, or unmatched.

		//case EEncoding::AlexaV3LogC: // ARRI Alexa V3 LogC -- unmatched

		//case EEncoding::CanonLog: // unmatched

		//case EEncoding::ProTune: // GoPro ProTune -- unmatched

		//case EEncoding::VLog: // Panasonic V-Log -- unmatched

		case EEncoding::HLG:
			return EApvMediaColorTransfer::Arib_std_b67;

		default:
			return TOptional<EApvMediaColorTransfer>();
		}
	}

	/**
	 * Convert OpenApv color transfer to UE encoding.
	 * @remark Must match GetApvColorTransfer above.
	 */
	inline Color::EEncoding GetColorEncoding(EApvMediaColorTransfer InApvTransfer)
	{
		// Only return matched enums from GetApvColorTransfer.
		using namespace Color;
		switch (InApvTransfer)
		{
		case EApvMediaColorTransfer::Unspecified:
			return EEncoding::None;

		case EApvMediaColorTransfer::Linear:
			return EEncoding::Linear;

		case EApvMediaColorTransfer::Bt709:
			return EEncoding::sRGB;

		case EApvMediaColorTransfer::Smpte2084:
			return EEncoding::ST2084;

		case EApvMediaColorTransfer::Log316:
			return EEncoding::SLog3;

		case EApvMediaColorTransfer::Arib_std_b67:
			return EEncoding::HLG;

		default:
			return EEncoding::sRGB;
		}
	}

	/**
	 * Convert UE color space to OpenApv color space if possible.
	 * @remark potential match only. (wip)
	 */
	inline TOptional<EApvMediaColorSpace> GetApvColorSpace(Color::EColorSpace InColorSpace)
	{
		using namespace Color;
		switch (InColorSpace)
		{
		case EColorSpace::None:
			return EApvMediaColorSpace::Unspecified;

		// sRGB / Rec709 (BT.709) color primaries, with D65 white point.
		case EColorSpace::sRGB:
			return EApvMediaColorSpace::Bt709;

		// Rec2020 (BT.2020) primaries with D65 white point.
		case EColorSpace::Rec2020:
			return EApvMediaColorSpace::Bt2020;

		// ACES AP0 wide gamut primaries, with D60 white point.
		//case EColorSpace::ACESAP0: // Unmatched

		// ACES AP1 / ACEScg wide gamut primaries, with D60 white point.
		//case EColorSpace::ACESAP1: // Unmatched

		// P3 (Theater) primaries, with DCI Calibration white point.
		//case EColorSpace::P3DCI: // Unmatched

		// P3 (Display) primaries, with D65 white point.
		//case EColorSpace::P3D65: // Unmatched

		// RED Wide Gamut primaries, with D65 white point.
		//case EColorSpace::REDWideGamut: // Unmatched

		// Sony S-Gamut/S-Gamut3 primaries, with D65 white point.
		//case EColorSpace::SonySGamut3: // Unmatched

		// Sony S-Gamut3 Cine primaries, with D65 white point.
		//case EColorSpace::SonySGamut3Cine: // Unmatched

		// Alexa Wide Gamut primaries, with D65 white point.
		//case EColorSpace::AlexaWideGamut: // Unmatched

		// Canon Cinema Gamut primaries, with D65 white point.
		//case EColorSpace::CanonCinemaGamut: // Unmatched

		// GoPro Protune Native primaries, with D65 white point.
		//case EColorSpace::GoProProtuneNative: // Umatched

		// Panasonic V-Gamut primaries, with D65 white point.
		//case EColorSpace::PanasonicVGamut: // Unmatched

		default:
			return TOptional<EApvMediaColorSpace>();
		}
	}

	/**
	 * Convert OpenApv color space to UE color space.
	 * @remark must match GetApvColorSpace above.
	 */
	inline Color::EColorSpace GetColorSpace(EApvMediaColorSpace InApvColorSpace)
	{
		// Only return matched enums from GetApvColorSpace.
		using namespace Color;
		switch (InApvColorSpace)
		{
		case EApvMediaColorSpace::Unspecified:
			return EColorSpace::None;

		case EApvMediaColorSpace::Bt709:
			return EColorSpace::sRGB;

		case EApvMediaColorSpace::Bt2020:
			return EColorSpace::Rec2020;

		default:
			return EColorSpace::sRGB;
		}
	}

	/**
	 * Convert Tmv (Yuv) color matrix to OpenApv color matrix.
	 */
	inline EApvMediaColorMatrix GetApvColorMatrix(ETmvMediaFrameColorMatrix InColorMatrix)
	{
		switch (InColorMatrix)
		{
		case ETmvMediaFrameColorMatrix::None:
			return EApvMediaColorMatrix::Unspecified;
		
		case ETmvMediaFrameColorMatrix::Identity:
			return EApvMediaColorMatrix::Gbr;
			
		case ETmvMediaFrameColorMatrix::Rec601:
			return EApvMediaColorMatrix::Bt470bg;

		case ETmvMediaFrameColorMatrix::Rec709:
			return EApvMediaColorMatrix::Bt709;

		case ETmvMediaFrameColorMatrix::Rec2020:
			return EApvMediaColorMatrix::Bt2020c;

		default:
			return EApvMediaColorMatrix::Bt709;
		}
	}

	/**
	 * Convert OpenApv color matrix enum to Tmv (Yuv) color matrix.
	 */
	inline ETmvMediaFrameColorMatrix GetColorMatrix(EApvMediaColorMatrix InApvColorMatrix)
	{
		switch (InApvColorMatrix)
		{
		case EApvMediaColorMatrix::Unspecified:
			return ETmvMediaFrameColorMatrix::None;
			
		case EApvMediaColorMatrix::Gbr:
			return ETmvMediaFrameColorMatrix::Identity;

		case EApvMediaColorMatrix::Bt470bg:
		case EApvMediaColorMatrix::Smpte170m:
			return ETmvMediaFrameColorMatrix::Rec601;

		case EApvMediaColorMatrix::Bt709:
			return ETmvMediaFrameColorMatrix::Rec709;
			
		case EApvMediaColorMatrix::Bt2020nc:
		case EApvMediaColorMatrix::Bt2020c:
			return ETmvMediaFrameColorMatrix::Rec2020;
			
		default:
			return ETmvMediaFrameColorMatrix::Rec709;
		}
	}

	/** Convert matrix coefficient range to Apv enum. */
	inline EApvMediaColorMatrixRange GetApvColorMatrixRange(ETmvMediaFrameColorMatrixRange InColorMatrixRange)
	{
		return InColorMatrixRange == ETmvMediaFrameColorMatrixRange::Limited ? EApvMediaColorMatrixRange::Limited : EApvMediaColorMatrixRange::Full;
	}

	/** Convert matrix coefficient range to Tmv enum. */
	inline ETmvMediaFrameColorMatrixRange GetColorMatrixRange(EApvMediaColorMatrixRange InApvColorMatrixRange)
	{
		return InApvColorMatrixRange == EApvMediaColorMatrixRange::Limited ? ETmvMediaFrameColorMatrixRange::Limited : ETmvMediaFrameColorMatrixRange::Full;
	}
}

