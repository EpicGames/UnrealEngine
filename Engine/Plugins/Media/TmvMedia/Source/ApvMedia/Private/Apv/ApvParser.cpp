// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApvParser.h"
#include "ApvMediaLog.h"
#include "HAL/FileManager.h"

namespace UE::ApvMedia
{
	FApvBitReader::FApvBitReader(const FString& InFilename, FArchive* InArchive)
		: Filename(InFilename)
		, FileReader(InArchive)
	{
		if (!InArchive)
		{
			OwnedFileReader.Reset(IFileManager::Get().CreateFileReader(*InFilename));
			FileReader = OwnedFileReader.Get();
		}
	}

	FApvBitReader::~FApvBitReader()
	{
		if (OwnedFileReader)
		{
			OwnedFileReader->Close();
		}
	}

	bool FApvParser::ParseFrameInfo(FApvBitReader& InStream, const FApvAccessUnitHeader& InAccessUnitHeader, TArray<FApvParserFrameHeader>& OutFrameHeaders)
	{
		if (InStream.AtEnd() || !InStream.IsValid())
		{
			UE_LOGF(LogApvMedia, Error, "FApvParser::ParseFrameInfo: No more data to read.");
			return false;
		}

		// Calculate the end of the access unit.
		int64 AccessUnitEnd = InAccessUnitHeader.StartOffset + InAccessUnitHeader.Size;

		uint32 Signature = 0;

		if (AccessUnitEnd - InStream.Tell() < static_cast<int32>(sizeof(Signature)))
		{
			UE_LOGF(LogApvMedia, Error, "FApvParser::ParseFrameInfo: Not enough data to read signature.");
			return false;
		}

		InStream << Signature;

		if (Signature != Apv1Signature)
		{
			UE_LOGF(LogApvMedia, Error, "FApvParser::ParseFrameInfo: Unexpected access unit signature.");
			return false;
		}

		constexpr int32 SizeOfPbuSize = sizeof(int32);

		// Parse PBUs within this access unit.
		while (!InStream.AtEnd() && InStream.Tell() + SizeOfPbuSize < AccessUnitEnd)
		{
			int32 PbuSize = 0;
			InStream << PbuSize;

			const int64 PbuStartOffset = InStream.Tell();

			if (PbuSize < 0 || (PbuSize + PbuStartOffset) > AccessUnitEnd)
			{
				UE_LOGF(LogApvMedia, Error, "FApvParser::ParseFrameInfo: Unexpected pbu size (%d).", PbuSize);
				return false;
			}
			
			// Pbu header
			FApvPbuHeader PbuHeader;
			PbuHeader.Read(InStream);
				
			// We only care about primary frame for now.
			if (PbuHeader.Type == OAPV_PBU_TYPE_PRIMARY_FRAME || PbuHeader.Type == OAPV_PBU_TYPE_NON_PRIMARY_FRAME)
			{
				FApvParserFrameHeader ParserFrameHeader;
				ParserFrameHeader.PbuStartOffset = PbuStartOffset;
				ParserFrameHeader.PbuSize = PbuSize;
				ParserFrameHeader.GroupId = PbuHeader.GroupId;
				ParserFrameHeader.FrameHeader.Read(InStream);
				OutFrameHeaders.Add(MoveTemp(ParserFrameHeader));
			}
			
			// Skip the remainder of the pbu.
			InStream.Seek(PbuStartOffset + PbuSize);
		}

		return !OutFrameHeaders.IsEmpty();
	}
}