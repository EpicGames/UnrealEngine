// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ApvCommon.h"

#include "Containers/UnrealString.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"
#include "Math/IntPoint.h"

struct FImgMediaFrameInfo;

namespace UE::ApvMedia
{
	/**
	 * Utility class to read bit stream out of an FArchive.
	 * Adapted from the oapv_bs (bitstream) code.
	 * Seems to be similar to FBitReader, except it is backed directly by an FArchive.
	 * 
	 * Benefits:
	 * - The bitstream can be parsed without having to load the whole buffer.
	 *   It is limited to parsing a int32 at a time (for simplicity), but the oapv parsing code doesn't need more than that.
	 * - Supports seeking the underlying archive and pick up bitstream parsing at any point.
	 * - It is also possible to use interchangeably with the oapv implementation, however it requires reading the
	 *   whole buffer in memory and is thus undesirable in the context of TMV.
	 */
	class FApvBitReader
	{
	public:
		/**
		 * Construct the bit reader.
		 * @param InFileName filename of the file to read from.
		 * @param InArchive Archive to use instead of opening one.
		 */
		explicit FApvBitReader(const FString& InFilename, FArchive* InArchive = nullptr);

		~FApvBitReader();

		bool IsValid() const
		{
			return FileReader != nullptr;
		}

		int64 Tell() const
		{
			return FileReader ? FileReader->Tell() - (LeftBits >> 3) : 0;
		}

		int64 TotalSize() const
		{
			return FileReader ? FileReader->TotalSize() : 0;
		}

		bool AtEnd() const
		{
			return FileReader ? (FileReader->AtEnd() && LeftBits == 0) : false;
		}

		void Seek(int64 InOffset)
		{
			if (FileReader)
			{
				FileReader->Seek(InOffset);
				Code = 0;
				LeftBits = 0;
			}
		}

		bool Flush(int32 InNumBytes)
		{
			if (InNumBytes <= 0 || InNumBytes > 4)
			{
				Code = 0;
				LeftBits = 0;
				return false;
			}

			if (!FileReader)
			{
				return false;
			}
			
			int64 Remaining = FileReader->TotalSize() - FileReader->Tell();
			if (static_cast<int64>(InNumBytes) > Remaining)
			{
				InNumBytes = static_cast<int32>(Remaining);
			}

			uint32 Shift = 24;
			LeftBits = InNumBytes << 3;
			Code = 0;
			while (InNumBytes)
			{
				uint8 ReadByte = 0;
				*FileReader << ReadByte;
				Code |= static_cast<uint32>(ReadByte) << Shift;
				InNumBytes--;
				Shift -= 8;
			}
			return true;
		}

		void Align8()
		{
			int32 Size = LeftBits & 0x7;
			Code <<= Size;
			LeftBits -= Size;
		}

		void SkipCode(int32 InNumBits)
		{
			if (InNumBits >= 32)
			{
				Code = 0;
				LeftBits = 0;
			}
			else
			{
				Code <<= InNumBits;
				LeftBits -= InNumBits;
			}
		}

		void Skip(int32 InNumBits)
		{
			check(InNumBits > 0);
			check(InNumBits <= 32);
			
			if (LeftBits < InNumBits)
			{
				InNumBits -= LeftBits;
				if(!Flush(4))
				{
					return;
				}
			}

			SkipCode(InNumBits);
		}

		uint32 ReadBits(int32 InNumBits)
		{
			uint32 OutCode = 0;
			if (LeftBits < InNumBits)
			{
				OutCode = Code >> (32 - InNumBits);
				InNumBits -= LeftBits;
				if (!Flush(4))
				{
					return static_cast<uint32>(-1);	// error state
				}
			}

			OutCode |= Code >> (32 - InNumBits);

			SkipCode(InNumBits);

			return OutCode;
		}

		template<typename T>
		void ReadValue(T& OutValue)
		{
			OutValue = static_cast<T>(ReadBits(sizeof(T)*8)); 
		}

		template<typename T>
		friend FApvBitReader& operator<<(FApvBitReader& InBitStream, T& OutValue)
		{
			OutValue = InBitStream.ReadBits(sizeof(T)*8);
			return InBitStream;
		}

		/** Loaded file name */
		FString Filename;
	
		/** File reader */
		FArchive* FileReader;

	private:
		/** Intermediate code buffer. */
		uint32 Code = 0;

		/** Number of bits left in Code. */
		int32 LeftBits = 0;

		/** File reader internally opened. */
		TUniquePtr<FArchive> OwnedFileReader;
	};

	static constexpr uint32 Apv1Signature = 0x61507631;	// "aPv1"

	struct FApvAccessUnitHeader
	{
		int32 Size = 0;
		int64 StartOffset = 0;

		void Read(FApvBitReader& InBitStream)
		{
			InBitStream << Size;

			// Keep track of current location
			StartOffset = InBitStream.Tell();
		}
	};

	struct FApvPbuHeader
	{
		int8 Type = 0;
		int16 GroupId = 0;

		void Read(FApvBitReader& InBitStream)
		{
			InBitStream << Type;
			InBitStream << GroupId;
			InBitStream.Skip(8); // Skip reserved 8 bits
		}
	};

	struct FApvFrameInfo
	{
		uint8 ProfileIdc;
		uint8 LevelIdc;
		uint8 BandIdc;
		uint32 Width;
		uint32 Height;
		uint8 ChromaFormatIdc;	// See: https://www.ietf.org/archive/id/draft-lim-apv-04.html#_table-chroma_format_idc
		uint8 BitDepth;
		uint8 CaptureTimeDistance;
		
		void Read(FApvBitReader& InBitStream)
		{
			InBitStream << ProfileIdc;
			InBitStream << LevelIdc;
			BandIdc = InBitStream.ReadBits(3);
			InBitStream.Skip(5);	// Skip reserved 5 bits.
			Width = InBitStream.ReadBits(24);
			Height = InBitStream.ReadBits(24);
			ChromaFormatIdc = InBitStream.ReadBits(4);
			BitDepth = InBitStream.ReadBits(4) + 8;	// stored as depth - 8
			InBitStream << CaptureTimeDistance;
			InBitStream.Skip(8); // Skip reserved 8 bits
		}
		
		int32 GetNumComponents() const
		{
			switch (ChromaFormatIdc)
			{
			case 0: return 1;	// 4:0:0
			case 4: return 4;	// 4:4:4:4 ?
			default: return 3;	// 4:2:0, 4:2:2 or 4:4:4
			}
		}

		int32 GetColorFormat() const
		{
			switch (ChromaFormatIdc)
			{
			case 0: return OAPV_CF_YCBCR400;
			case 1: return OAPV_CF_YCBCR420;
			case 2: return OAPV_CF_YCBCR422;
			case 3: return OAPV_CF_YCBCR444;
			case 4: return OAPV_CF_YCBCR4444;
			default: return OAPV_CF_UNKNOWN;				
			}
		}

		/** In oapv, "Colorspace" if a FrameInfo is defined as the combo of bit depth and color format. */
		int32 GetColorSpace() const
		{
			// Reference: fi_to_finfo
			int32 ColorSpace = OAPV_CS_SET(GetColorFormat(), static_cast<int32>(BitDepth), 0);
			return ColorSpace;
		}

		/** Align the given width to Macroblock dimensions. */
		static inline int32 AlignWidthToMB(int32 InWidth)
		{
			return ((InWidth + (OAPV_MB_W - 1)) >> OAPV_LOG2_MB_W) << OAPV_LOG2_MB_W;
		}

		/** Align the given height to Macroblock dimensions. */
		static inline int32 AlignHeightToMB(int32 InHeight)
		{
			return ((InHeight + (OAPV_MB_H - 1)) >> OAPV_LOG2_MB_H) << OAPV_LOG2_MB_H;
		}

		/**
		 * Return the given component's plane width.
		 * @param InComponent Specify the component
		 * @param bInMBAligned Indicate if the value is to be aligned to Macroblock. 
		 */
		int32 GetPlaneWidth(int32 InComponent, bool bInMBAligned) const
		{
			// Luminance component is full size.
			int32 PlaneWidth = (InComponent == 0) ? Width : Width / GetSubWidthC();
			return bInMBAligned ? AlignWidthToMB(PlaneWidth) : PlaneWidth;
		}

		/** Get the Macro block aligned height. */
		int32 GetPlaneHeight(int32 InComponent, bool bInMBAligned) const
		{
			// Luminance component is full size.
			int32 PlaneHeight = (InComponent == 0) ? Height : Height / GetSubHeightC();
			return bInMBAligned ? AlignHeightToMB(PlaneHeight) : PlaneHeight;
		}

		/** Get the full frame's width aligned to macroblock. */
		int32 GetWidth_MBAligned() const
		{
			return AlignWidthToMB(Width);
		}

		/** Get the full frame's height aligned to macroblock. */
		int32 GetHeight_MBAligned() const
		{
			return AlignHeightToMB(Height);
		}
		
		int32 GetSubWidthC() const
		{
			// 420 or 422
			return (ChromaFormatIdc == 1 || ChromaFormatIdc == 2) ? 2 : 1;
		}

		int32 GetSubHeightC() const
		{
			// 420
			return ChromaFormatIdc == 1 ? 2 : 1;
		}

		/** Utility to convert the parsed frame info back to the oapv type for inter op. */
		void ToFrameInfo(const FApvPbuHeader& InPbuHeader, oapv_frm_info_t& OutFrameInfo) const
		{
			OutFrameInfo.w = static_cast<int>(Width);
			OutFrameInfo.h = static_cast<int>(Height);
			OutFrameInfo.cs = OAPV_CS_SET(GetColorFormat(), BitDepth, 0);
			OutFrameInfo.pbu_type = InPbuHeader.Type;
			OutFrameInfo.group_id = InPbuHeader.GroupId;
			OutFrameInfo.profile_idc = ProfileIdc;
			OutFrameInfo.level_idc = LevelIdc;
			OutFrameInfo.band_idc = BandIdc;
			OutFrameInfo.chroma_format_idc = ChromaFormatIdc;
			OutFrameInfo.bit_depth = BitDepth;
			OutFrameInfo.capture_time_distance = CaptureTimeDistance;
		}

		void SetChromaFormatFromColorSpace(int InColorSpace)
		{
			switch(OAPV_CS_GET_FORMAT(InColorSpace))
			{
			case OAPV_CF_YCBCR400: ChromaFormatIdc = 0; break;
			case OAPV_CF_YCBCR420: ChromaFormatIdc = 1; break;
			case OAPV_CF_YCBCR422: ChromaFormatIdc = 2; break;
			case OAPV_CF_YCBCR444: ChromaFormatIdc = 3; break;
			case OAPV_CF_YCBCR4444: ChromaFormatIdc = 4; break;
			default: ChromaFormatIdc = 0; break;
			}
		}

		void FromFrameInfo(const oapv_frm_info_t& InFrameInfo)
		{
			ProfileIdc = InFrameInfo.profile_idc;
			LevelIdc = InFrameInfo.level_idc;
			BandIdc = InFrameInfo.band_idc;
			Width = InFrameInfo.w;
			Height = InFrameInfo.h;
			SetChromaFormatFromColorSpace(InFrameInfo.cs);	
			BitDepth = InFrameInfo.bit_depth;
			CaptureTimeDistance = InFrameInfo.capture_time_distance;
		}
	};

	struct FApvColorDescription
	{
		uint8 ColorPrimaries = 2;	// Unspecified
		uint8 TransferCharacteristic = 2; // Unspecified
		uint8 MatrixCoefficients = 2; // Unspecified
		uint8 FullRangeFlag = 0;

		void Read(FApvBitReader& InBitStream)
		{
			InBitStream << ColorPrimaries;
			InBitStream << TransferCharacteristic;
			InBitStream << MatrixCoefficients;
			FullRangeFlag = InBitStream.ReadBits(1);
		}
	};

	struct FApvTileInfo
	{
		int32 TileWidthInMbs = 0;
		int32 TileHeightInMbs = 0;
		int8 TileSizePresentInHeaderFlag = 0;
		TArray<uint32> TileSizes;

		void Read(const FApvFrameInfo& InFrameInfo, FApvBitReader& InBitStream)
		{
			// dec_vlc_tile_info
			TileWidthInMbs = InBitStream.ReadBits(20);
			TileHeightInMbs = InBitStream.ReadBits(20);
			TileSizePresentInHeaderFlag = InBitStream.ReadBits(1);

			if (TileSizePresentInHeaderFlag)
			{
				const FIntPoint NumTiles = GetNumTiles(InFrameInfo);
				// Clamp to OpenAPV limits to prevent overflow and large allocations.
				const int32 TotalTiles = FMath::Min(NumTiles.X * NumTiles.Y, OAPV_MAX_TILES);				
				TileSizes.SetNumUninitialized(TotalTiles);

				for (uint32& TileSize : TileSizes)
				{
					if (InBitStream.AtEnd())
					{
						break;
					}
					TileSize = InBitStream.ReadBits(32);
				}
			}
		}

		int32 GetTileWidthInSamples() const
		{
			return TileWidthInMbs * OAPV_MB_W;
		}

		int32 GetTileHeightInSamples() const
		{
			return TileHeightInMbs * OAPV_MB_H;
		}

		FIntPoint GetTileSizeInSamples() const
		{
			return FIntPoint(GetTileWidthInSamples(), GetTileHeightInSamples());
		}

		FIntPoint GetNumTiles(const FApvFrameInfo& InFrameInfo) const
		{
			const int32 TileWidth = GetTileWidthInSamples();
			const int32 TileHeight = GetTileHeightInSamples();

			if (TileWidth != 0 && TileHeight != 0)
			{
				const int32 FrameWidth = ((InFrameInfo.Width + (OAPV_MB_W - 1)) >> OAPV_LOG2_MB_W) << OAPV_LOG2_MB_W;
				const int32 FrameHeight = ((InFrameInfo.Height + (OAPV_MB_H - 1)) >> OAPV_LOG2_MB_H) << OAPV_LOG2_MB_H;

				// Clamp to OpenAPV limits to prevent overflow and large allocations.
				const int32 NumTileColumns = FMath::Clamp((FrameWidth + (TileWidth - 1)) / TileWidth, 1, OAPV_MAX_TILE_COLS);
				const int32 NumTileRows = FMath::Clamp((FrameHeight + (TileHeight - 1)) / TileHeight, 1, OAPV_MAX_TILE_ROWS);
				return FIntPoint(NumTileColumns, NumTileRows);
			}
			return FIntPoint(1, 1);
		}
	};

	// https://www.ietf.org/archive/id/draft-lim-apv-04.html#name-frame-header
	struct FApvFrameHeader
	{	
		FApvFrameInfo FrameInfo;
		uint8 ColorDescriptionPresentFlag = 0;
		FApvColorDescription ColorDescription;
		uint8 UseQMatrix = 0;
		uint8 QMatrix[OAPV_MAX_CC][OAPV_BLK_H][OAPV_BLK_W];
		FApvTileInfo TileInfo;

		void Read(FApvBitReader& InBitStream)
		{
			FrameInfo.Read(InBitStream);
			InBitStream.Skip(8);	// Skip reserved 8 bit
			
			ColorDescriptionPresentFlag = InBitStream.ReadBits(1);
			if (ColorDescriptionPresentFlag)
			{
				ColorDescription.Read(InBitStream);
			}
			else
			{
				ColorDescription = FApvColorDescription();
			}
			UseQMatrix = InBitStream.ReadBits(1);
			if (UseQMatrix)
			{
				ReadQMatrix(InBitStream);
			}
			else
			{
				InitQMatrix();
			}

			TileInfo.Read(FrameInfo, InBitStream);

			InBitStream.Skip(8);	// Skip reserved 8 bits
			InBitStream.Align8();	// Byte align.
		}
		
		void ToFrameInfo(const FApvPbuHeader& InPbuHeader, oapv_frm_info_t& OutFrameInfo) const
		{
			FrameInfo.ToFrameInfo(InPbuHeader, OutFrameInfo);
			OutFrameInfo.use_q_matrix = UseQMatrix;
			for(int32 c = 0; c < OAPV_MAX_CC; c++)
			{
				constexpr int32 Mod = (1 << OAPV_LOG2_BLK) - 1;
				for(int32 i = 0; i < OAPV_BLK_D; i++)
				{
					OutFrameInfo.q_matrix[c][i] = QMatrix[c][i >> OAPV_LOG2_BLK][i & Mod];
				}
			}
			OutFrameInfo.color_description_present_flag = ColorDescriptionPresentFlag;
			OutFrameInfo.color_primaries = ColorDescription.ColorPrimaries;
			OutFrameInfo.transfer_characteristics =	ColorDescription.TransferCharacteristic;
			OutFrameInfo.matrix_coefficients = ColorDescription.MatrixCoefficients;
			OutFrameInfo.full_range_flag = ColorDescription.FullRangeFlag;
		}

		FIntPoint GetTileSizeInSamples() const
		{
			return TileInfo.GetTileSizeInSamples();
		}

		FIntPoint GetNumTiles() const
		{
			return TileInfo.GetNumTiles(FrameInfo);
		}

	private:
		void ReadQMatrix(FApvBitReader& InBitStream)
		{
			int32 num_comp = FrameInfo.GetNumComponents();
			for(int32 cidx = 0; cidx < num_comp; cidx++) 
			{
				for(int32 y = 0; y < OAPV_BLK_H; y++)
				{
					for(int32 x = 0; x < OAPV_BLK_W; x++)
					{
						InBitStream << QMatrix[cidx][y][x];
					}
				}
			}
		}

		void InitQMatrix()
		{
			const int32 num_comp = FrameInfo.GetNumComponents();
			for(int32 cidx = 0; cidx < num_comp; cidx++) 
			{
				for(int32 y = 0; y < OAPV_BLK_H; y++)
				{
					for(int32 x = 0; x < OAPV_BLK_W; x++)
					{
						QMatrix[cidx][y][x] = 16;
					}
				}
			}
		}
	};

	// Wraps a frame header during parsing to be able to seek again.
	struct FApvParserFrameHeader
	{
		// Keep track of the frame pbu start offset (after reading pbu size) in the file so we can seek to it.
		int64 PbuStartOffset = 0;

		// Keep track of the frame pbu size.
		int32 PbuSize = 0;
		
		// Frame pbu group Id.
		int16 GroupId = 0;
		
		// Parsed frame header.
		FApvFrameHeader FrameHeader;
	};

	/**
	 * Apv Bitstream Parser utility functions.
	 */
	struct FApvParser
	{
		/**
		 * This function is a version of oapvd_info that doesn't require reading the whole access unit in memory.
		 * It parses info on all primary and non-primary frames.
		 */
		static bool ParseFrameInfo(FApvBitReader& InStream, const FApvAccessUnitHeader& InAccessUnitHeader, TArray<FApvParserFrameHeader>& OutFrameHeaders);
	};
}