// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApvMediaLog.h"
#include "ApvMediaTmvDecoderFactory.h"
#include "ApvMediaTmvEncoderFactory.h"
#include "ITmvMediaModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogApvMedia);

/** Implementation of ApvMedia (OpenApv codec) Module. */
class FApvMediaModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		DecoderFactory = MakeShared<FApvMediaTmvDecoderFactory>();
		ITmvMediaModule::GetOrLoad().RegisterDecoderFactory(DecoderFactory);
		EncoderFactory = MakeShared<FApvMediaTmvEncoderFactory>();
		ITmvMediaModule::GetOrLoad().RegisterEncoderFactory(EncoderFactory);
	}

	virtual void ShutdownModule() override
	{
		if (ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get())
		{
			TmvMediaModule->UnregisterDecoderFactory(DecoderFactory);
			TmvMediaModule->UnregisterEncoderFactory(EncoderFactory);
		}
		DecoderFactory.Reset();
		EncoderFactory.Reset();
	}
	//~ End IModuleInterface

private:
	// Tmv Decoder factory for OpenApv codec.
	TSharedPtr<FApvMediaTmvDecoderFactory, ESPMode::ThreadSafe> DecoderFactory;

	// Tmv Encoder factory for OpenApv codec.
	TSharedPtr<FApvMediaTmvEncoderFactory, ESPMode::ThreadSafe> EncoderFactory;
};

IMPLEMENT_MODULE(FApvMediaModule, ApvMedia);