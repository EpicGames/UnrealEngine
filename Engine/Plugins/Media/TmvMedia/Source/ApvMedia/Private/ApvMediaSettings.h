// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "ApvMediaSettings.generated.h"

UCLASS(config=Engine, defaultconfig)
class UApvMediaSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UApvMediaSettings()
	{
		CategoryName = TEXT("Plugins");
		SectionName  = TEXT("Apv Media");
	}
	
	/**
	 * Number of worker thread to decode tiles (0 = automatically use the available number of cores up to a max of 32).
	 */
	UPROPERTY(config, EditAnywhere, Category = Decoder)
	uint32 DecoderThreads = 0;

	/**
	 * Maximum size for the encoder bit stream buffer.
	 * 
	 * OpenApv requires specification of a maximum bit stream buffer size since the encoder
	 * doesn't reallocate its bit stream buffer. We must specify enough space to store the
	 * whole encoded access unit including all mips.
	 */
	UPROPERTY(config, EditAnywhere, Category = Encoder)
	uint32 MaxEncoderBitStreamBufferSize = 128*1024*1024;
};
