// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApvMediaTmvDecoder.h"

#include "Apv/ApvDecoder.h"
#include "Apv/ApvMipBufferUtils.h"
#include "Apv/ApvParser.h"
#include "ApvMediaLog.h"
#include "HAL/IConsoleManager.h"
#include "Math/IntRect.h"
#include "Misc/ScopeExit.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "SampleConverter/TmvMediaTileResidency.h"
#include "SampleConverter/TmvSharedTilePool.h"

// When enabled, the APV decoder produces tile-major mip buffers (each tile
// stored contiguously with planes interleaved within the tile) instead of
// the historical scanline-strided layout. Tiled buffers let the
// structured-buffer upload path issue one CopyBufferRegion per dirty tile
// (~256 KB each, no overcopy in X) rather than per-row regions that copy
// the full plane width. This is the layout the renderer's TiledPartial
// path expects, and is the default (1) - the tiled upload path is active
// end-to-end. Set 0 to fall back to the historical scanline pipeline. The
// CVar is latched at mip-info creation time so toggling mid-clip does not tear.
int32 GTmvUseTiledLayoutForApv = 1;
static FAutoConsoleVariableRef CVarTmvUseTiledLayoutForApv(
	TEXT("r.TmvMedia.UseTiledLayoutForApv"),
	GTmvUseTiledLayoutForApv,
	TEXT("If 1 (default), the APV decoder emits tile-major mip buffers (each tile is ")
	TEXT("one contiguous block of all planes); if 0, the scanline layout. ")
	TEXT("Tiled enables per-tile CopyBufferRegion uploads via the structured-buffer ")
	TEXT("path (eliminates X overcopy and per-row call-count explosion). Requires ")
	TEXT("decoder-side support (oapv_imgb_t::tiled_layout). Latched at mip-info ")
	TEXT("creation time."),
	ECVF_Default | ECVF_RenderThreadSafe);

namespace UE::ApvMedia
{
#if OAPV_HAS_SELECTIVE_DECODE_API
	struct FApvAccessUnitInputStream : public oapvd_istream_t
	{
		FApvAccessUnitInputStream(ITmvMediaDecoderAccessUnit* InAccessUnit)
		{
			check(InAccessUnit);
			data = InAccessUnit;
			tell = FApvAccessUnitInputStream::TellImpl;
			seek = FApvAccessUnitInputStream::SeekImpl;
			read = FApvAccessUnitInputStream::ReadImpl;
		}

		static long long TellImpl(oapvd_istream_t* InInputStream)
		{
			return reinterpret_cast<ITmvMediaDecoderAccessUnit*>(InInputStream->data)->Tell();
		}

		static int SeekImpl(oapvd_istream_t* InInputStream, long long InOffset, int InOrigin)
		{
			ITmvMediaDecoderAccessUnit* AccessUnit = reinterpret_cast<ITmvMediaDecoderAccessUnit*>(InInputStream->data);

			bool bSuccess;

			switch (InOrigin)
			{
			case SEEK_CUR:
				bSuccess = AccessUnit->Seek(AccessUnit->Tell() + InOffset);
				break;
			case SEEK_END:
				bSuccess = AccessUnit->Seek(AccessUnit->GetTotalSize() + InOffset);
				break;
			default:
				bSuccess = AccessUnit->Seek(InOffset);
				break;
			}
			return bSuccess ? 0 : -1;
		}

		static size_t ReadImpl(oapvd_istream_t* InInputStream, void* OutBuffer, size_t InSize, size_t InCount)
		{
			if (InSize != 0 && InCount != 0)
			{
				int64 ReadBytes = reinterpret_cast<ITmvMediaDecoderAccessUnit*>(InInputStream->data)->Read(OutBuffer, InSize * InCount);
				return ReadBytes >= 0 ? static_cast<size_t>(ReadBytes)/InSize : 0;	// Return the number of elements (match fread return value).
			}
			return 0;
		}
	};
#endif
	
	void SetupTileRequest(const TArray<FIntRect>& InTileRegions, const FIntPoint& InNumTiles, oapv_mip_request& OutMipRequest)
	{
		OutMipRequest.num_tiles = 0;
		for (const FIntRect& TileRegion : InTileRegions)
		{
			// This clamp is to make sure that tile region is not out of bounds in case the region wasn't calculated incorrectly for some reason.
			const FIntRect& ClampedTileRegion = FIntRect(
				FIntPoint(FMath::Max(TileRegion.Min.X, 0), FMath::Max(TileRegion.Min.Y, 0)),
				FIntPoint(FMath::Min(TileRegion.Max.X, InNumTiles.X), FMath::Min(TileRegion.Max.Y, InNumTiles.Y)));

			for (int32 TileRow = ClampedTileRegion.Min.Y; TileRow < ClampedTileRegion.Max.Y; TileRow++)
			{
				for (int32 TileCol = ClampedTileRegion.Min.X; TileCol < ClampedTileRegion.Max.X; TileCol++)
				{
					// Sanity check.
					if (!ensure(OutMipRequest.num_tiles < OAPV_MAX_TILES))
					{
						return;
					}
	
					int32 TileIdx = OutMipRequest.num_tiles;
					OutMipRequest.tile_coords[TileIdx * 2] = TileCol;
					OutMipRequest.tile_coords[TileIdx * 2 + 1] = TileRow;
					OutMipRequest.num_tiles++;
				}
			}
		}
	}
}

ETmvMediaDecoderResult FApvMediaParser::ParseMipInfos(ITmvMediaDecoderAccessUnit& InAccessUnit, TArray<FTmvMediaFrameMipInfo>& OutMipInfos)
{
	using namespace UE::ApvMedia;
	FApvBitReader BitStreamReader(InAccessUnit.GetFilename(), InAccessUnit.GetUnderlyingArchive());
	if (!BitStreamReader.IsValid())
	{
		return ETmvMediaDecoderResult::Fail;
	}

	// Parse the access unit header
	FApvAccessUnitHeader AccessUnitHeader;
	AccessUnitHeader.Read(BitStreamReader);

	TArray<FApvParserFrameHeader> FrameHeaders; 
	FrameHeaders.Reserve(OAPV_MAX_NUM_FRAMES);
	if (!FApvParser::ParseFrameInfo(BitStreamReader, AccessUnitHeader, FrameHeaders) || FrameHeaders.IsEmpty())
	{
		return ETmvMediaDecoderResult::Fail;
	}

	OutMipInfos.Reserve(FrameHeaders.Num());
	for (const FApvParserFrameHeader& FrameHeader : FrameHeaders)
	{
		const FApvFrameInfo& FrameInfo = FrameHeader.FrameHeader.FrameInfo;
		FTmvMediaFrameMipInfo MipInfo;
		MipInfo.MipLevel = OutMipInfos.Num();
		MipInfo.Height = FrameInfo.Height;
		MipInfo.Width = FrameInfo.Width;
		MipInfo.NumComponents = FrameInfo.GetNumComponents();
		MipInfo.ColorModel = ETmvMediaFrameColorModel::YUV;
		MipInfo.ColorInfo.Encoding = GetColorEncoding(static_cast<EApvMediaColorTransfer>(FrameHeader.FrameHeader.ColorDescription.TransferCharacteristic));
		// APV header's transfer characteristic follows ITU-T H.273, we default to BT.2408 reference white.
		const bool bIsHDR = MipInfo.ColorInfo.Encoding == UE::Color::EEncoding::ST2084 || MipInfo.ColorInfo.Encoding == UE::Color::EEncoding::HLG;
		MipInfo.ColorInfo.ReferenceWhiteOverride = bIsHDR ? UE::Color::EReferenceWhite::BT2408 : UE::Color::EReferenceWhite::None;
		MipInfo.ColorInfo.ColorSpace = GetColorSpace(static_cast<EApvMediaColorSpace>(FrameHeader.FrameHeader.ColorDescription.ColorPrimaries));
		MipInfo.ColorInfo.YuvMatrix = GetColorMatrix(static_cast<EApvMediaColorMatrix>(FrameHeader.FrameHeader.ColorDescription.MatrixCoefficients));
		MipInfo.ColorInfo.YuvMatrixRange = GetColorMatrixRange(static_cast<EApvMediaColorMatrixRange>(FrameHeader.FrameHeader.ColorDescription.FullRangeFlag));
		MipInfo.TileWidth = FrameHeader.FrameHeader.TileInfo.GetTileWidthInSamples();
		MipInfo.TileHeight = FrameHeader.FrameHeader.TileInfo.GetTileHeightInSamples();
		MipInfo.NumTiles = FrameHeader.FrameHeader.GetNumTiles();
		// Layout is latched at mip-info creation time. Toggling the CVar mid-clip
		// does not tear because the player rebuilds mip-info on open.
		const bool bUseTiled = (GTmvUseTiledLayoutForApv != 0);
		MipInfo.Layout = bUseTiled
			? ETmvMediaFrameBufferLayout::Tiled
			: ETmvMediaFrameBufferLayout::ScanLine;
		MipInfo.Planes.Reserve(FrameInfo.GetNumComponents());
		const int32 NumBytesPerComp = (FrameInfo.BitDepth + 7)/8;	// This is ceil, so 10, 12 bits take a full 16 bits.
		for (int32 ComponentIndex = 0; ComponentIndex < FrameInfo.GetNumComponents(); ++ComponentIndex)
		{
			FTmvMediaFramePlaneInfo PlaneInfo;
			PlaneInfo.NumComponents = 1;
			PlaneInfo.BitDepth = FrameInfo.BitDepth;
			if (bUseTiled)
			{
				// In tiled mode, Planes[] describes per-tile geometry, not the picture.
				// Stride is the per-row byte advance within a single tile; NumLines is
				// the tile height in component samples. Width/Height stay equal to the
				// picture-plane extent so downstream sizing logic still has it.
				const int32 WidthRatio  = (FrameInfo.GetPlaneWidth(ComponentIndex, false)  != 0)
					? (FrameInfo.Width  / FrameInfo.GetPlaneWidth(ComponentIndex, false))  : 1;
				const int32 HeightRatio = (FrameInfo.GetPlaneHeight(ComponentIndex, false) != 0)
					? (FrameInfo.Height / FrameInfo.GetPlaneHeight(ComponentIndex, false)) : 1;
				const int32 TileWPlane = MipInfo.TileWidth  / FMath::Max(1, WidthRatio);
				const int32 TileHPlane = MipInfo.TileHeight / FMath::Max(1, HeightRatio);
				PlaneInfo.Width       = FrameInfo.GetPlaneWidth(ComponentIndex, false);
				PlaneInfo.Height      = FrameInfo.GetPlaneHeight(ComponentIndex, false);
				PlaneInfo.Stride      = TileWPlane * NumBytesPerComp;
				PlaneInfo.NumLines    = TileHPlane;
				PlaneInfo.WidthRatio  = WidthRatio;
				PlaneInfo.HeightRatio = HeightRatio;
			}
			else
			{
				PlaneInfo.Width = FrameInfo.GetPlaneWidth(ComponentIndex, /*bInMBAligned*/ false);
				PlaneInfo.Height = FrameInfo.GetPlaneHeight(ComponentIndex,  /*bInMBAligned*/ false);
				PlaneInfo.Stride = FrameInfo.GetPlaneWidth(ComponentIndex,  /*bInMBAligned*/ true) * NumBytesPerComp;
				PlaneInfo.NumLines = FrameInfo.GetPlaneHeight(ComponentIndex,  /*bInMBAligned*/ true);
				PlaneInfo.WidthRatio = PlaneInfo.Width != 0 ? FrameInfo.Width / PlaneInfo.Width : 1;
				PlaneInfo.HeightRatio = PlaneInfo.Height != 0 ? FrameInfo.Height / PlaneInfo.Height : 1;
			}
			PlaneInfo.Type = ETmvMediaFrameComponentType::Int;
			MipInfo.Planes.Add(PlaneInfo);
		}
		OutMipInfos.Add(MipInfo);
	}

	return ETmvMediaDecoderResult::Success;
}

FApvMediaTmvDecoder::FApvMediaTmvDecoder(int32 InNumDecodeThreads)
{
	DecoderContext = MakeUnique<UE::ApvMedia::FApvDecoderContext>(InNumDecodeThreads);
}

FApvMediaTmvDecoder::~FApvMediaTmvDecoder() = default;

ETmvMediaDecoderResult FApvMediaTmvDecoder::Decode(ITmvMediaDecoderAccessUnit& InAccessUnit, TArrayView<FTmvMediaDecoderMipRequest> InMipRequests)
{
	using namespace UE::ApvMedia;

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("ApvMedia.Decode %d"), InAccessUnit.GetFrameId()));

	if (!DecoderContext || !DecoderContext->IsValid())
	{
		return ETmvMediaDecoderResult::Fail;
	}

	// Just early out if there is nothing to do.
	if (InMipRequests.IsEmpty())
	{
		return ETmvMediaDecoderResult::Skipped;
	}

#if OAPV_HAS_SELECTIVE_MULTI_MIPS_DECODE_API

	// Acquire read locks on every shared pool referenced by the incoming mip
	// requests. The async grow path takes a write lock for its buffer swap;
	// holding read locks here guarantees the mapped pointers we obtain via
	// ApvSetupImageBuffer remain valid through the entire
	// oapvd_decode_selective_multi_mips call. Sort by pool pointer for
	// deadlock-free global ordering across concurrent decode threads.
	TArray<const FTmvSharedTilePool*, TInlineAllocator<8>> PoolsToLock;
	for (const FTmvMediaDecoderMipRequest& MipRequest : InMipRequests)
	{
		const FTmvMediaTileResidency* Residency = MipRequest.MipBuffer.IsValid() ? MipRequest.MipBuffer->GetResidency() : nullptr;
		if (Residency && Residency->IsShared())
		{
			const FTmvSharedTilePool* Pool = Residency->SharedPool.Get();
			if (Pool)
			{
				PoolsToLock.AddUnique(Pool);
			}
		}
	}
	PoolsToLock.Sort([](const FTmvSharedTilePool& A, const FTmvSharedTilePool& B)
	{
		return &A < &B;
	});
	const double LockStartT = FPlatformTime::Seconds();
	for (const FTmvSharedTilePool* Pool : PoolsToLock)
	{
		Pool->AcquireReadLock();
	}
	const double LockAcquiredT = FPlatformTime::Seconds();
	const double ReadLockMs = (LockAcquiredT - LockStartT) * 1000.0;
	if (ReadLockMs > 50.0)
	{
		UE_LOGF(LogApvMedia, Display,
			"ApvMediaTmvDecoder.Decode: read-lock acquire took %.0f ms across %d pools (frame %d)",
			ReadLockMs, PoolsToLock.Num(), InAccessUnit.GetFrameId());
	}
	ON_SCOPE_EXIT
	{
		for (int32 i = PoolsToLock.Num() - 1; i >= 0; --i)
		{
			PoolsToLock[i]->ReleaseReadLock();
		}
	};

	TArray<oapv_mip_request> ApvMipRequests;
	ApvMipRequests.Reserve(InMipRequests.Num());

	// Prepare the mip requests for OpenApv.
	for (FTmvMediaDecoderMipRequest& MipRequest : InMipRequests)
	{
		// Snapshot the buffer's pool-side GPU resources while we still hold
		// the pool read lock. The reader's later CopyTileRegions reads from
		// the buffer's own member-captured refs - NOT the pool's live refs at
		// queue time - so a grow that lands between decode and CopyTileRegions
		// can't mis-route the per-tile GPU copy to a fresh empty buffer (the
		// green-frame race). Default no-op for non-shared buffers.
		if (MipRequest.MipBuffer.IsValid())
		{
			MipRequest.MipBuffer->CaptureWriteState();
		}

		if (oapv_imgb_t* OutputBuffer = ApvSetupImageBuffer(MipRequest.MipInfo, *MipRequest.MipBuffer))
		{
			oapv_mip_request& ApvMipRequest = ApvMipRequests.Emplace_GetRef();
			FMemory::Memzero(&ApvMipRequest, sizeof(oapv_mip_request));
			ApvMipRequest.mip_level = MipRequest.MipInfo.MipLevel;
			ApvMipRequest.output_buffer = OutputBuffer;

			// Virtualized path: explicit per-tile coords + dst-slot routing.
			// The decoder writes each tile at (slot * tile_size) instead of the
			// natural (row * num_tile_cols + col) offset, so the output buffer
			// can be a bounded slot pool rather than full-tile-count storage. The
			// routing rides on the (Internal) mip buffer, not the ABI-frozen
			// request struct, so we read it through the request's MipBuffer handle.
			const TArray<FIntPoint>* TileCoords = nullptr;
			const TArray<int32>* TileDstSlots = nullptr;
			if (MipRequest.MipBuffer.IsValid())
			{
				TileCoords   = &MipRequest.MipBuffer->GetPendingDecodeTileCoords();
				TileDstSlots = &MipRequest.MipBuffer->GetPendingDecodeTileDstSlots();
			}
			if (TileCoords && !TileCoords->IsEmpty() && TileCoords->Num() == TileDstSlots->Num())
			{
				const int32 NumTiles = FMath::Min<int32>(TileCoords->Num(), OAPV_MAX_TILES);
				for (int32 t = 0; t < NumTiles; ++t)
				{
					const FIntPoint& Tile = (*TileCoords)[t];
					ApvMipRequest.tile_coords[t * 2]     = Tile.X;
					ApvMipRequest.tile_coords[t * 2 + 1] = Tile.Y;
				}
				ApvMipRequest.num_tiles = NumTiles;
				ApvMipRequest.tile_dst_slots = TileDstSlots->GetData();
			}
			else
			{
				SetupTileRequest(MipRequest.TileRegions, MipRequest.MipInfo.NumTiles, ApvMipRequest);
			}
		}
	}

	// Something was invalid in the input requests.
	// Todo: error reporting api.
	if (ApvMipRequests.IsEmpty())
	{
		return ETmvMediaDecoderResult::Fail;
	}

	ON_SCOPE_EXIT
	{
		for (const oapv_mip_request& ApvMipRequest : ApvMipRequests)
		{
			if (ApvMipRequest.output_buffer)
			{
				ApvMipRequest.output_buffer->release(ApvMipRequest.output_buffer);
			}
		}
	};

	FApvAccessUnitInputStream InputStream(&InAccessUnit);

	oapv_multi_mip_decode_t DecodeMultiMip;
	DecodeMultiMip.num_mips = ApvMipRequests.Num();
	DecodeMultiMip.mip_requests = ApvMipRequests.GetData();

	oapvd_stat_t stat;
	FMemory::Memset(&stat, 0, sizeof(oapvd_stat_t));
	
	int ret = oapvd_decode_selective_multi_mips(DecoderContext->did, &InputStream, &DecodeMultiMip, DecoderContext->mid, &stat);
	
	if (OAPV_FAILED(ret))
	{
		UE_LOGF(LogApvMedia, Error, "Failed to decode frame %d in file \"%ls\": Code %d", InAccessUnit.GetFrameId(), *InAccessUnit.GetFilename(), ret);
		return ETmvMediaDecoderResult::Fail;
	}

	// Clear metadata, there is no way to output metadata afaik.
	oapvm_rem_all(DecoderContext->mid);

	// Propagate the results for each mip requests.
	for (const oapv_mip_request& ApvMipRequest : ApvMipRequests)
	{
		FTmvMediaDecoderMipRequest* MipRequest = InMipRequests.FindByPredicate([MipLevel = ApvMipRequest.mip_level](const FTmvMediaDecoderMipRequest& InMipRequest)
		{
			return InMipRequest.MipInfo.MipLevel == MipLevel;
		});
		
		if (MipRequest)
		{
			MipRequest->OutResult = ApvMipRequest.status == OAPV_OK ? ETmvMediaDecoderResult::Success : ETmvMediaDecoderResult::Fail;
			MipRequest->OutNumTilesDecoded = ApvMipRequest.num_tiles;
		}
	}
	return ETmvMediaDecoderResult::Success;
#endif	
	return ETmvMediaDecoderResult::Fail;
}
