// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Decoder/ITmvMediaDecoder.h"

namespace UE::ApvMedia
{
	struct FApvDecoderContext;
}

/**
 * Implementation of a APV1 access unit parser.
 */
class FApvMediaParser : public ITmvMediaParser
{
public:
	FApvMediaParser() = default;
	virtual ~FApvMediaParser() override = default;

	//~ Begin ITmvMediaParser
	virtual ETmvMediaDecoderResult ParseMipInfos(ITmvMediaDecoderAccessUnit& InAccessUnit, TArray<FTmvMediaFrameMipInfo>& OutMipInfos) override;
	//~ End ITmvMediaParser
};

/**
 * Implementation of a TMV decoder for the APV codec.
 */
class FApvMediaTmvDecoder : public ITmvMediaDecoder
{
public:
	FApvMediaTmvDecoder(int32 InNumDecodeThreads);
	virtual ~FApvMediaTmvDecoder() override;

	//~ Begin ITmvMediaDecoder
	virtual ETmvMediaDecoderResult Decode(ITmvMediaDecoderAccessUnit& InAccessUnit, TArrayView<FTmvMediaDecoderMipRequest> InMipRequests) override;
	//~ End ITmvMediaDecoder

private:
	TUniquePtr<UE::ApvMedia::FApvDecoderContext> DecoderContext;
};