// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApvMediaTmvDecoderFactory.h"

#include "ApvMediaLog.h"
#include "Apv/ApvCommon.h"
#include "ApvMediaSettings.h"
#include "ApvMediaTmvDecoder.h"

const FString& FApvMediaTmvDecoderFactory::GetName() const
{
	static const FString DecoderName = TEXT("APV");
	return DecoderName;
}

TArray<FString> FApvMediaTmvDecoderFactory::GetSupportedFileExtensions() const
{
	const TArray<FString> Extensions = { TEXT("apv1")};
	return Extensions;
}

int32 FApvMediaTmvDecoderFactory::SupportsFormat(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions) const
{
	static const FString Apv1(TEXT("apv1"));

	// Ignoring the case for now because it can be a file extension.
	if (InCodecFormat == Apv1)
	{
		return 80;
	}
	return 0;
}

void FApvMediaTmvDecoderFactory::GetParserOptions(TMap<FString, FVariant>& OutOptions) const
{
	// todo: parameters
}

TSharedPtr<ITmvMediaParser, ESPMode::ThreadSafe> FApvMediaTmvDecoderFactory::CreateParser(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions)
{
	return MakeShared<FApvMediaParser, ESPMode::ThreadSafe>();
}

void FApvMediaTmvDecoderFactory::GetDecoderOptions(TMap<FString, FVariant>& OutOptions) const
{
	// todo: parameters
}

TSharedPtr<ITmvMediaDecoder, ESPMode::ThreadSafe> FApvMediaTmvDecoderFactory::CreateDecoder(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions)
{
	const UApvMediaSettings* Settings = GetDefault<UApvMediaSettings>();
	int32 NumThreads = (Settings && Settings->DecoderThreads > 0) ? Settings->DecoderThreads : FMath::Min(FPlatformMisc::NumberOfCores(), OAPV_MAX_THREADS);
	if (NumThreads > OAPV_MAX_THREADS)
	{
		UE_LOGF(LogApvMedia, Warning, "Explicitly requested number of threads (%d) exceeds maximum OpenAPV threads (%d).", NumThreads, OAPV_MAX_THREADS);
	}
	NumThreads = FMath::Clamp(NumThreads, 1, OAPV_MAX_THREADS);
	return MakeShared<FApvMediaTmvDecoder, ESPMode::ThreadSafe>(NumThreads);
}
