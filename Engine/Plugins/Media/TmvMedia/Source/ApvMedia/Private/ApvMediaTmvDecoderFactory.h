// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Decoder/ITmvMediaDecoderFactory.h"

class FApvMediaTmvDecoderFactory : public ITmvMediaDecoderFactory
{
public:
	virtual ~FApvMediaTmvDecoderFactory() = default;

	//~ Begin ITmvMediaDecoderFactory
	virtual const FString& GetName() const override;
	virtual TArray<FString> GetSupportedFileExtensions() const override;
	virtual int32 SupportsFormat(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions) const override;
	virtual void GetParserOptions(TMap<FString, FVariant>& OutOptions) const override;
	virtual TSharedPtr<ITmvMediaParser, ESPMode::ThreadSafe> CreateParser(const FString& InAccessUnitFormat, const TMap<FString, FVariant>& InOptions) override;
	virtual void GetDecoderOptions(TMap<FString, FVariant>& OutOptions) const override;
	virtual TSharedPtr<ITmvMediaDecoder, ESPMode::ThreadSafe> CreateDecoder(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions) override;
	//~ End ITmvMediaDecoderFactory
};