// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApvMediaTmvEncoder.h"

#include "Apv/ApvEncoder.h"
#include "Apv/ApvMipBufferUtils.h"
#include "ApvMediaSettings.h"
#include "ImageCoreUtils.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "Utils/TmvMediaMessageContext.h"
#include "Utils/TmvMediaUtils.h"

#define LOCTEXT_NAMESPACE "ApvMediaTmvEncoder"

namespace UE::ApvMedia::Encoder
{
	/** Align the given width to Macroblock dimensions. */
	int32 AlignWidthToMB(int32 InWidth)
	{
		return ((InWidth + (OAPV_MB_W - 1)) >> OAPV_LOG2_MB_W) << OAPV_LOG2_MB_W;
	}

	/** Align the given height to Macroblock dimensions. */
	int32 AlignHeightToMB(int32 InHeight)
	{
		return ((InHeight + (OAPV_MB_H - 1)) >> OAPV_LOG2_MB_H) << OAPV_LOG2_MB_H;
	}

	/** Indicate if the plane is one of the chroma (UV) component. */
	bool IsChromaPlane(int32 InPlaneIndex)
	{
		return InPlaneIndex > 0 && InPlaneIndex <= 2;
	}

	/** Calculate the actual plane width according to the plane index. */ 
	int32 GetPlaneWidth(int32 InPlaneIndex, EApvMediaChromaFormat InFormat, int32 InWidth, bool bInMBAligned)
	{
		const int32 PlaneWidth = IsChromaPlane(InPlaneIndex) ? InWidth / GetSubWidthC(InFormat) : InWidth;
		return bInMBAligned ? AlignWidthToMB(PlaneWidth) : PlaneWidth;
	}

	/** Calculate the actual plane height according to the plane index. */ 
	int32 GetPlaneHeight(int32 InPlaneIndex, EApvMediaChromaFormat InFormat, int32 InHeight, bool bInMBAligned)
	{
		const int32 PlaneHeight = IsChromaPlane(InPlaneIndex) ? InHeight / GetSubHeightC(InFormat) : InHeight;
		return bInMBAligned ? AlignHeightToMB(PlaneHeight) : PlaneHeight;
	}

	/** Determine if the given mip size is valid given the chroma format. */
	bool IsMipSizeValid(int32 InMipWidth, int32 InMipHeight, EApvMediaChromaFormat InFormat)
	{
		// Note: 420 is not supported by OpenApv (current version), so there is no constraint on height at this time.
		if ((InFormat == EApvMediaChromaFormat::YCbCr422 || InFormat == EApvMediaChromaFormat::YCbCr422_P2) && InMipWidth & 0x1)
		{
			return false;	// Width must be multiple of 2.
		}
		return true;
	}
}


FApvMediaTmvEncoder::FApvMediaTmvEncoder(const FApvMediaTmvEncoderOptions& InEncoderOptions)
	: EncoderOptions(InEncoderOptions)
{
	// Don't create the apv encoder context until we know what resolution we are going to have as input.
}

FApvMediaTmvEncoder::~FApvMediaTmvEncoder() = default;

ETmvMediaEncoderResult FApvMediaTmvEncoder::RequestMipInfos(
	const FTmvMediaFrameTimeInfo& InTimeInfo,
	const FTmvMediaEncoderMipInfo& InFrameInfo,
	TArray<FTmvMediaFrameMipInfo>& OutFrameMipInfo,
	FTmvMediaMessageContext* OutMessageContext)
{
	using namespace UE::ApvMedia;

	if (InFrameInfo.Width <= 0 || InFrameInfo.Height <= 0)
	{
		FText Message = FText::Format(
			LOCTEXT("PrimaryFrameInvalidDim", "Primary Frame has invalid dimensions ({0} x {1})"),
			FText::AsNumber(InFrameInfo.Width), FText::AsNumber(InFrameInfo.Height));
		UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("RequestMipInfos"), Message);
		return ETmvMediaEncoderResult::Fail;
	}

	const int32 MaxNumMips = FTmvMediaUtils::GetMaxMipCountFromDimensions(InFrameInfo.Width, InFrameInfo.Height);
	const int32 NumMips = InFrameInfo.bEnableMips ? MaxNumMips : 1;
	OutFrameMipInfo.Reserve(NumMips);

	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		const int32 MipWidth = FMath::Max(1, InFrameInfo.Width >> MipIndex);
		const int32 MipHeight = FMath::Max(1, InFrameInfo.Height >> MipIndex);

		if (!Encoder::IsMipSizeValid(MipWidth, MipHeight, EncoderOptions.GetChromaFormat()))
		{
			// If the first mip (primary frame) has invalid dimensions, this is a fatal error for this operation.
			if (MipIndex == 0)
			{
				FText Message = FText::Format(
					LOCTEXT("PrimaryFrameInvalidDimForFormat", "Primary Frame has invalid dimensions ({0} x {1}) for the selected chroma format ({2})"),
					FText::AsNumber(MipWidth), FText::AsNumber(MipHeight), UE::TmvMedia::Utils::StaticEnumToText(EncoderOptions.GetChromaFormat()));
				UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("RequestMipInfos"), Message);
				return ETmvMediaEncoderResult::Fail;
			}
			break;
		}

		const EApvMediaChromaFormat ChromaFormat = EncoderOptions.GetChromaFormat();
		const uint8 BitDepth = EncoderOptions.GetBitDepth();
		
		FTmvMediaFrameMipInfo MipInfo;
		MipInfo.MipLevel = MipIndex;
		MipInfo.Width = MipWidth;
		MipInfo.Height = MipHeight;
		MipInfo.NumComponents = GetNumComponents(ChromaFormat);
		MipInfo.ColorModel = ETmvMediaFrameColorModel::YUV;
		// Setup a neutral color transform, then we will apply the converter options.
		MipInfo.ColorInfo.Encoding = UE::Color::EEncoding::None;
		MipInfo.ColorInfo.ColorSpace = UE::Color::EColorSpace::None;
		MipInfo.ColorInfo.YuvMatrix = ETmvMediaFrameColorMatrix::Rec709;
		MipInfo.ColorInfo.YuvMatrixRange = ETmvMediaFrameColorMatrixRange::Full;
		// Apply overrides from encoder settings
		MipInfo.ColorInfo.ApplyOverrides(InFrameInfo.ColorInfo);
		MipInfo.TileWidth = EncoderOptions.TileSize.X;
		MipInfo.TileHeight = EncoderOptions.TileSize.Y;
		MipInfo.NumTiles = 0; // Calculate tiles for given mips.
		MipInfo.Layout = ETmvMediaFrameBufferLayout::ScanLine;	// Apv Decoder outputs frames in scanline layout.
		const int32 NumPlanes = GetNumPlanes(ChromaFormat);
		MipInfo.Planes.Reserve(NumPlanes);
		const int32 NumBytesPerComp = (BitDepth + 7)/8;	// This is ceil, so 10, 12 bits take a full 16 bits.
		for (int32 PlaneIndex = 0; PlaneIndex < NumPlanes; ++PlaneIndex)
		{
			FTmvMediaFramePlaneInfo PlaneInfo;
			PlaneInfo.NumComponents = GetPlaneNumComponents(PlaneIndex, ChromaFormat);
			PlaneInfo.BitDepth = BitDepth;
			PlaneInfo.Width = Encoder::GetPlaneWidth(PlaneIndex, ChromaFormat, MipWidth, /*bInMBAligned*/ false);
			PlaneInfo.Height = Encoder::GetPlaneHeight(PlaneIndex, ChromaFormat, MipHeight, /*bInMBAligned*/ false);
			PlaneInfo.Stride = Encoder::GetPlaneWidth(PlaneIndex, ChromaFormat, MipWidth, /*bInMBAligned*/ true) * PlaneInfo.NumComponents * NumBytesPerComp;
			PlaneInfo.NumLines = Encoder::GetPlaneHeight(PlaneIndex, ChromaFormat, MipHeight, /*bInMBAligned*/ true);
			PlaneInfo.WidthRatio = PlaneInfo.Width != 0 ? MipWidth / PlaneInfo.Width : 1;
			PlaneInfo.HeightRatio = PlaneInfo.Height != 0 ? MipHeight / PlaneInfo.Height : 1;
			PlaneInfo.Type = ETmvMediaFrameComponentType::Int;
			MipInfo.Planes.Add(PlaneInfo);
		}
		OutFrameMipInfo.Add(MipInfo);
	}

	return ETmvMediaEncoderResult::Success;
}

ETmvMediaEncoderResult FApvMediaTmvEncoder::Encode(
	const FTmvMediaFrameTimeInfo& InTimeInfo,
	ITmvMediaEncoderAccessUnit& InAccessUnit,
	TArrayView<FTmvMediaEncoderMipRequest> InMipRequests,
	FTmvMediaMessageContext* OutMessageContext)
{
	if (InMipRequests.IsEmpty()) 
	{ 
		return ETmvMediaEncoderResult::Fail; 
	}

	if (!EncoderContext)
	{
		if (FTmvMediaFrameMipBufferHandle MipBuffer = InMipRequests.IsValidIndex(0) ? InMipRequests[0].MipBuffer : nullptr)
		{
			FTmvMediaEncoderMipInfo EncoderMipInfo;
			EncoderMipInfo.Width = MipBuffer->GetMipInfoRef().Width;
			EncoderMipInfo.Height = MipBuffer->GetMipInfoRef().Height;
			EncoderMipInfo.ColorInfo = MipBuffer->GetMipInfoRef().ColorInfo;
			EncoderMipInfo.bEnableMips = InMipRequests.Num() > 1 ? true : false;
			EncoderContext = MakeUnique<UE::ApvMedia::FApvEncoderContext>(EncoderOptions, EncoderMipInfo, InTimeInfo, OutMessageContext);
		}
		else 
		{
			FText Message = LOCTEXT("MissingMipBufferIndex0", "Missing mip buffer for index 0");
			UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("Encode"), Message);
			return ETmvMediaEncoderResult::Fail;
		}
	}

	if (!EncoderContext || !EncoderContext->IsValid())
	{
		return ETmvMediaEncoderResult::Fail;
	}

	// Prepare the frames
	UE::ApvMedia::FApvFrames InputFrames;
	InputFrames.num_frms = FMath::Min(InMipRequests.Num(), OAPV_MAX_NUM_FRAMES);
	for (int32 FrameIndex = 0; FrameIndex < InputFrames.num_frms; ++FrameIndex)
	{
		FTmvMediaFrameMipBufferHandle& MipBuffer = InMipRequests[FrameIndex].MipBuffer;
		if (!MipBuffer.IsValid()) 
		{
			FText Message = FText::Format(LOCTEXT("MissingMipBufferIndex", "Missing mip buffer for index {0}"), FText::AsNumber(FrameIndex));
			UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("Encode"), Message);
			return ETmvMediaEncoderResult::Fail;
		}

		InputFrames.frm[FrameIndex].imgb = UE::ApvMedia::ApvSetupImageBuffer(MipBuffer->GetMipInfoRef(), *MipBuffer);
		if (InputFrames.frm[FrameIndex].imgb == nullptr)
		{
			FText Message = FText::Format(LOCTEXT("FailedAllocApvImageBuffer", "Failed to set up OpenAPV image buffer for index {0}"), FText::AsNumber(FrameIndex));
			UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("Encode"), Message);
			return ETmvMediaEncoderResult::Fail;
		}

		InputFrames.frm[FrameIndex].group_id = 1 + FrameIndex;
		InputFrames.frm[FrameIndex].pbu_type = FrameIndex == 0 ? OAPV_PBU_TYPE_PRIMARY_FRAME : OAPV_PBU_TYPE_NON_PRIMARY_FRAME;
	}
	
	// Bitstream buffer
	if (BitStreamBuffer.IsEmpty())
	{
		BitStreamBuffer.SetNumUninitialized(GetDefault<UApvMediaSettings>()->MaxEncoderBitStreamBufferSize);
	}

	oapv_bitb_t BitStream;
	BitStream.addr = BitStreamBuffer.GetData();
	BitStream.bsize = BitStreamBuffer.Num();

	// Frames for "recording" encoder result, disabled in our case.
	UE::ApvMedia::FApvFrames DummyFrames;

	oapve_stat_t EncodeStat;
	FMemory::Memset(&EncodeStat, 0, sizeof(oapve_stat_t));

	int32 EncodeResult = oapve_encode(EncoderContext->eid, &InputFrames, EncoderContext->mid, &BitStream, &EncodeStat, &DummyFrames);
	if (OAPV_FAILED(EncodeResult))
	{
		FText Message = FText::Format(
			LOCTEXT("FailedEncodeFrame", "Failed to encode frame: Error {0} ({1})"),
			FText::FromString(UE::ApvMedia::GetApvErrorString(EncodeResult)), FText::AsNumber(EncodeResult));
		UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("Encode"), Message);
		return ETmvMediaEncoderResult::Fail;
	}

	// Validate the written data size.
	if (EncodeStat.write < 0 || EncodeStat.write > BitStreamBuffer.Num())
	{
		FText Message = FText::Format(
			LOCTEXT("EncodedDataOverflow", "Encoded data size ({0}) overflows encoded buffer size ({1})"),
			FText::AsNumber(EncodeStat.write), FText::AsNumber(BitStreamBuffer.Num()));
		UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogApvMedia, Error, TEXT("Encode"), Message);
		return ETmvMediaEncoderResult::Fail;
	}

	// All encoded frames are in a single access unit.
	InAccessUnit.Write(BitStreamBuffer.GetData(), EncodeStat.write);
	return ETmvMediaEncoderResult::Success;
}

#undef LOCTEXT_NAMESPACE