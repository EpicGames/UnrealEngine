// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ApvMediaTmvEncoderOptions.h"
#include "Encoder/ITmvMediaEncoder.h"

namespace UE::ApvMedia
{
	struct FApvEncoderContext;
}

/**
 * Implementation of a TMV encoder for the APV codec.
 */
class FApvMediaTmvEncoder : public ITmvMediaEncoder
{
public:
	FApvMediaTmvEncoder(const FApvMediaTmvEncoderOptions& InEncoderOptions);
	virtual ~FApvMediaTmvEncoder() override;

	//~ Begin ITmvMediaEncoder
	virtual ETmvMediaEncoderResult RequestMipInfos(
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		const FTmvMediaEncoderMipInfo& InFrameInfo,
		TArray<FTmvMediaFrameMipInfo>& OutFrameMipInfo,
		FTmvMediaMessageContext* OutMessageContext) override;

	virtual ETmvMediaEncoderResult Encode(
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		ITmvMediaEncoderAccessUnit& InAccessUnit,
		TArrayView<FTmvMediaEncoderMipRequest> InMipRequests,
		FTmvMediaMessageContext* OutMessageContext) override;
	//~ End ITmvMediaEncoder

private:
	/** Copy of the Apv Encoder options for safe access in the encoding worker thread. */
	FApvMediaTmvEncoderOptions EncoderOptions;

	/** Wrapper for the OpenApv encoding related objects. Reason: the lifetime is not the same as the TmvEncoder itself. */ 
	TUniquePtr<UE::ApvMedia::FApvEncoderContext> EncoderContext;

	/** Temporary bitstream buffer for encoding. Note: encoders are already pooled, no need to pool this buffer. */
	TArray64<uint8_t> BitStreamBuffer;
};