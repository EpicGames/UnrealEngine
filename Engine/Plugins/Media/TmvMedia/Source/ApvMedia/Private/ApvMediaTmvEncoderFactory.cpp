// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApvMediaTmvEncoderFactory.h"

#include "ApvMediaLog.h"
#include "ApvMediaTmvEncoder.h"
#include "ApvMediaTmvEncoderOptions.h"
#include "StructUtils/InstancedStruct.h"
#include "Utils/TmvMediaUtils.h"

#define LOCTEXT_NAMESPACE "ApvMediaTmvEncoderFactory"

const FLazyName FApvMediaTmvEncoderFactory::EncoderName(TEXT("OpenApv"));

FName FApvMediaTmvEncoderFactory::GetName() const
{
	return EncoderName;
}

FText FApvMediaTmvEncoderFactory::GetDisplayName() const
{
	return LOCTEXT("FactoryDisplayName", "OpenApv");
}

void FApvMediaTmvEncoderFactory::GetEncoderOptions(TInstancedStruct<FTmvMediaEncoderOptions>& OutOptions) const
{
	OutOptions.InitializeAs<FApvMediaTmvEncoderOptions>();
}

TSharedPtr<ITmvMediaEncoder, ESPMode::ThreadSafe> FApvMediaTmvEncoderFactory::CreateEncoder(const FString& InCodecFormat, const TInstancedStruct<FTmvMediaEncoderOptions>& InOptions)
{
	if (const FApvMediaTmvEncoderOptions* ApvEncoderOptions = InOptions.GetPtr<FApvMediaTmvEncoderOptions>())
	{
		return MakeShared<FApvMediaTmvEncoder, ESPMode::ThreadSafe>(*ApvEncoderOptions);
	}

	UE_LOGF(LogApvMedia, Error, "Unexpected encoder options (%ls).",
		InOptions.GetScriptStruct() ? *InOptions.GetScriptStruct()->GetName() : TEXT("<null>"));
	return nullptr;
}

FName FApvMediaTmvEncoderOptions::GetEncoderName() const
{
	return FApvMediaTmvEncoderFactory::EncoderName;
}

uint32 FApvMediaTmvEncoderOptions::GetCodecFourCC() const
{
	// FOURCC 'apv1'
	return UE::TmvMedia::Utils::MakeFourCC('a', 'p', 'v', '1');
}

#undef LOCTEXT_NAMESPACE
