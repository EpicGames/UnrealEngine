// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Encoder/ITmvMediaEncoderFactory.h"

/** Implementation of OpenApv Tmv Encoder Factory */
class FApvMediaTmvEncoderFactory : public ITmvMediaEncoderFactory
{
public:
	virtual ~FApvMediaTmvEncoderFactory() override = default;

	//~ Begin ITmvMediaEncoderFactory
	virtual FName GetName() const override;
	virtual FText GetDisplayName() const override;
	virtual bool SupportsMemoryAccessUnit() const override { return true; }
	virtual void GetEncoderOptions(TInstancedStruct<FTmvMediaEncoderOptions>& OutOptions) const override;
	virtual TSharedPtr<ITmvMediaEncoder, ESPMode::ThreadSafe> CreateEncoder(const FString& InCodecFormat, const TInstancedStruct<FTmvMediaEncoderOptions>& InOptions) override;
	//~ End ITmvMediaEncoderFactory

	/** Encoder Name. */
	static const FLazyName EncoderName;
};