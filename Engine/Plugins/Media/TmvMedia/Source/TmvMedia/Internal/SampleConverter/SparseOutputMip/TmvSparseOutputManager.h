// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "Math/IntPoint.h"
#include "PixelFormat.h"
#include "Templates/SharedPointer.h"

class FOutputDevice;
class FRHITexture;
class FTmvSparseOutputBackend;
struct FTmvSparseOutputLayout;

/**
 * Per-clip request describing the smallest sparse-mip index the converter
 * intends to write/sample this frame. Derived from the minimum key in
 * FTmvMediaTextureSampleConverterParameters::Viewports at converter entry.
 */
struct FTmvSparseOutputFrameRequest
{
	/** The output media texture this request applies to. */
	FRHITexture* Texture = nullptr;

	/**
	 * Smallest mip index the converter is about to write this frame. The
	 * manager commits all sparse-eligible mips in [TargetFloor, MaxSparse]
	 * that are not yet committed. Mips below TargetFloor are left committed
	 * (no timer ever decommits); physical memory is released only when the
	 * texture unregisters - see the class header.
	 */
	int32 TargetFloor = 0;
};

/**
 * Platform-agnostic owner of sparse-output residency state. Holds per-texture
 * sparse-eligibility bitsets, LRU clip ordering, and the published
 * ResidentFloor the converter uses to skip below-floor write passes. Physical
 * commit/steal on the GPU is delegated to a platform-specific
 * FTmvSparseOutputBackend (demand-driven chunk pool; see backend docs).
 *
 * Read-side safety: the sparse path only activates for textures created as
 * D3D12 reserved resources, which the RHI enables only on Tiled Resources
 * Tier 2+ (see GRHIGlobals.ReservedResources.Supported). On that hardware,
 * sampling a non-resident (unmapped) tile returns 0 rather than faulting, so a
 * material that samples below the current ResidentFloor during a transient
 * commit/camera mismatch reads black for a frame instead of crashing. An
 * explicit read-side MostDetailedMip clamp on the sampling SRV would turn that
 * black into the next-coarser resident mip; it is a deferred quality
 * enhancement, NOT a crash-safety requirement.
 *
 * Lifecycle: RequestFloor commits everything in [TargetFloor, MaxSparse]
 * that isn't already mapped. The backend may steal from another texture
 * to satisfy a commit - when it does, the manager raises the prior owner's
 * ResidentFloor and the swizzler's below-floor write skip picks up the new
 * floor on the next pass. No timer ever decommits. Physical memory is released only
 * when an FRHITexture unregisters explicitly (called from process drain
 * at OnEnginePreExit); there is no in-session GC tick, because
 * FMediaTextureResource may still hold an SRV bound to materials after
 * its converter stops calling RequestFloor - unmapping under a live SRV
 * faults the GPU.
 *
 * Threading: all public methods are render-thread; internal state is guarded
 * by a single critical section since the manager is rarely contended (one
 * RequestFloor per output per converter pass).
 */
class TMVMEDIA_API FTmvSparseOutputManager
{
public:
	/** Process-wide singleton. Lazily constructs on first access. */
	static FTmvSparseOutputManager& Get();

	/** Returns true iff the manager resolved a backend for the running RHI at startup. */
	bool IsEnabled() const { return Backend.IsValid(); }

	/**
	 * Per-resource setup. Forwards to the backend for tile-count query and
	 * always-resident pre-commit; tracks the texture in the LRU and the
	 * total-committed-bytes accounting. Returns false on any backend failure;
	 * caller should treat the texture as non-sparse in that case.
	 */
	bool RegisterOutput(FRHITexture* Texture, const FTmvSparseOutputLayout& Layout);

	/** Per-resource teardown. Forwards to the backend, drops LRU entry. */
	void UnregisterOutput(FRHITexture* Texture);

	/**
	 * Converter entry point. Eager-commits sparse mips at or above TargetFloor;
	 * publishes the new resident_floor so the swizzler skips write passes for
	 * non-resident mips (read-side safety comes from the Tier-2 reserved-resource
	 * read-zero guarantee - see the class header).
	 *
	 * If a commit steals from another texture (oldest-LRU chunk past
	 * hysteresis), the manager raises that texture's ResidentFloor in the
	 * same call. Pool growth is gated by the configured budget; if the cap
	 * is hit, the backend force-steals the oldest chunk regardless of
	 * hysteresis. No timer ever decommits.
	 */
	void RequestFloor(const FTmvSparseOutputFrameRequest& Request);

	/**
	 * Pin all sparse mips committed for a specific texture, ignoring the
	 * camera-derived floor for the duration of a render pass. Used by MRQ /
	 * capture passes that sample mip 0 outside the player camera prediction.
	 * Released by the matching ReleaseFullResidency call.
	 */
	void RequestFullResidency(FRHITexture* Texture);
	void ReleaseFullResidency(FRHITexture* Texture);

	/**
	 * Smallest mip index currently backed by physical memory for this texture.
	 * The swizzler uses this to skip below-floor write passes; a read-side
	 * MostDetailedMip clamp on the sampling SRV is optional (see the class
	 * header - Tier-2 read-zero makes below-floor sampling safe). Takes the
	 * manager lock for the duration of the read.
	 */
	int32 GetResidentFloor(FRHITexture* Texture) const;

	/** Backend for tmvmedia.dumpmipbuffers - emits per-texture, per-mip committed-byte rows. */
	void Dump(FOutputDevice& Ar) const;

private:
	FTmvSparseOutputManager();
	~FTmvSparseOutputManager();

	FTmvSparseOutputManager(const FTmvSparseOutputManager&) = delete;
	FTmvSparseOutputManager& operator=(const FTmvSparseOutputManager&) = delete;

	struct FTextureState
	{
		int32 NumMips = 0;
		int32 MaxSparseMipIndex = -1;
		int32 ResidentFloor = MAX_int32; // smallest committed sparse mip index
		int32 FullResidencyRefCount = 0;
		TBitArray<> SparseEligible; // size NumMips
	};

	void DrainAll_NoLock();

	void HandleMediaTextureReleased(FRHITexture* Texture);

	// Backend ownership and per-texture state are guarded by Lock.
	mutable FCriticalSection Lock;
	TSharedPtr<FTmvSparseOutputBackend, ESPMode::ThreadSafe> Backend;
	TMap<FRHITexture*, FTextureState> States;
	TArray<FRHITexture*> Lru; // most-recently-requested at end

	FDelegateHandle OnEnginePreExitHandle;
	FDelegateHandle OnMediaTextureReleasedHandle;
};
