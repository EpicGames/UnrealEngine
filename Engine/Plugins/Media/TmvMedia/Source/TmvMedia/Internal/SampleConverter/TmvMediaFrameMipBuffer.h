// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIFwd.h"
#include "SampleConverter/TmvMediaFrameMipBufferFwd.h"
#include "TmvMediaFrameInfo.h"
#include "TmvMediaShaderDefines.h"

#define UE_API TMVMEDIA_API

// Logical-to-physical tile mapping for a virtualized mip buffer. Defined in
// SampleConverter/TmvMediaTileResidency.h; forward-declared here so this header
// stays light (only FTmvMediaFrameMipStructuredBuffer stores one by value).
struct FTmvMediaTileResidency;

/**
 * Abstract based class for a Tmv Frame mip buffer.
 * 
 * A mip buffer for a Tmv frame supports partial updates by a decoder that will write some tiles on the
 * cpu buffer. Then the modified regions of the buffer will be uploaded to the gpu to be used by the Tmv
 * Sample converter shaders to also partially update the final converted texture.
 *
 * The memory storage and gpu buffer implementations are abstracted to be able to support different layouts
 * (i.e. tiled vs scanline) depending on the decoder needs.
 */
class FTmvMediaFrameMipBuffer : public TSharedFromThis<FTmvMediaFrameMipBuffer>
{
public:
	/* Constructor */
	FTmvMediaFrameMipBuffer() = default;

	/** Destructor to clean up render resources */
	virtual ~FTmvMediaFrameMipBuffer() = default;

	/** Returns the total allocation size in bytes. */
	SIZE_T GetAllocatedBufferSize() const
	{
		return AllocatedBufferSize;
	}

	/** Returns the descriptor of the memory layout and color format. */
	const FTmvMediaFrameMipInfo& GetMipInfoRef() const
	{
		return MipInfo;
	}

	/**
	 * Requests an allocation of the buffer suitable for the given mip descriptor.
	 * If the allocation is performed on the render thread, the Mapped buffer is not available immediately.
	 * To ensure the allocation is available, call WaitAllocation.
	 * 
	 * @param InMipInfo Descriptor for the allocation.
	 * @return true if the allocation is valid, false otherwise.
	 */
	virtual bool RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo) = 0;

	/**
	 * If the allocation is done on the render thread, wait for it to be completed.
	 * Should we do this internally in the buffer accessor instead? (probably)
	 */
	virtual void WaitAllocation() = 0;

	/**
	 * Determines from the given allocation descriptor if the current buffer can be used, if it
	 * can, update it.
	 * This is used when recycling buffers from the pool to make sure the buffer is compatible with
	 * the requested memory layout.
	 * 
	 * @param InMipInfo Descriptor for the allocation. 
	 * @return true if the buffer can be used for the given allocation.
	 */
	virtual bool TryUpdateMipInfo(const FTmvMediaFrameMipInfo& InMipInfo) = 0;
	
	/** Access the full mapped buffer. Only available for structured buffer. */
	virtual void* GetMappedBuffer() = 0;

	/**
	 * Returns the start address of the memory plane buffer that contains the given component.
	 * @remark In the case of interleaved components in a memory plane, an additional offset
	 * provided by GetStartComponentOffsetInBytes(), must be applied to the returned value to
	 * get the start of the component within the plane buffer.
	 */
	virtual void* GetPlaneBufferForComponent(int32 InComponentIndex) = 0;

	/**
	 * Returns the shader resource view that contains the given component.
	 * @remark A single shader resource can have multiple components (either packed or interleaved). 
	 * @param InComponentIndex Index of the component to retrieve.
	 * @return Shader resource view of the plane that has this component.
	 */
	virtual FShaderResourceViewRHIRef GetShaderResourceView(int32 InComponentIndex) const = 0;

	/**
	 * Enqueues the render commands to update the buffer regions corresponding to the specified set of tile regions.
	 * @remark Expected to be called from the decoder/reader worker thread.
	 *
	 * @param InFrameId Id of the current frame being updated. Used for stats and debugging.
	 * @param InTmvMipInfo Current frame mip info.
	 * @param InTileRegions Tile regions (specified in tile coordinates) to update.
	 * @param InTileInfos Lookup table for tile buffer memory layout. This maps tile coordinates with buffer offsets.
	 *
	 * Shared-pool buffers consult their own member-captured upload/shader-access refs (snapshotted
	 * by the decoder via CaptureWriteState() under the pool read lock). No captured refs travel
	 * through this signature.
	 */
	virtual void CopyTileRegions(int32 InFrameId, const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions, TConstArrayView<FTmvMediaShaderTileDesc> InTileInfos) = 0;

	/**
	 * Snapshot the buffer's current pool-side GPU resources for the imminent
	 * decode + render-thread CopyTileRegions pair. Called by the decoder once
	 * per buffer after it has acquired the pool read lock(s) but before any
	 * tile bytes are written. Default no-op; FTmvMediaFrameMipStructuredBuffer
	 * (shared mode) overrides to capture its SharedPool->GetUploadBuffer() /
	 * GetShaderAccessBuffer() into private members.
	 *
	 * The captures stay valid until the following CopyTileRegions consumes
	 * them; CopyTileRegions clears them at end of call so a stale capture is
	 * never reused across frames.
	 */
	virtual void CaptureWriteState() {}

	/** Returns true if the buffer can be used for rendering. */
	virtual bool IsValidForRendering() const = 0;

	/** Returns true if the buffer is implemented as a structured buffer. */
	virtual bool IsStructuredBuffer() const
	{
		return false;
	}

	/**
	 * Grow the underlying GPU storage so it can hold at least InMinCapacity
	 * tile slots, expanding the residency map in lockstep. No-op (returns true)
	 * if Residency.Capacity is already >= InMinCapacity. Returns false if the
	 * buffer either doesn't support growth (non-virtualized, or non-structured
	 * variant) or can't grow further (already at the GetMaxGrowSlotCount cap).
	 *
	 * Synchronously reallocates the RHI buffer and GPU-copies existing slot
	 * contents to the new allocation, so on return the buffer can be addressed
	 * via the same slot indices it previously held. Caller is responsible for
	 * holding the buffer alive (via its FTmvMediaFrameMipBufferHandle).
	 */
	virtual bool EnsureCapacityFor(int32 InMinCapacity)
	{
		// Default for non-structured / non-virtualizable buffers: no residency,
		// effective capacity 0, nothing to grow.
		return InMinCapacity <= 0;
	}

	/**
	 * Tile residency map for virtualized buffers, or nullptr for buffer variants
	 * that don't virtualize (the scanline texture/memory/image buffers). Only
	 * FTmvMediaFrameMipStructuredBuffer owns one; keeping the data off this base
	 * avoids the dead Capacity==0 member every other variant used to inherit.
	 * The decoder/pool/converter reach a buffer's residency polymorphically
	 * through this accessor.
	 */
	virtual FTmvMediaTileResidency* GetResidency() { return nullptr; }
	virtual const FTmvMediaTileResidency* GetResidency() const { return nullptr; }

	/**
	 * True when this buffer was allocated with a virtualized (budgeted) residency.
	 * This is the latched allocation-time decision: RequestAllocation initializes
	 * the residency to a positive capacity (or shared-pool mode) only for
	 * virtualized mips, and TryUpdateMipInfo keeps it in sync on pool reuse.
	 *
	 * Consumers downstream of allocation (the reader's slot assignment and the
	 * post-decode descriptor rebuild) MUST read this rather than
	 * FTmvMediaFrameMipInfo::IsVirtualized(), which re-reads the live CVars on
	 * every call - a mid-clip toggle would otherwise make those consumers
	 * disagree with how the buffer was actually sized.
	 */
	UE_API bool IsAllocatedVirtualized() const;

	// --- Render gate -----------------------------------------------------------
	// Monotonic upload-sequence tracking so the converter doesn't sample a buffer
	// before its latest decoded tiles have landed in the shader-access buffer
	// (otherwise it reads not-yet-uploaded "green"/stale memory). A monotonic seq
	// - NOT the media frame id - is required because the paused-refresh path
	// re-decodes new tiles for the SAME media frame across many engine frames; a
	// frame-id-keyed gate would see "already recorded for frame F" and wrongly
	// pass. BeginUpload() (worker, in CopyTileRegions) bumps the enqueued seq and
	// returns it; the render thread passes that seq to RecordUploadExecuted from
	// inside the GPU copy command. Two uint64s; the worker->render cross-read is
	// benign (monotonic, only ever compared <).

	/** Reset on pool acquire/reuse so the gate starts clean for the new owner. */
	void ResetUploadTracking()
	{
		UploadEnqueuedSeq = 0;
		UploadRecordedSeq = 0;
		// Virtualized decode routing is per-frame; start clean for the new owner.
		PendingDecodeTileCoords.Reset();
		PendingDecodeTileDstSlots.Reset();
		PendingDecodeOverflowCount = 0;
	}

	/**
	 * Decode/worker thread: call once per upload, BEFORE enqueuing the GPU copy.
	 * Returns the sequence to hand to RecordUploadExecuted from the copy command.
	 */
	uint64 BeginUpload()
	{
		return ++UploadEnqueuedSeq;
	}

	/** Render thread: the GPU copy for sequence InSeq has been recorded. */
	void RecordUploadExecuted(uint64 InSeq)
	{
		// Monotonic: never move the recorded seq backwards if commands somehow
		// retire out of order.
		if (InSeq > UploadRecordedSeq)
		{
			UploadRecordedSeq = InSeq;
		}
	}

	/**
	 * Render gate. True when an upload has been enqueued (BeginUpload) whose GPU
	 * copy has not yet been recorded on the render thread - i.e. the converter is
	 * about to sample this buffer before its newest decoded tiles have landed.
	 * The converter skips/falls back for such a mip (keep-last or coarse-current)
	 * instead of sampling not-yet-uploaded slots. Unlike a frame-id-keyed gate
	 * this stays correct under paused-refresh (same media frame, many refreshes).
	 */
	bool IsUploadPending() const
	{
		return UploadRecordedSeq < UploadEnqueuedSeq;
	}

	// --- Virtualized decode routing --------------------------------------------
	// Per-frame explicit (tile coord -> dst slot) decode list for the virtualized
	// path, set by the reader's slot assignment and consumed by the decoder. Kept
	// here (Internal) rather than on the public FTmvMediaDecoderMipRequest so that
	// struct's layout stays ABI-frozen for the hotfix; the decoder reaches them via
	// the request's MipBuffer handle. Transient: overwritten each setup that
	// decodes, cleared on pool reuse.

	/** Reader: publish this frame's explicit (tile coord, dst slot) decode list. */
	void SetPendingDecodeRouting(TArray<FIntPoint>&& InTileCoords, TArray<int32>&& InTileDstSlots)
	{
		PendingDecodeTileCoords   = MoveTemp(InTileCoords);
		PendingDecodeTileDstSlots = MoveTemp(InTileDstSlots);
	}

	/** Decoder: parallel tile-coord / dst-slot arrays for virtualized routing (empty = legacy TileRegions path). */
	const TArray<FIntPoint>& GetPendingDecodeTileCoords() const { return PendingDecodeTileCoords; }
	const TArray<int32>& GetPendingDecodeTileDstSlots() const { return PendingDecodeTileDstSlots; }

	/** Reader: count of visible tiles that overflowed (no slot) this frame, for the overflow hysteresis. */
	void SetPendingDecodeOverflowCount(int32 InCount) { PendingDecodeOverflowCount = InCount; }
	int32 GetPendingDecodeOverflowCount() const { return PendingDecodeOverflowCount; }

protected:
	/** Allocated buffer size, used for debugging. */
	SIZE_T AllocatedBufferSize = 0;

	/** Memory frame info this buffer was allocated with. */
	FTmvMediaFrameMipInfo MipInfo;

	// Render-gate upload tracking (see BeginUpload / RecordUploadExecuted /
	// IsUploadPending). Monotonic; EnqueuedSeq written on the worker in
	// CopyTileRegions, RecordedSeq written on the render thread in the copy command.
	uint64 UploadEnqueuedSeq = 0;
	uint64 UploadRecordedSeq = 0;

	// Virtualized decode routing (see SetPendingDecodeRouting / GetPendingDecode*).
	// Set by the reader, read by the decoder via the request's MipBuffer handle;
	// held here so FTmvMediaDecoderMipRequest's public layout stays ABI-frozen.
	TArray<FIntPoint> PendingDecodeTileCoords;
	TArray<int32> PendingDecodeTileDstSlots;
	int32 PendingDecodeOverflowCount = 0;
};

#undef UE_API