// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "Templates/SharedPointer.h"

#define UE_API TMVMEDIA_API

struct FImage;

/**
 * Implementation of texture cpu buffering backed by FImage.
 * If we needed to upload that to rhi textures, we could have a version inheriting from FTmvMediaFrameMipTextureBuffer
 * where the FImage is the "plane buffer". Note: FImage doesn't support R8 nor R16, severely limiting the usefulness for plane formats.
 */
class FTmvMediaFrameMipImageBuffer : public FTmvMediaFrameMipBuffer
{
public:
	/* Constructor */
	UE_API FTmvMediaFrameMipImageBuffer();

	/** Construct from an existing image, using the image as backing buffer. */
	UE_API FTmvMediaFrameMipImageBuffer(const TSharedPtr<FImage>& InImage, int32 InMipLevel);

	/** Destructor to clean up render resources */
	UE_API virtual ~FTmvMediaFrameMipImageBuffer() override;

	//~ Begin FTmvMediaFrameMipBuffer
	UE_API virtual bool RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo) override;
	virtual void WaitAllocation() override {}
	virtual bool TryUpdateMipInfo(const FTmvMediaFrameMipInfo& InMipInfo) override { return false; }
	UE_API virtual void* GetMappedBuffer() override;
	UE_API virtual void* GetPlaneBufferForComponent(int32 InComponentIndex) override;
	UE_API virtual FShaderResourceViewRHIRef GetShaderResourceView(int32 InComponentIndex) const override;
	virtual void CopyTileRegions(int32 InFrameId, const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions, TConstArrayView<FTmvMediaShaderTileDesc> InTileInfos) override {}
	virtual bool IsValidForRendering() const override
	{
		return false;	// This buffer type can't be used for rendering.
	}
	//~ End FTmvMediaFrameMipBuffer

private:
	/** Image buffer for each plane. */
	TArray<TSharedPtr<FImage>> PlaneImages;
};

#undef UE_API
