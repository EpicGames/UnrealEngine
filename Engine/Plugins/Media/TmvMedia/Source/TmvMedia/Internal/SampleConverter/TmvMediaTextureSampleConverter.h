// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseMediaSourceColorSettings.h"
#include "ColorManagement/ColorSpace.h"
#include "IMediaTextureSampleConverter.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "Templates/UniquePtr.h"
#include "TmvMediaShaderDefines.h"

#define UE_API TMVMEDIA_API

/*
* Tmv Sample conversion parameters.
*/
struct FTmvMediaTextureSampleConverterParameters
{
	/** Does this frame has tiles? */
	bool bHasTiles = false;

	/** Frame Id. */
	int32 FrameId = 0;

	/** Resolution of the highest quality mip. */
	FIntPoint FullResolution = FIntPoint(0, 0);

	/** Dimension of the tile including the overscan borders. */
	FIntPoint TileDimWithBorders = FIntPoint(0, 0);

	/** Used for rendering tiles in bulk regions per mip level. */
	TSortedMap<int32, TArray<FIntRect>> Viewports;

	/**
	 * Per mip level, the FULL visible region for this frame (intended coverage),
	 * regardless of how much has actually been decoded into Viewports yet. The
	 * coarse-current fallback (r.TmvMedia.CoarseCurrentFallback) compares this to
	 * Viewports to detect an under-decoded mip and fill only that case from a
	 * coarser ready mip - so good (fully decoded) frames pay nothing.
	 */
	TSortedMap<int32, TArray<FIntRect>> FullVisibleViewports;

	/**
	 * Contain information about individual tiles. Used to convert buffer data into a 2D Texture.
	 * The size of this array is the exact number of complete and partial tiles for each mip level.
	 */
	TArray<TArray<FTmvMediaShaderTileDesc>> TileInfoPerMipLevel;

	/** A lower quality mip will be upscaled if value is 0 or above. At 0 highest quality mip will always be read fully. */
	int32 UpscaleMip = -1;

	/** Manual source color space & encoding overrides. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> SourceColorSettings;
};

/**
 * Implementation of a sample converter for Tmv Frames.
 * 
 * It supports:
 * - Converting planar formats (i.e. components separated in memory planes) with either tiled or scanline memory layouts.
 * - Partial and incremental updates of the buffer regions.
 * - YUV color model
 * - Debugging features like mip tinting.
 */
class FTmvMediaTextureSampleConverter : public IMediaTextureSampleConverter
{
public:
	// Forward declaration of the PIMPL'd tile-descriptor GPU cache. Definition lives in
	// the .cpp so FBufferRHIRef / FShaderResourceViewRHIRef stay out of this header. The
	// name is exposed publicly only so the .cpp's free helpers can reference the type by
	// reference parameter; no caller outside the converter ever constructs or inspects it.
	struct FTileDescGpuCache;

	UE_API FTmvMediaTextureSampleConverter();
	UE_API virtual ~FTmvMediaTextureSampleConverter() override;

	//~ Begin IMediaTextureSampleConverter
	UE_API virtual bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override;
	//~ End IMediaTextureSampleConverter

	/** Returns true if the given mip level already has a cached mip buffer in the converter. */
	UE_API bool HasMipLevelBuffer(int32 RequestedMipLevel) const;

	/** A Mip Buffer for the given mip level will either be acquired from the internal cache if available or created using the provided function if not. */
	UE_API FTmvMediaFrameMipBufferHandle GetOrCreateMipLevelBuffer(int32 RequestedMipLevel, TFunction<FTmvMediaFrameMipBufferHandle()> AllocatorFunc);

	/**
	 * Publish empty owner-visibility on the shared tile pool for every
	 * cached mip buffer not in the active set this frame. Without this,
	 * a mip the plate stopped using (e.g. mip 1+2 when it zooms out to
	 * mip 4+5) keeps its old `VisibleTiles` snapshot in that pool - those
	 * slots stay non-evictable, other plates can't reuse them, and the
	 * pool grows past the actual concurrent demand. This is the root of
	 * the "we're allocating much more than visible tiles need" behavior.
	 *
	 * Call this in the reader, just before iterating MipTilesCopy. The
	 * iteration will re-set the visibility for active mips; this clears
	 * stale visibility for the rest in one pass.
	 */
	UE_API void ClearInactiveOwnerVisibility(const TSet<int32>& InActiveMipLevels);

	/** Safely get a copy of the converter parameters. */
	FTmvMediaTextureSampleConverterParameters GetParams_ThreadSafe()
	{
		FScopeLock ScopeLock(&ParamsCriticalSection);
		return ConverterParams;
	}

	/** Safely set the converter parameters. */
	void SetParams_ThreadSafe(const FTmvMediaTextureSampleConverterParameters& InParams)
	{
		FScopeLock ScopeLock(&ParamsCriticalSection);
		ConverterParams = InParams;
	}

private:
	/** These are all required parameters to convert the buffer into texture successfully. */
	mutable FCriticalSection ParamsCriticalSection;
	FTmvMediaTextureSampleConverterParameters ConverterParams;

	/** Lock to be used exclusively on reader threads.*/
	mutable FCriticalSection MipBufferCriticalSection;

	/** Cached mip buffers that will be used in the render thread for updating the final texture. */
	TMap<int32,FTmvMediaFrameMipBufferHandle> MipBuffers;

	/**
	 * Per-mip GPU cache of the tile-descriptor structured buffer + SRV used by the conversion
	 * shader to look up tile locations within the mip's data buffer.
	 *
	 * Without this, RenderMip_StructuredBuffer creates a fresh FRHIBuffer + SRV every visible
	 * mip every frame, even when the tile-descriptor set is unchanged. The resulting per-frame
	 * churn fills the RHI deferred-delete queue during mip transitions on zoom and triggers
	 * FlushPendingDeleteRHIResources_NonRealtime stalls on the render thread.
	 *
	 * PIMPL'd to keep FBufferRHIRef / FShaderResourceViewRHIRef out of any consumer of this
	 * header. Only accessed from the render thread (inside Convert()), so no locking required.
	 */
	TUniquePtr<FTileDescGpuCache> TileDescGpuCache;
};

#undef UE_API