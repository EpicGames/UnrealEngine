// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/CoreMiscDefines.h"
#include "SampleConverter/TmvSharedTilePool.h"
#include "Templates/SharedPointer.h"

/**
 * Logical-to-physical tile mapping for a virtualized mip buffer.
 *
 * When mip-buffer virtualization is enabled (r.TmvMedia.VirtualizeMipBuffers),
 * the underlying GPU buffer holds a bounded number of physical tile slots
 * instead of one slot per worst-case tile. This struct tracks which logical
 * tile coordinate (X, Y) occupies which physical slot, plus an LRU ordering
 * so newly-visible tiles can evict the oldest non-visible slot.
 *
 * Lifetime: persists across game-frame renders of the SAME TMV frame, so
 * incremental decode keeps working (only newly-visible tiles re-decode). Reset
 * on pool reuse (different TMV frame / potentially different clip).
 */
struct FTmvMediaTileResidency
{
	/** Sentinel for "no tile here" in SlotToTile. */
	static FIntPoint InvalidTile() { return FIntPoint(INT32_MIN, INT32_MIN); }

	int32 Capacity = 0;

	/** Slot index -> logical tile, or InvalidTile() if empty. Size == Capacity. (local mode only) */
	TArray<FIntPoint> SlotToTile;

	/** Logical tile -> slot index. Contains currently-known tiles for this buffer. */
	TMap<FIntPoint, int32> TileToSlot;

	/** Slot LRU ordering (local mode only). */
	TArray<int32> LruOrder;

	/**
	 * Shared-pool mode: when SharedPool is set, slot allocation routes
	 * through the pool and TileToSlot is a per-frame mirror of the pool's
	 * view for this owner. SlotToTile / LruOrder are unused; the pool
	 * manages those globally.
	 */
	TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe> SharedPool;
	FTmvSharedTilePool::FOwnerHandle SharedOwner;

	bool IsShared() const { return SharedPool.IsValid(); }

	/** Local-mode init: sets capacity and resets to all-empty. */
	void Initialize(int32 InCapacity)
	{
		SharedPool.Reset();
		SharedOwner = FTmvSharedTilePool::FOwnerHandle{};
		Capacity = FMath::Max(0, InCapacity);
		SlotToTile.Reset();
		SlotToTile.Init(InvalidTile(), Capacity);
		TileToSlot.Reset();
		LruOrder.Reset();
		LruOrder.Reserve(Capacity);
	}

	/** Switch to shared-pool mode. The buffer must already be registered with the pool. */
	void InitializeShared(const TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe>& InPool, FTmvSharedTilePool::FOwnerHandle InOwner)
	{
		SharedPool = InPool;
		SharedOwner = InOwner;
		SlotToTile.Reset();
		LruOrder.Reset();
		TileToSlot.Reset();
		Capacity = SharedPool.IsValid() ? SharedPool->GetCapacity() : 0;
	}

	/** Clear residency without resizing slots. Called when a pooled buffer is reused. */
	void Reset()
	{
		if (IsShared())
		{
			// Shared mode: leaving residency to pool; just clear the mirror.
			TileToSlot.Reset();
			return;
		}
		for (FIntPoint& Slot : SlotToTile)
		{
			Slot = InvalidTile();
		}
		TileToSlot.Reset();
		LruOrder.Reset();
	}

	int32 GetSlot(FIntPoint InTile) const
	{
		if (IsShared())
		{
			return SharedPool->GetSlot(SharedOwner, InTile);
		}
		if (const int32* Found = TileToSlot.Find(InTile))
		{
			return *Found;
		}
		return INDEX_NONE;
	}

	bool IsResident(FIntPoint InTile) const
	{
		if (IsShared())
		{
			return SharedPool->GetSlot(SharedOwner, InTile) != INDEX_NONE;
		}
		return TileToSlot.Contains(InTile);
	}

	int32 AllocateSlot(FIntPoint InTile, const TSet<FIntPoint>& InVisibleSet)
	{
		if (IsShared())
		{
			const int32 Slot = SharedPool->AllocateSlot(SharedOwner, InTile);
			if (Slot != INDEX_NONE)
			{
				TileToSlot.Add(InTile, Slot);
			}
			return Slot;
		}

		if (Capacity <= 0)
		{
			return INDEX_NONE;
		}
		if (const int32* Existing = TileToSlot.Find(InTile))
		{
			const int32 Slot = *Existing;
			LruOrder.RemoveSingle(Slot);
			LruOrder.Add(Slot);
			return Slot;
		}

		if (LruOrder.Num() < Capacity)
		{
			const int32 Slot = LruOrder.Num();
			SlotToTile[Slot] = InTile;
			TileToSlot.Add(InTile, Slot);
			LruOrder.Add(Slot);
			return Slot;
		}

		for (int32 i = 0; i < LruOrder.Num(); ++i)
		{
			const int32 CandidateSlot = LruOrder[i];
			const FIntPoint CandidateTile = SlotToTile[CandidateSlot];
			if (CandidateTile == InvalidTile() || !InVisibleSet.Contains(CandidateTile))
			{
				TileToSlot.Remove(CandidateTile);
				LruOrder.RemoveAt(i);
				SlotToTile[CandidateSlot] = InTile;
				TileToSlot.Add(InTile, CandidateSlot);
				LruOrder.Add(CandidateSlot);
				return CandidateSlot;
			}
		}

		return INDEX_NONE;
	}
};
