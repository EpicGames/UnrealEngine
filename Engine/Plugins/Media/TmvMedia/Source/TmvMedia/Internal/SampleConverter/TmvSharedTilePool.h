// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "Math/IntPoint.h"
#include "Misc/ScopeRWLock.h"
#include "RHIFwd.h"
#include "Templates/SharedPointer.h"
#include "TmvMediaFrameInfo.h"

#include <atomic>

#define UE_API TMVMEDIA_API

/**
 * Global shared tile pool for virtualized mip buffers.
 *
 * One FTmvSharedTilePool owns a single large GPU structured buffer carved
 * into fixed-size slots. Multiple FTmvMediaFrameMipBuffer instances (across
 * different clips and mip levels) act as "owners" of slots in the same pool,
 * subject to the constraint that all owners agree on per-slot byte size and
 * plane count. The pool is keyed by (per-tile-bytes, plane-count) so plates of
 * the same APV format coalesce while incompatible plates land in separate pools
 * (see FKey for why per-plane layout is deliberately excluded from the key).
 *
 * Lifetime: pools are owned by the FTmvSharedTilePoolManager singleton. They
 * are created lazily on first request and live for the duration of the
 * TmvMedia module. Owners register/unregister via opaque OwnerHandle.
 *
 * Eviction (two-LRU): slots live in one of three intrusive doubly-linked
 * lists keyed by their current state - Free, VisibleLRU, or NonVisibleLRU.
 * AllocateSlot pops the Free list head first (no eviction work), else the
 * tail (LRU end) of NonVisibleLRU; if both are empty every slot is pinned
 * to some owner's visible set and the caller must grow. Touch on an
 * existing slot moves it to the head (MRU) of VisibleLRU. SetOwnerVisibility
 * walks only this owner's slots (FOwnerInfo::SlotIndices), reclassifying
 * each into VisibleLRU or NonVisibleLRU. All pool ops are O(1) amortized
 * (allocations / touches) or O(per-owner-slots) (visibility updates).
 * Critical-section guarded; safe under concurrent decode threads.
 *
 * Growth: when no evictable slot exists (every slot's tile is in some
 * owner's visible set), the pool reallocates its GPU buffer at 2x capacity
 * and copies existing slot contents over. Same pattern as
 * FTmvMediaFrameMipStructuredBuffer::EnsureCapacityFor.
 */
class FTmvSharedTilePool : public TSharedFromThis<FTmvSharedTilePool, ESPMode::ThreadSafe>
{
public:
	/**
	 * Identifies a pool's tile compatibility class. Same key = same pool.
	 *
	 * Keyed by (PerTileBytes, NumPlanes) only - deliberately NOT by per-plane
	 * dimensions/stride. Different mips of the same APV format have the same
	 * tile byte size and plane count (TMV pads tiles to a uniform per-mip-
	 * level size that happens to agree across mips for this format) but
	 * different per-plane width/height/stride. We unify them anyway because
	 * the pool only stores opaque bytes; the per-mip layout (Offsets, Strides)
	 * is carried in the per-tile shader descriptor on the converter side, not
	 * in the pool. Unifying lets cross-mip slot reuse work: when a plate
	 * stops using mip 0 (zooms out), its mip-0 tiles' slots become evictable
	 * and the next plate's mip-3 tiles can take them. Total VRAM tracks the
	 * peak across-all-mips concurrent residency, not the sum of per-mip
	 * peaks.
	 */
	struct FKey
	{
		int32 PerTileBytes = 0;
		int32 NumPlanes    = 0;

		bool operator==(const FKey& Other) const
		{
			return PerTileBytes == Other.PerTileBytes
				&& NumPlanes    == Other.NumPlanes;
		}

		friend uint32 GetTypeHash(const FKey& K)
		{
			return HashCombine(GetTypeHash(K.PerTileBytes), GetTypeHash(K.NumPlanes));
		}
	};

	/** Per-owner handle into the pool. Treat as opaque. */
	struct FOwnerHandle
	{
		int32 OwnerId = INDEX_NONE;
		bool IsValid() const { return OwnerId != INDEX_NONE; }
	};

	UE_API static FKey MakeKey(const FTmvMediaFrameMipInfo& InMipInfo);

	UE_API FTmvSharedTilePool(const FKey& InKey, int32 InInitialCapacity);
	UE_API ~FTmvSharedTilePool();

	UE_API FOwnerHandle RegisterOwner();
	UE_API void UnregisterOwner(FOwnerHandle InHandle);

	/** Number of currently-registered owners. Zero means the pool is unused. */
	UE_API int32 GetOwnerCount() const;

	/** Returns slot index for the given owner+tile, or INDEX_NONE if not resident. */
	UE_API int32 GetSlot(FOwnerHandle InOwner, FIntPoint InTile) const;

	/**
	 * Allocates a slot for the given owner+tile. Returns the slot index, or
	 * INDEX_NONE if the pool is full of currently-visible tiles across all
	 * owners (caller should grow or fall back to shader discard).
	 */
	UE_API int32 AllocateSlot(FOwnerHandle InOwner, FIntPoint InTile);

	/** Publishes the owner's currently-visible tile set; used by eviction. */
	UE_API void SetOwnerVisibility(FOwnerHandle InOwner, const TSet<FIntPoint>& InVisible);

	/**
	 * Request the GPU buffer grow so total slot count >= InMinCapacity.
	 * Non-blocking: enqueues the grow render command and returns immediately.
	 * If another grow is already in flight, this is a no-op (the next call
	 * after grow completes will see the larger pool).
	 * Returns true if grow was initiated or already sufficient.
	 */
	UE_API bool EnsureCapacityFor(int32 InMinCapacity);

	/**
	 * True once the initial GPU allocation has completed. Until then,
	 * GetMappedBuffer / GetShaderResourceView return null and AllocateSlot
	 * returns INDEX_NONE. Lock-free read.
	 */
	bool IsReady() const { return bGpuReady.load(std::memory_order_relaxed); }

	/**
	 * Block the calling thread until the initial GPU allocation completes
	 * (bGpuReady transitions to true). No-op if already ready. Used by the
	 * shared-mode FTmvMediaFrameMipStructuredBuffer::WaitAllocation so the
	 * decoder doesn't try to write through a still-null mapped pointer in
	 * the window between RequestAllocation and the render thread completing
	 * the async alloc - that race was producing 10s blocking-media timeouts
	 * (ImgMedia.BlockingFetch deadline) and stale slot reads ("green frames")
	 * during multi-plate ramp-up.
	 */
	UE_API void WaitForReady();

	/**
	 * Block the calling (worker / decode) thread until any in-flight grow
	 * completes - i.e. until bGrowing transitions back to false. No-op if
	 * no grow is pending. Used by the reader's slot-allocation path to make
	 * AllocateSlot effectively non-failing under blocking-media mode: when
	 * AllocateSlot returns INDEX_NONE (pool full of pinned tiles), the
	 * reader triggers a grow, calls this, then retries. The cost is a brief
	 * (~30 ms typical) worker stall during the grow; the alternative is
	 * shipping a sample with missing tiles which the shader then either
	 * upscales from a lower mip (perf hit on the upscale-fallback path) or
	 * reads as stale slot bytes ("green frames").
	 */
	UE_API void WaitForGrowComplete();

	/**
	 * Public read-lock helper for decoders.
	 *
	 * A decoder thread must hold a read lock on GrowLock for the duration of
	 * its mapped-buffer accesses (from GetPlaneBufferForComponent through the
	 * last write in oapvd_decode_*). The async grow path takes the write lock
	 * around its buffer swap so decoders can't observe a torn pointer/buffer
	 * pair. Use the RAII helper below:
	 *
	 *     {
	 *         FTmvSharedTilePool::FReadLockScope Lock(*SharedPool);
	 *         // safe to call GetMappedBuffer(), write into it, etc.
	 *     }
	 */
	struct FReadLockScope
	{
		explicit FReadLockScope(FTmvSharedTilePool& InPool)
			: Scope(InPool.GrowLock, SLT_ReadOnly)
		{}
	private:
		FRWScopeLock Scope;
	};

	/**
	 * Manual read-lock entry points for callers that need to hold read locks
	 * on multiple pools simultaneously (e.g., the decoder, when a single
	 * ApvMediaTmvDecoder::Decode call writes to N pools at once). Callers
	 * MUST sort the pool list by pointer-address before calling AcquireRead
	 * on each, to ensure deterministic global lock ordering and avoid
	 * deadlock between concurrent decode threads.
	 */
	void AcquireReadLock() const { GrowLock.ReadLock(); }
	void ReleaseReadLock() const { GrowLock.ReadUnlock(); }

	/** Returns true if a buffer-info matches this pool's compatibility class. */
	UE_API bool MatchesMipInfo(const FTmvMediaFrameMipInfo& InMipInfo) const;

	// GPU resource accessors (used by FTmvMediaFrameMipBuffer in shared mode).
	// Return null until the pool is Ready. Callers must hold an FReadLockScope
	// across any decoder writes that use the mapped pointer.
	UE_API void* GetMappedBuffer() const;
	UE_API FShaderResourceViewRHIRef GetShaderResourceView() const;
	UE_API FBufferRHIRef GetUploadBuffer() const;
	UE_API FBufferRHIRef GetShaderAccessBuffer() const;

	int32 GetCapacity() const   { return Capacity; }
	int32 GetSlotBytes() const  { return Key.PerTileBytes; }
	const FKey& GetKey() const  { return Key; }
	SIZE_T GetAllocatedBufferSize() const { return AllocatedBufferSize; }

	/**
	 * Sum of per-owner VisibleTiles counts. This is the *true minimum*
	 * capacity needed to never return INDEX_NONE from AllocateSlot at the
	 * current moment - every owner's visible tile must have a slot. Used by
	 * the reader's INDEX_NONE-retry path to size the grow precisely to
	 * demand instead of blindly doubling. Thread-safe via the Cs critical
	 * section.
	 */
	UE_API int32 GetTotalVisibleTileCount() const;

private:
	/**
	 * Which intrusive doubly-linked list a slot is currently a member of.
	 * Used by the two-LRU eviction scheme below: visible-pinned slots and
	 * evictable slots live in separate lists so allocation can pop the
	 * non-visible tail in O(1) without scanning past pinned slots.
	 */
	enum class ESlotList : uint8
	{
		Free,           // OwnerId == INDEX_NONE, no tile
		VisibleLRU,     // Assigned, tile is in owner's VisibleTiles -> pinned
		NonVisibleLRU,  // Assigned, tile NOT in owner's VisibleTiles -> evictable
	};

	struct FSlot
	{
		int32 OwnerId       = INDEX_NONE;
		FIntPoint Tile      = FIntPoint(INT32_MIN, INT32_MIN);
		// Intrusive doubly-linked list pointers (slot indices, INDEX_NONE = end).
		int32 PrevInList    = INDEX_NONE;
		int32 NextInList    = INDEX_NONE;
		ESlotList List      = ESlotList::Free;
	};

	struct FOwnerInfo
	{
		TSet<FIntPoint> VisibleTiles;
		// Slot indices currently owned by this owner (in either VisibleLRU or
		// NonVisibleLRU). Lets SetOwnerVisibility iterate only this owner's
		// slots instead of the whole pool.
		TArray<int32> SlotIndices;
	};

	// LRU list heads/tails. Head = MRU (recently touched), Tail = LRU (will be
	// evicted next). Free list only needs head; pops/pushes from head are O(1).
	int32 FreeHead          = INDEX_NONE;
	int32 VisibleLruHead    = INDEX_NONE;
	int32 VisibleLruTail    = INDEX_NONE;
	int32 NonVisibleLruHead = INDEX_NONE;
	int32 NonVisibleLruTail = INDEX_NONE;

	// Intrusive list ops. All callers must hold Cs.
	void ListRemove(int32 SlotIdx);
	void ListInsertAtHead_Free(int32 SlotIdx);
	void ListInsertAtHead_VisibleLRU(int32 SlotIdx);
	void ListInsertAtHead_NonVisibleLRU(int32 SlotIdx);
	void MoveToList(int32 SlotIdx, ESlotList NewList);
	// Returns INDEX_NONE if neither a free slot nor an evictable non-visible
	// slot is available (caller will trigger grow).
	int32 PickFreeOrEvictableSlot();

	/**
	 * Async initial allocation: enqueues the render command, returns
	 * immediately. On render thread the command sets the RHI refs and
	 * flips bGpuReady to true. Must be called by the manager AFTER the
	 * pool is owned by a TSharedPtr (AsWeak asserts otherwise).
	 */
	void EnqueueInitialAllocation_GameOrDecodeThread(int32 InCapacity, SIZE_T InAllocSize);

public:
	// Called by FTmvSharedTilePoolManager::GetOrCreatePool once the pool's
	// TSharedPtr is established. Initiates async GPU allocation.
	void BeginInitialAllocation()
	{
		if (Capacity > 0 && AllocatedBufferSize > 0)
		{
			EnqueueInitialAllocation_GameOrDecodeThread(Capacity, AllocatedBufferSize);
		}
	}

private:

	FKey Key;
	int32 Capacity = 0;
	SIZE_T AllocatedBufferSize = 0;

	mutable FCriticalSection Cs;

	/**
	 * Grow lock. Decoders take a read lock around their mapped-buffer writes;
	 * the async grow takes a write lock for the buffer-swap step. Public via
	 * FReadLockScope (above) so the lock is visible at the decoder call site
	 * but the lock object itself stays private.
	 */
	mutable FRWLock GrowLock;

	/** True once initial GPU allocation completed. */
	std::atomic<bool> bGpuReady{false};

	/**
	 * Triggered by the initial-alloc render command immediately after
	 * bGpuReady is set true. Used by WaitForReady() to block callers cleanly
	 * (no spin-wait) until the GPU side of the pool exists.
	 * Manual-reset: once triggered, stays signaled for the pool's lifetime.
	 */
	FEvent* GpuReadyEvent = nullptr;

	/** True while an async grow render command is in flight; suppresses duplicate triggers. */
	std::atomic<bool> bGrowing{false};

	/**
	 * Auto-reset event triggered when bGrowing transitions back to false at
	 * the end of a grow render command. Workers blocked in WaitForGrowComplete
	 * wake up here. Auto-reset (not manual-reset like GpuReadyEvent) because
	 * grows happen repeatedly - each grow should wake exactly the threads
	 * that were waiting for it, and the next grow's waiters should block
	 * again until that grow's own trigger.
	 *
	 * Multi-waiter pattern: many workers can call WaitForGrowComplete
	 * concurrently; the grow render command should Trigger() this event
	 * enough times for all of them. We track the wait count atomically and
	 * Trigger as many times as needed.
	 */
	FEvent* GrowDoneEvent = nullptr;

	/**
	 * Number of threads currently parked inside WaitForGrowComplete. The
	 * grow-completion code uses this to Trigger() the auto-reset event the
	 * right number of times (one trigger per blocked thread).
	 */
	std::atomic<int32> GrowWaiters{0};

	TArray<FSlot> Slots;
	TMap<TPair<int32, FIntPoint>, int32> OwnerKeyToSlot;
	TMap<int32, FOwnerInfo> Owners;
	int32 NextOwnerId = 0;

	// GPU resources (allocated on render thread). UploadBufferMapped is
	// updated under the write lock during grow so decoders holding read
	// locks see consistent state.
	FBufferRHIRef UploadBufferRef;
	void* UploadBufferMapped = nullptr;
	FBufferRHIRef ShaderAccessBufferRef;
	FShaderResourceViewRHIRef ShaderResourceView;
};

/** Singleton manager that keeps one pool per compatibility key. */
class FTmvSharedTilePoolManager
{
public:
	UE_API static FTmvSharedTilePoolManager& Get();

	UE_API TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe> GetOrCreatePool(const FTmvMediaFrameMipInfo& InMipInfo, int32 InitialCapacity);

	/**
	 * Unregister an owner from its pool and, if that was the pool's last owner,
	 * drop the pool from the manager so its GPU buffers are freed now rather than
	 * lingering for the whole process. Re-opening a clip of the same format
	 * recreates the pool on demand. Callers should release their own TSharedPtr
	 * to the pool after this returns (the structured buffer does so when it is
	 * destroyed) so the final reference goes away and ~FTmvSharedTilePool runs.
	 */
	UE_API void ReleaseOwner(const TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe>& InPool, FTmvSharedTilePool::FOwnerHandle InOwner);

	UE_API void ReleaseAll(); // for shutdown

private:
	TMap<FTmvSharedTilePool::FKey, TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe>> Pools;
	FCriticalSection PoolsLock;
};

#undef UE_API
