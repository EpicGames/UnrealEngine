// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "HAL/Platform.h"
#include "Math/IntPoint.h"
#include "Misc/CoreMiscDefines.h"

class FTmvMediaFrameMipBuffer;
struct FTmvMediaTileResidency;

/**
 * Per-(mip-buffer) helper for one frame's virtualized slot assignment. Wraps
 * the shared-tile-pool interaction the reader (FTmvImgMediaReader) used to
 * inline: capacity growth, this owner's visibility publishing + local residency
 * mirror, and the bounded grow-and-retry on slot exhaustion.
 *
 * Constructed cheaply per mip request (holds references, owns no state). The
 * pool's own locking still governs concurrency; this only relocates the policy
 * (retry bound, grow-target math) off the reader so the reader is left with
 * just decode-list building.
 */
class FTmvSharedTileSession
{
public:
	enum class ESlotResult : uint8
	{
		/** Tile already had a slot; no decode needed (slot mirrored for descriptor rebuild). */
		AlreadyResident,
		/** A slot was assigned; the tile must be decoded into it. */
		Allocated,
		/** No slot available even after grow-and-retry; caller treats as overflow. */
		Overflowed,
	};

	struct FSlotResolution
	{
		ESlotResult Result = ESlotResult::Overflowed;
		int32 Slot = INDEX_NONE;
	};

	explicit TMVMEDIA_API FTmvSharedTileSession(FTmvMediaFrameMipBuffer& InBuffer);

	/** Grow the buffer to hold at least InVisibleCount slots if it is currently short. */
	TMVMEDIA_API void EnsureCapacityFor(int32 InVisibleCount);

	/**
	 * Shared mode: publish this owner's visible set for cross-owner eviction
	 * protection and clear the local TileToSlot mirror (the pool may have
	 * evicted slots since last frame; ResolveSlot re-resolves them). No-op in
	 * local mode.
	 */
	TMVMEDIA_API void PublishVisibility(const TSet<FIntPoint>& InVisibleSet);

	/**
	 * Resolve a slot for InTile: returns its existing slot if resident (mirroring
	 * it locally in shared mode), otherwise allocates; on exhaustion runs a
	 * bounded grow-and-retry. See FSlotResolution::Result for the three outcomes.
	 */
	TMVMEDIA_API FSlotResolution ResolveSlot(FIntPoint InTile, const TSet<FIntPoint>& InVisibleSet);

	/** Max blocking grow-and-retry attempts before a tile is declared overflowed. */
	static constexpr int32 MaxGrowRetries = 8;

private:
	FTmvMediaFrameMipBuffer& Buffer;
	FTmvMediaTileResidency& Residency;
};
