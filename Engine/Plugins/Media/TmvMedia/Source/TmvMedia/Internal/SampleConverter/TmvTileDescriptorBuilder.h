// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"

struct FTmvMediaFrameMipInfo;
struct FTmvMediaShaderTileDesc;
struct FTmvMediaTileResidency;

/**
 * Builds the per-tile GPU descriptor array (one FTmvMediaShaderTileDesc per
 * tile) that the structured-buffer swizzle shader and the CPU partial-copy
 * logic use to locate each tile inside a tile-major mip buffer.
 *
 * Pure transform over (mip-info [+ residency]) -> descriptor array: no engine,
 * RHI, or threading state, so it can be unit-tested in isolation. Extracted
 * from FTmvImgMediaReader, which previously owned both fill paths inline.
 */
struct FTmvTileDescriptorBuilder
{
	/**
	 * Non-virtualized tiled mip: the tile at natural index (Y*NumTilesX + X)
	 * lives at byte base TileBytes*Idx. Sizes OutDescs to the tile count, zeroes
	 * it, then fills every tile. For a virtualized mip the descriptors instead
	 * come from the residency map post-decode (slots don't track tile coords),
	 * so OutDescs is only sized+zeroed here. No-op when the mip has no tiles.
	 */
	static TMVMEDIA_API void BuildNaturalIndex(const FTmvMediaFrameMipInfo& InMipInfo, TArray<FTmvMediaShaderTileDesc>& OutDescs);

	/**
	 * Virtualized tiled mip: each resident tile (X,Y) lives at byte base
	 * TileBytes*Slot per the residency map; non-resident tiles stay zeroed so
	 * they trip the shader's "no tile here" sentinel. Reuses the existing
	 * OutDescs allocation when the tile count is unchanged (only re-zeroing it),
	 * otherwise resizes. No-op when the mip has no tiles.
	 */
	static TMVMEDIA_API void BuildFromResidency(const FTmvMediaFrameMipInfo& InMipInfo, const FTmvMediaTileResidency& InResidency, TArray<FTmvMediaShaderTileDesc>& OutDescs);
};
