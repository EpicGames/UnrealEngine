// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

// Internal CVar accessors that live outside the public TmvMediaFrameInfo.h
// because no inline method in the public surface reaches them. Sister modules
// (ImgMedia's TmvImgMediaTextureSample) and private TmvMedia TUs include this
// header directly.
namespace UE::TmvMedia
{
	/** True when r.TmvMedia.UseSharedTilePool routes virtualized buffers through the global shared pool. */
	TMVMEDIA_API bool IsSharedTilePoolEnabled();

	// ---------------------------------------------------------------------
	// Sparse output media texture (D3D12 reserved-resource per-mip residency).
	// Cuts per-clip output-texture VRAM by holding the large mips in a
	// reserved (tiled) resource whose physical backing is committed only when
	// the decoder is actually feeding those mips. Gated end-to-end by
	// IsSparseOutputMipEnabled(); when off, the create-flags hint returns
	// None and the rest of the pipeline is bit-identical to baseline.
	// ---------------------------------------------------------------------

	/** True when r.TmvMedia.SparseOutputMip is on. Primary switch for sparse output residency. */
	TMVMEDIA_API bool IsSparseOutputMipEnabled();

	/**
	 * Per-mip byte-threshold for sparse eligibility. A mip is sparse-managed
	 * when mip_bytes >= GetSparseOutputMipMinBytesMB() * 1 MB. Default 128 MB
	 * so an HD-plate output texture skips the sparse path (all mips below
	 * threshold) while a 16K plate puts mips 0..3 in the sparse set.
	 */
	TMVMEDIA_API int32 GetSparseOutputMipMinBytesMB();

	/**
	 * Hard ceiling on heap-pool size across all sparse output mips. When a
	 * fresh allocation would push the pool past this, the backend force-steals
	 * the oldest-LRU owned chunk regardless of hysteresis (safety valve). In
	 * normal operation the pool sits at peak concurrent demand, well below
	 * the budget; the budget only matters under pathological pressure or
	 * when intentionally tightened for testing. Default 2048 MB.
	 */
	TMVMEDIA_API int32 GetSparseOutputMipBudgetMB();

	/**
	 * Minimum wall-clock age of an owned chunk before it becomes eligible to
	 * be stolen by another texture's commit. Prevents ping-pong thrash on
	 * rapid camera cuts; no decommit ever fires at this threshold (memory is
	 * only released at UnregisterOutput). Independent of frame rate.
	 * Default 2 (seconds) - long enough to ride out rapid camera cuts
	 * without re-committing, short enough to release VRAM promptly.
	 */
	TMVMEDIA_API int32 GetSparseOutputMipHysteresisSeconds();

	/**
	 * Debug: pin the target floor for all sparse output textures. -1
	 * (default) means "follow Viewports". 0 means "always commit mip 0 too";
	 * a positive N means "never commit mips finer than N." Used to A/B
	 * what the residency tradeoff looks like without depending on camera
	 * motion.
	 */
	TMVMEDIA_API int32 GetSparseOutputMipDebugForceFloor();
}
