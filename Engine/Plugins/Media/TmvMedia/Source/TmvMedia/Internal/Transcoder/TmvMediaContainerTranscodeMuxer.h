// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Event.h"
#include "Transcoder/TmvMediaTranscodeMuxer.h"
#include "Encoder/ITmvMediaMuxer.h"

#include "TmvMediaContainerTranscodeMuxer.generated.h"

#define UE_API TMVMEDIA_API

class ITmvMediaMuxer;

/**
 * Container muxer: muxes access units into a container file using ITmvMediaMuxer.
 *
 * Bridges the push-based transcoder pipeline (ReceiveAccessUnit) with the pull-based
 * ITmvMediaMuxer (FSampleRequestDelegate). Incoming access units are buffered and
 * delivered when the muxer's worker thread requests them.
 *
 * @see UTmvMediaTranscodeMuxer
 */
UCLASS(MinimalAPI)
class UTmvMediaContainerTranscodeMuxer : public UTmvMediaTranscodeMuxer
{
	GENERATED_BODY()
public:
	UTmvMediaContainerTranscodeMuxer();

	// ~Begin UTmvMediaTranscodeStage
	UE_API virtual bool Start(UTmvMediaTranscodeJob* InParentJob) override;
	UE_API virtual void RequestStop(UTmvMediaTranscodeJob* InParentJob) override;
	// ~End UTmvMediaTranscodeStage

	// ~Begin UTmvMediaTranscodeMuxer
	UE_API virtual int32 OpenStream(UTmvMediaTranscodeJob* InParentJob, const FString& InStreamName, const FString& InExtension) override;
	UE_API virtual void ReceiveAccessUnit(
		UTmvMediaTranscodeJob* InParentJob,
		int32 InStreamId,
		const FTmvMediaFrameTimeInfo& TimeInfo,
		TSharedPtr<TArray64<uint8>>&& InAccessUnit) override;
	// ~End UTmvMediaTranscodeMuxer

	/**
	 * Set the track configuration for a stream opened with OpenStream.
	 * Must be called before the first ReceiveAccessUnit for this stream.
	 * @param InStreamId Stream index returned by OpenStream.
	 * @param InTrackConfig Track configuration for the container.
	 */
	UE_API void SetStreamTrackConfig(int32 InStreamId, const FTmvMediaMuxerTrackConfig& InTrackConfig);

private:
	/** Buffered access unit waiting to be consumed by the muxer worker thread. */
	struct FBufferedSample
	{
		TSharedPtr<TArray64<uint8>> Data;
		uint32 SampleNumber = 0;
	};

	/** Try to fulfill pending muxer requests from the buffer. */
	void FlushPendingRequests();

	/** Extended stream info with container track mapping. */
	struct FContainerStreamInfo
	{
		/** Track index in the ITmvMediaMuxer (-1 if not yet added). */
		int32 MuxerTrackIndex = INDEX_NONE;

		/** Track configuration (set via SetStreamTrackConfig). */
		FTmvMediaMuxerTrackConfig TrackConfig;

		/** Whether track config has been set. */
		bool bHasTrackConfig = false;
	};

	TArray<FContainerStreamInfo> ContainerStreams;

	/** The underlying container muxer implementation. */
	TSharedPtr<ITmvMediaMuxer, ESPMode::ThreadSafe> Muxer;

	/** Whether the muxer has been started (ITmvMediaMuxer::Start called). */
	bool bMuxerStarted = false;
	
	/** Whether the muxer has failed to start and shouldn't be started again. */
	bool bMuxerFailed = false;

	/** Whether stop has been requested (signals final samples). */
	std::atomic<bool> bStopRequested = false;

	/**
	 * Per-stream buffer state for the push-to-pull bridge.
	 * One instance per stream, all fields protected by SampleBufferCS.
	 */
	struct FStreamBufferState
	{
		/** Buffered samples keyed by sample number. */
		TMap<uint32, FBufferedSample> Samples;

		/** Highest sample number received from the encoder (to detect last sample on stop). */
		uint32 HighestReceivedSampleNumber = 0;

		/** Set of sample numbers that have been successfully delivered to the muxer. */
		TSet<uint32> DeliveredSampleNumbers;
	};

	/**
	 * Critical section protecting StreamBuffers, MuxerTrackToStreamIndex,
	 * PendingRequests, bMuxerStarted and bMuxerFailed.
	 */
	FCriticalSection SampleBufferCS;

	/** Per-stream buffer state. One entry per stream opened with OpenStream. */
	TArray<TUniquePtr<FStreamBufferState>> StreamBuffers;

	/** Maps muxer track index to stream index. Populated in SetStreamTrackConfig. */
	TMap<int32, int32> MuxerTrackToStreamIndex;

	/** Pending requests from the muxer worker thread. May contain multiple requests per track. */
	TArray<ITmvMediaMuxer::FSampleRequest> PendingRequests;

	/** Event signaled by the muxer status delegate when muxing finishes or fails. */
	FEventRef MuxerFinishedEvent;
};

#undef UE_API
