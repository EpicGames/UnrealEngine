// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transcoder/TmvMediaFrameProducer.h"

#include "TmvMediaFileSequenceFrameProducer.generated.h"

/**
 * Transcoding Frame Producer Stage implementation for image file sequence using ImageCore.
 */
UCLASS(MinimalAPI)
class UTmvMediaFileSequenceFrameProducer : public UTmvMediaFrameProducer
{
	GENERATED_BODY()
public:
	UTmvMediaFileSequenceFrameProducer();
	
	//~ Begin UTmvMediaTranscodeStage
	virtual bool Start(UTmvMediaTranscodeJob* InParentJob) override;
	virtual void RequestStop(UTmvMediaTranscodeJob* InParentJob) override;
	//~ End UTmvMediaTranscodeStage

private:
	/** Main loop of the frame producer. */
	void ProcessAllImages(UTmvMediaTranscodeJob* InParentJob, const FString& InSequencePath);

	/** Process one image. */
	bool ProcessImage(UTmvMediaTranscodeJob* InParentJob, const FString& InImagePath, int32 InFirstFrameIndex);

	/** Returns true if the job is cancelled. */
	bool IsCancelled(const UTmvMediaTranscodeJob* InParentJob) const;

	/** Number of frame being currently processed (in flight). */
	std::atomic<int32> NumActiveFrames = 0;
};