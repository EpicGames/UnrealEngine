// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transcoder/TmvMediaTranscodeMuxer.h"

#include "TmvMediaFileSequenceTranscodeMuxer.generated.h"

#define UE_API TMVMEDIA_API

/**
 * File sequence muxer: writes each access unit as a separate file.
 * @see UTmvMediaTranscodeMuxer
 */
UCLASS(MinimalAPI)
class UTmvMediaFileSequenceTranscodeMuxer : public UTmvMediaTranscodeMuxer
{
	GENERATED_BODY()
public:
	UE_API virtual int32 OpenStream(UTmvMediaTranscodeJob* InParentJob, const FString& InStreamName, const FString& InExtension) override;

	UE_API virtual void ReceiveAccessUnit(
		UTmvMediaTranscodeJob* InParentJob,
		int32 InStreamId,
		const FTmvMediaFrameTimeInfo& TimeInfo,
		TSharedPtr<TArray64<uint8>>&& InAccessUnit) override;
};

#undef UE_API
