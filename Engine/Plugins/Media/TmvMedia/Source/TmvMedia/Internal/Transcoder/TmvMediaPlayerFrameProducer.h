// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Transcoder/TmvMediaFrameProducer.h"

#include "TmvMediaPlayerFrameProducer.generated.h"

class FTmvMediaPlayerFrameProducerResource;
class IMediaTextureSample;
class UMediaPlayer;
class UMediaSource;
class UMediaTexture;
enum class EMediaEvent;
struct FRHITextureDesc;
struct FTmvMediaFrameTimeInfo;

/**
 * Transcoding Frame Producer Stage implementation using a media player and media texture.
 * 
 * This will control the player using block on range in order to produce each frame of the sequence.
 * It also uses the media texture rendering callback to capture the rendered media samples and
 * feed it to the following transcoder stage (which should be the converter).
 *
 * Refactor notes:
 * - The player controller aspect could be made into its own class to be more reusable.
 * - The media texture sample capture could also be made into its own class, for the same reason.
 */
UCLASS(MinimalAPI)
class UTmvMediaPlayerFrameProducer : public UTmvMediaFrameProducer
{
	GENERATED_BODY()
public:
	UTmvMediaPlayerFrameProducer();

	//~ Begin UTmvMediaTranscodeStage
	virtual bool Start(UTmvMediaTranscodeJob* InParentJob) override;
	virtual void RequestStop(UTmvMediaTranscodeJob* InParentJob) override;
	virtual void Tick(UTmvMediaTranscodeJob* InParentJob, const FTmvMediaTranscodeJobTime& InTime) override;
	//~ End UTmvMediaTranscodeStage

protected:
	/** Sub-Tick function that handles the "playing" state. */
	void HandlePlaying(UTmvMediaTranscodeJob* InParentJob, float InDeltaTime);

	/** CLose the player and associated resources. */
	void Close(const UTmvMediaTranscodeJob* InParentJob);

	/** Queries playback range, frame rate, frame duration, from the player if available. */
	void TryUpdateMediaInfo();

	/** Queries the player for the playback range, if it is available, will update local value. */
	void TryUpdatePlaybackRange();

	/** Queries the player for the frame duration, if it is available, will update local value. */
	void TryUpdateFrameDuration();

	/** Internal MediaPlayer's OnMediaClosed callback function. */
	UFUNCTION()
	void OnMediaClosed();

	/** Internal MediaPlayer's OnMediaOpened callback function. */
	UFUNCTION()
	void OnMediaOpened(FString InURL);

	/** Internal MediaPlayer's OnMediaOpenFailed callback function. */
	UFUNCTION()
	void OnMediaOpenFailed(FString InURL);

	/** Internal MediaPlayer's OnEndReached callback function. */
	UFUNCTION()
	void OnMediaEndReached();

	/** Internal MediaPlayer's OnPlaybackResumed callback function. */
	UFUNCTION()
	void OnMediaResumed();

	/** Internal MediaPlayer's OnPlaybackSuspended callback function. */
	UFUNCTION()
	void OnMediaSuspended();

	/** Internal MediaPlayer's OnSeekCompleted callback function. */
	UFUNCTION()
	void OnMediaSeekCompleted();

	/** Internal MediaPlayer's OnTracksChanged callback function. */
	UFUNCTION()
	void OnMediaTrackChanged();

	/** Internal MediaPlayer's OnBufferingStart callback function. */
	UFUNCTION()
	void OnMediaBuffering();

	/** Internal MediaPlayer's OnBufferingCompleted callback function. */
	UFUNCTION()
	void OnMediaBufferingCompleted();

	/**
	 * Handler for the player's low-level event multicast (FOnMediaEvent). Unlike the UFUNCTION
	 * OnMediaOpened, this fires early enough during media open processing that a Seek issued
	 * here lands before any auto-emitted first sample. Mirrors the seek-on-open pattern used by
	 * FMovieSceneMediaData::HandleMediaPlayerEvent.
	 */
	void HandleMediaPlayerEvent(EMediaEvent Event);

	/** Used to read in media. */
	UPROPERTY()
	TObjectPtr<UMediaPlayer> MediaPlayer;

	/** Media Texture for the player. */
	UPROPERTY()
	TObjectPtr<UMediaTexture> MediaTexture;

	/** Source for the player. */
	UPROPERTY()
	TObjectPtr<UMediaSource> MediaSource;

	/** Render Resource for the frame producer. */
	TSharedPtr<FTmvMediaPlayerFrameProducerResource> ProducerResource;

	/** How long a frame is. */
	FTimespan MediaFrameDuration;

	/** Media Frame Rate */
	FFrameRate MediaFrameRate;

	/** Playback range of the media. */
	TRange<FTimespan> MediaPlaybackRange;

	/** What we consider to be the start time of the media (derived from the first decoded sample). */
	FTimespan MediaStartTime;
	
	/** The current time we are processing. */
	FTimespan CurrentTime;
	
	/** Keep track of the previous tick's rendered sample time to detect when new frames are rendered. */
	FTimespan PreviousRenderedSampleTime;

	/** Maximum number of concurrently processing frames. */
	int32 MaxProcessingFrames = 1;

	/**
	 * Once the media is completed (MediaCompleted state), this counter keeps track of how many frames still in flight
	 * we are waiting on before the job is completed (other stages).
	 */
	int32 LastSubmittedCount = 0;

	/** Enum for the frame producer's states. */
	enum class EState
	{
		Closed,
		Opening,
		Open,
		Preparing,
		Playing,
		MediaCompleted,
		NotSupported,
		Failed,
		TimedOut,
		Errored
	};

	/** Current state of the frame producer. */
	EState CurrentState = EState::Closed;

	/** Watch dog counter for states that can get stuck and time out. */
	float WatchdogTimeRemaining = 0.0f;
};