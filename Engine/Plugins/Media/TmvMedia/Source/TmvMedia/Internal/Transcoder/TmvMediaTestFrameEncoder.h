// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transcoder/TmvMediaFrameEncoder.h"
#include "TmvMediaTestFrameEncoder.generated.h"

/**
 * Implementation of an encoder stage that writes to files directly.
 * Bypass muxer stage.
 * For testing only.
 */
UCLASS(MinimalAPI)
class UTmvMediaTestFrameEncoder : public UTmvMediaFrameEncoder
{
	GENERATED_BODY()
public:
	//~ Begin UTmvMediaTranscodeStage
	virtual bool Start(UTmvMediaTranscodeJob* InParentJob) override;
	//~ End UTmvMediaTranscodeStage

	//~ Begin UTmvMediaFrameEncoder
	virtual void ReceiveMips(UTmvMediaTranscodeJob* InParentJob, TUniquePtr<FTmvMediaFrameMips>&& InMips) override;
	//~ End UTmvMediaFrameEncoder
};