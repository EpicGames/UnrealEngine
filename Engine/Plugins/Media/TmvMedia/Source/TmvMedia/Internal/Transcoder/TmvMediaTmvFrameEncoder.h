// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transcoder/TmvMediaFrameEncoder.h"
#include "TmvMediaTmvFrameEncoder.generated.h"

class FTmvMediaTmvEncoderPool;
class ITmvMediaEncoder;
class UTmvMediaContainerTranscodeMuxer;

/**
 * Implementation of an encoder stage of the transcode pipeline that wraps a TMV encoder.
 * @see UTmvMediaTranscodeJob
 */
UCLASS(MinimalAPI)
class UTmvMediaTmvFrameEncoder : public UTmvMediaFrameEncoder
{
	GENERATED_BODY()
public:
	//~ Begin UTmvMediaTranscodeStage
	virtual bool Start(UTmvMediaTranscodeJob* InParentJob) override;
	virtual void RequestStop(UTmvMediaTranscodeJob* InParentJob) override;
	//~ End UTmvMediaTranscodeStage

	//~ Begin UTmvMediaFrameEncoder
	virtual void SetEncoderOptions(const TInstancedStruct<FTmvMediaEncoderOptions>& InOptions) override;
	virtual const FTmvMediaEncoderOptions* GetEncoderOptions() const override;
	virtual bool RequestMipInfos(UTmvMediaTranscodeJob* InParentJob, const FTmvMediaFrameTimeInfo& InSourceTimeInfo, const FTmvMediaFrameMipInfo& InSourceMipInfo, TArray<FTmvMediaFrameMipInfo>& OutFrameMipInfo) override;
	virtual void ReceiveMips(UTmvMediaTranscodeJob* InParentJob, TUniquePtr<FTmvMediaFrameMips>&& InMips) override;
	//~ End UTmvMediaFrameEncoder

private:
	/** Encoder options, must be set before the job is started and can't be modified. */
	TInstancedStruct<FTmvMediaEncoderOptions> EncoderOptions;

	/** Encoder pool to keep encoder objects around between the worker threads. */
	TSharedPtr<FTmvMediaTmvEncoderPool> EncoderPool;

	/** StreamId when opening a stream with the muxer. */
	int32 StreamId = INDEX_NONE;

	/** Name of the currently opened stream. */
	FString StreamName;

	/** Pointer to container muxer if using container output (null for file sequence). */
	UPROPERTY(Transient)
	TObjectPtr<UTmvMediaContainerTranscodeMuxer> ContainerMuxer;

	/** Whether the track config has been set on the container muxer. */
	bool bContainerTrackConfigSet = false;
};
