// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Encoder/TmvMediaEncoderOptions.h"
#include "StructUtils/InstancedStruct.h"

#define UE_API TMVMEDIA_API

class FArchive;
class UTmvMediaTranscodeList;
struct FTmvMediaEncoderOptions;
struct FTmvMediaTranscodeJobSettings;
struct FTmvMediaTranscodeListItem;

namespace UE::TmvMedia::TranscodeSerialization
{
	/** Serialize the given job settings and options to the given archive in json format. */
	UE_API bool SerializeTranscodeJobSettingsToJson(
		FArchive& InArchive,
		const FTmvMediaTranscodeJobSettings* InJobSettings,
		const TInstancedStruct<FTmvMediaEncoderOptions>& InEncoderOptions);

	/** Deserialize the given job settings and options from the given archive in json format. */
	UE_API bool DeserializeTranscodeJobSettingsFromJson(
		FArchive& InArchive,
		FTmvMediaTranscodeJobSettings* OutJobSettings,
		TInstancedStruct<FTmvMediaEncoderOptions>& OutEncoderOptions);
	
	/** Serialize the given transcode list to the given archive in json format. */
	UE_API bool SerializeTranscodeListToJson(
		FArchive& InArchive,
		const UTmvMediaTranscodeList& InTranscodeList);

	/** Deserialize the given transcode list from the given archive in json format. */
	UE_API bool DeserializeTranscodeListFromJson(
		FArchive& InArchive,
		UTmvMediaTranscodeList& OutTranscodeList);

	/** Serialize the given transcode list items to the given archive in json format. */
	UE_API bool SerializeTranscodeListItemsToJson(
		FArchive& InArchive,
		const TArray<FTmvMediaTranscodeListItem>& InTranscodeListItems);

	/** Deserialize the given transcode list items from the given archive in json format. */
	UE_API bool DeserializeTranscodeListItemsFromJson(
		FArchive& InArchive,
		TArray<FTmvMediaTranscodeListItem>& OutTranscodeListItems);
}

#undef UE_API