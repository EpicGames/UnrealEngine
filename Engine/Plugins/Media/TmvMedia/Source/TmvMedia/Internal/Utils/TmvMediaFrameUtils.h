// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMathUtility.h"
#include "SampleConverter/TmvMediaFrameMipBufferFwd.h"
#include "TmvMediaFrameInfo.h"

#define UE_API TMVMEDIA_API

class IFileHandle;
enum EPixelFormat : uint8;
struct FImageInfo;
struct FImageView;

namespace UE::TmvMedia::FrameUtils
{
	/**
	 * Returns the YUV offset to apply in YUV <-> RGB conversion.
	 * @param InType Component type (float, int)
	 * @param InColorMatrixRange Matrix coefficient range (full, limited)
	 * @param InBitDepth Component bit depth 
	 */
	UE_API FVector GetYuvOffset(ETmvMediaFrameComponentType InType, ETmvMediaFrameColorMatrixRange InColorMatrixRange, uint8 InBitDepth);

	/**
	 * Returns the RGB to YUV conversion matrix (without the YUV offset). Only 3x3 sub-matrix is populated.
	 * @param InColorMatrix Specify Yuv matrix
	 * @param InColorMatrixRange Specify coefficient range
	 */
	UE_API FMatrix GetRgbToYuvMatrix(ETmvMediaFrameColorMatrix InColorMatrix, ETmvMediaFrameColorMatrixRange InColorMatrixRange);

	/**
	 * Returns the YUV to RGB conversion matrix (without the YUV offset). Only 3x3 sub-matrix is populated.
	 * @param InColorMatrix Specify Yuv matrix
	 * @param InColorMatrixRange Specify coefficient range
	 */
	UE_API FMatrix GetYuvToRgbMatrix(ETmvMediaFrameColorMatrix InColorMatrix, ETmvMediaFrameColorMatrixRange InColorMatrixRange);

	/**
	* Utility function to populate an FImageInfo from the FrameMipInfo if the format allows, i.e. single plane packed format.
	* @param InMipInfo Source frame mip info.
	* @param OutImageInfo Populated image info.
	* @param OutError Optional string populated with the specific reason for the incompatibility.
	* @return true if the format is supported by FImage, false otherwise.
	*/
	UE_API bool PopulateImageInfo(const FTmvMediaFrameMipInfo& InMipInfo, FImageInfo& OutImageInfo, FString* OutError = nullptr);

	/**
	 * Utility function to populate a FrameMipInfo from an ImageInfo.
	 * @param InMipLevel Mip level corresponding to this frame mip buffer
	 * @param InImageInfo Source image info
	 * @param OutMipInfo  Populated FrameMipInfo
	 */
	UE_API void PopulateMipInfoFromImageInfo(int32 InMipLevel, const FImageInfo& InImageInfo, FTmvMediaFrameMipInfo& OutMipInfo);
	
	/**
	* Get the packed pixel format exactly matching the plane's memory layout.
	* Mostly support 1, 2 and 4 component(s) packed formats. Mainly validated against DXGI formats.
	* @param InPlaneInfo Plane configuration
	* @param bInNormalized In case of an integer format indicate if the format should be normalized or not.
	* @return Request pixel format if supported, PF_Unknown if not.
	*/
	UE_API EPixelFormat GetPlanePixelFormat(const FTmvMediaFramePlaneInfo& InPlaneInfo, bool bInNormalized);

	/**
	 * Helper function to save mip buffers to file for inspection.
	 * The file format to use is internally determined depending on the mip buffer memory layout and
	 * color model.
	 * 
	 * @param InFilepath Destination file path, excluding extension.
	 * @param InMipBuffer Mip Buffer to save.
	 * @return true if operation succeeded.
	 */
	UE_API bool WriteMipBufferToFile(const FString& InFilepath, const FTmvMediaFrameMipBufferHandle& InMipBuffer);
}

/** Helper functions to debug YUV frame conversion. */
namespace UE::TmvMedia::FrameUtils::Y4M
{
	/** Write Y4M header to the given archive. */
	bool WriteHeader(IFileHandle& InArchive, const FTmvMediaFrameMipInfo& InMipInfo);

	/** Write the Y4M frame header (marker) to the given archive. */
	bool WriteFrameHeader(IFileHandle& InArchive);

	/** Write the Y4M frame body to the given archive. */
	bool WriteFrameBody(IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer);

	/** Write the Y4M frame (frame header + frame body) to the given archive. */
	bool WriteFrame(IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer);

	/** Returns true if the given frame format is supported by Y4M writer. */
	bool IsWriteFrameSupported(const FTmvMediaFrameMipInfo& InMipInfo);
}

/** Helper function to debug RGB frame conversion. */
namespace UE::TmvMedia::FrameUtils::Pfm
{
	/**
	 * Write the frame to the given archive in PFM (portable float map) format.
	 * @param InArchive file archive to write to.
	 * @param InImageView image view to write.
	 */
	bool WriteFrame(IFileHandle& InArchive, const FImageView& InImageView);

	/**
	 * Validate if the WriteFrame supports the given mip buffer format.
	 * @param MipInfo mip buffer format to validate.
	 * @param OutError Optional string populated with specific reason why format is not supported.
	 * @return true if the format is supported.
	 * @see WriteFrame
	 */
	bool IsWriteFrameSupported(const FTmvMediaFrameMipInfo& MipInfo, FString* OutError = nullptr);

	/**
	 * Write the frame to the given archive in PFM (portable float map) format.
	 * This function is extremely slow, use only for troubleshooting color conversions.
	 * 
	 * @param InArchive file archive to write to.
	 * @param InMipBuffer Mip buffer to write.
	 * @param OutError Optional parameter to be populated with an error message if the format is not supported.
	 * @remark Not all possible formats are supported, only the ones that convert trivially to rgb float16 or float32.
	 */
	bool WriteFrame(IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer, FString* OutError = nullptr);
}

#undef UE_API