// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

class UMediaSource;

#define UE_API TMVMEDIA_API

/**
 * Path utility functions.
 * We want the TmvMedia to work with the paths from File/Img Media Sources.
 */
namespace UE::TmvMedia::PathUtils
{
	/**
	 * Utility function to ensure the path is starting with a "./".
	 * This is the format of the img/file media source use for internal relative paths.
	 */
	UE_API FString EnsureStartWithDotSlash(const FString& InPath);

	/**
	 * Utility function to extract the full directory path from a given sanitized file or directory path.
	 * This function will work reliably for existing paths, otherwise, it will fall back to FPaths::GetPath.
	 * @remark Relative paths are assumed to be relative to the project directory, unless using explicit tokens.
	 * 
	 * @param InSanitizedPath Can be a file or directory path in the sanitized format.
	 */
	UE_API FString GetDirectoryFullPath(const FString& InSanitizedPath);

	/**
	 * Utility function to convert a given path (file or directory) to a sanitized version
	 * according to the tmv media supported paths format:
	 * - relative path under project folder (will start with ./)
	 * - absolute path if outside of project folder.
	 * - support path tokens, i.e. supported tokens will remain.
	 *
	 * @param InPath Input path, can be a file or folder.
	 * @return Sanitized path (file or folder).
	 */
	UE_API FString GetSanitizedPath(const FString& InPath);

	/**
	 * Utility function to convert the given path (file or directory, relative or absolute) to
	 * a full path. This is intended to be the inverse operation of GetSanitizedPath().
	 * 
	 * @param InSanitizedPath Path to convert to absolute, can be a directory or file, relative or absolute. Also support path tokens.
	 * @return Absolute path
	 */
	UE_API FString ConvertSanitizedPathToFull(const FString& InSanitizedPath);

	/**
	 * Utility function to make a filename from a given base name, possibly including tokens.
	 * 
	 * @param InBaseName Base name to use, may contain token {frame_number} (only token supported for now) 
	 * @param InFrameIndex Frame index (or frame number) of the frame.
	 * @param InZeroPadFrameNumbers Number of zeros to pad the frame number with.
	 * @param InExtension file extension to append.
	 * @return formatted filename with frame index and extension appended. 
	 */
	UE_API FString MakeFrameFilename(const FString& InBaseName, int32 InFrameIndex, int32 InZeroPadFrameNumbers, const FString& InExtension);

	/**
	 * Utility function to retrieve the path that a media source is pointing to.
	 * @param InMediaSource Media source
	 * @return cleaned up path to an existing file or folder, or empty string if not supported by the media source.
	 */
	UE_API FString GetMediaSourceMediaFullPath(const UMediaSource* InMediaSource);
}

#undef UE_API
