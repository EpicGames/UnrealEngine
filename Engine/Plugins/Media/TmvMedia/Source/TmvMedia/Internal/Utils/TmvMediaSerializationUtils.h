// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

#define UE_API TMVMEDIA_API

class FJsonObject;
class FText;
class UStruct;

namespace UE::TmvMedia::SerializationUtils
{
	/**
	 * Utility function to deserialize a struct object from json which supports InstancedStruct fields. 
	 */
	UE_API bool DeserializeFromJson(
		const TSharedRef<FJsonObject>& InJsonObject,
		const UStruct* InStruct,
		void* InNativeObject,
		FText& OutErrorMessage);

	/**
	 * Utility function to serialize a struct object to json which supports InstancedStruct fields. 
	 */
	UE_API TSharedPtr<FJsonObject> SerializeToJson(const UStruct* InStruct, const void* InObject);

	/** Converts a serialized value stored as raw bytes into a string. */
	UE_API FString BytesToString(const TArray<uint8>& InValueAsBytes);
	
	/** This ArrayView version is meant to work with FMemoryReaderView. */
	inline TArrayView<const uint8> ValueToConstBytesView(const FString& InValueAsString)
	{
		// Len() excludes the terminating character, and it is desired so it matches what FJsonStructSerializerBackend does.
		return TArrayView<const uint8>(reinterpret_cast<const uint8*>(*InValueAsString), InValueAsString.Len() * sizeof(TCHAR));
	}
}

#undef UE_API