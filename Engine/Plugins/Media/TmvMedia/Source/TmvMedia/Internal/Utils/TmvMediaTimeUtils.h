// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/FrameRate.h"

namespace UE::TmvMedia::TimeUtils
{
	/** 
	 * Compute the frame rate (num, den) for the given frame rate decimal value.
	 * This function will consider the frame rate with a fractional part as "drop frame".
	 */
	inline FFrameRate GetFrameRate(const double InFrameRate)
	{
		// Fractional frame rates are considered "drop frame"
		if (FMath::Fractional(InFrameRate) > UE_KINDA_SMALL_NUMBER)
		{
			return FFrameRate(FMath::CeilToInt(InFrameRate) * 1000, 1001);
		}
		return FFrameRate(InFrameRate, 1);
	}

	/**
	 * Compute the frame rate (num, den) for the given duration in seconds.
	 * 
	 * @param InDurationInSeconds Interval duration in seconds.
	 * @param InDefaultFrameRate Default frame rate returns in case duration is invalid.
	 * @return computed frame rate or default.
	 */
	inline FFrameRate ComputeFrameRateFromDuration(const double InDurationInSeconds, const FFrameRate& InDefaultFrameRate)
	{
		if (InDurationInSeconds > UE_DOUBLE_SMALL_NUMBER)
		{
			return GetFrameRate(1.0/InDurationInSeconds);
		}
		return InDefaultFrameRate;
	}
	
	/**
	 * Compute the frame rate (num, den) for the given interval duration.
	 * This function will consider the fractional frame rate as "drop frame".
	 * 
	 * @param InIntervalDuration interval duration
	 * @param InDefaultFrameRate Default frame rate returns in case duration is invalid.
	 * @return Computed frame rate or default.
	 */
	inline FFrameRate ComputeFrameRateFromDuration(const FTimespan& InIntervalDuration, const FFrameRate& InDefaultFrameRate)
	{
		return ComputeFrameRateFromDuration(InIntervalDuration.GetTotalSeconds(), InDefaultFrameRate);	
	}

	/**
	 * Returns the given media frame rate if valid, otherwise compute it from the given sample duration.
	 * @param InMediaFrameRate Media's frame rate determined by the player.
	 * @param InSampleDuration Current sample duration
	 * @return the resulting framerate, can be invalid.
	 */
	inline FFrameRate GetOrComputeFrameRate(const FFrameRate& InMediaFrameRate, const FTimespan& InSampleDuration)
	{
		if (InMediaFrameRate.IsValid())
		{
			return InMediaFrameRate;
		}
		return ComputeFrameRateFromDuration(InSampleDuration.GetTotalSeconds(), FFrameRate(0,0));
	}

	/**
	 * Snap a time-on-the-frame-grid (in seconds) to its nearest frame index at the given rate.
	 *
	 * FFrameRate::AsFrameNumber() floors, which under-counts by one when a sample time has lost
	 * sub-tick precision (e.g. 23.976 fps frames at N * 1001 / 24000 seconds can't be represented
	 * exactly in FTimespan's 100-ns ticks for most N). RoundToFrame on the FFrameTime split handles
	 * that case correctly.
	 */
	inline int32 RoundToFrameIndex(const FFrameRate& InFrameRate, double InSeconds)
	{
		return InFrameRate.AsFrameTime(InSeconds).RoundToFrame().Value;
	}
}
