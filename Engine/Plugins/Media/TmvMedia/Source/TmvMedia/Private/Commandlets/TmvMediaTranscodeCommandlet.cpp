// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaTranscodeCommandlet.h"

#include "AppMediaTimeSource.h"
#include "Encoder/ITmvMediaEncoderFactory.h"
#include "Encoder/TmvMediaEncoderOptions.h"
#include "Engine/Engine.h"
#include "HAL/FileManager.h"
#include "IMediaModule.h"
#include "ITmvMediaModule.h"
#include "Misc/App.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceNull.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "StructUtils/InstancedStruct.h"
#include "TmvMediaLog.h"
#include "Transcoder/ITmvMediaTranscodeJobRunner.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeJobBuilder.h"
#include "Transcoder/TmvMediaTranscodeList.h"
#include "Transcoder/TmvMediaTranscodeSerialization.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaTranscodeCommandlet)

namespace UE::TmvMedia
{
	/** Split `InPath` at the first '.' or '/' separator. Returns true if a separator was found. */
	bool SplitPropertyPath(FStringView InPath, FStringView& OutHead, FStringView& OutTail)
	{
		for (int32 Index = 0; Index < InPath.Len(); ++Index)
		{
			if (InPath[Index] == TEXT('.') || InPath[Index] == TEXT('/'))
			{
				OutHead = InPath.Left(Index);
				OutTail = InPath.RightChop(Index + 1);
				return true;
			}
		}
		OutHead = InPath;
		OutTail = FStringView();
		return false;
	}

	/** Result of attempting to apply a CLI parameter to a property path. */
	enum class EApplyParamResult
	{
		/** The path did not resolve to a property on the given struct (not an error). */
		NotFound,
		/** The path resolved but the value could not be imported. */
		ImportFailed,
		/** Value was imported successfully. */
		Success,
	};

	/**
	 * Recursively apply a value to a property identified by a dot/slash separated path within the given struct.
	 * e.g. "Muxer.Name" on FTmvMediaTranscodeJobSettings descends into the Muxer sub-struct and sets its Name field.
	 *
	 * FFilePath / FDirectoryPath leaves additionally accept a bare string value (mapped to their inner field)
	 * as a convenience so `-InputPath=C:\foo` works in addition to `-InputPath.FilePath=C:\foo`.
	 */
	EApplyParamResult ApplyParamPathToProperty(const UScriptStruct* InStructType, void* InStructData, FStringView InPath, const FString& InValue)
	{
		if (!InStructType || !InStructData || InPath.IsEmpty())
		{
			return EApplyParamResult::NotFound;
		}

		FStringView Head;
		FStringView Tail;
		const bool bHasNested = SplitPropertyPath(InPath, Head, Tail);

		FProperty* Property = InStructType->FindPropertyByName(FName(Head));
		if (!Property)
		{
			return EApplyParamResult::NotFound;
		}

		void* PropertyData = Property->ContainerPtrToValuePtr<void>(InStructData);

		// Nested path: descend into the sub-struct.
		if (bHasNested)
		{
			FStructProperty* StructProp = CastField<FStructProperty>(Property);
			if (!StructProp)
			{
				return EApplyParamResult::NotFound;
			}
			return ApplyParamPathToProperty(StructProp->Struct, PropertyData, Tail, InValue);
		}

		// Leaf: for FFilePath / FDirectoryPath accept a bare string (not a struct literal).
		if (FStructProperty* StructProp = CastField<FStructProperty>(Property); StructProp && !InValue.StartsWith(TEXT("(")))
		{
			static const FName FilePathStructName(TEXT("FilePath"));
			static const FName DirectoryPathStructName(TEXT("DirectoryPath"));
			const FName StructName = StructProp->Struct->GetFName();
			const FName InnerFieldName =
				(StructName == FilePathStructName) ? FName(TEXT("FilePath")) :
				(StructName == DirectoryPathStructName) ? FName(TEXT("Path")) :
				NAME_None;

			if (!InnerFieldName.IsNone())
			{
				if (FProperty* InnerProp = StructProp->Struct->FindPropertyByName(InnerFieldName))
				{
					void* InnerData = InnerProp->ContainerPtrToValuePtr<void>(PropertyData);
					FOutputDeviceNull NullOutput;
					if (InnerProp->ImportText_Direct(*InValue, InnerData, nullptr, PPF_None, &NullOutput) != nullptr)
					{
						return EApplyParamResult::Success;
					}
				}
			}
		}

		// Leaf: standard import. Silence the default GWarn output; we report our own diagnostics.
		FOutputDeviceNull NullOutput;
		return Property->ImportText_Direct(*InValue, PropertyData, nullptr, PPF_None, &NullOutput) != nullptr
			? EApplyParamResult::Success
			: EApplyParamResult::ImportFailed;
	}

	struct FApplyParamsStats
	{
		int32 NumSet = 0;
		int32 NumImportFailed = 0;
	};

	/**
	 * Apply command-line parameters to the properties of a UScriptStruct instance using reflection.
	 * Parameter keys may reference nested sub-fields using dot or slash notation, e.g. "-Muxer.Name=Tmv"
	 * or "-Muxer/Name=Tmv". Params that don't match any property path are silently ignored so the same
	 * map can be applied to multiple structs. Returns per-struct stats so callers can decide whether
	 * an import failure (e.g. user typo like -bUseMediaPlayer=truthy) should be fatal.
	 */
	FApplyParamsStats ApplyParamsToStructProperties(const TMap<FString, FString>& InParams, const UScriptStruct* InStructType, void* InStructData)
	{
		FApplyParamsStats Stats;
		if (!InStructType || !InStructData)
		{
			return Stats;
		}

		for (const TPair<FString, FString>& Param : InParams)
		{
			switch (ApplyParamPathToProperty(InStructType, InStructData, Param.Key, Param.Value))
			{
			case EApplyParamResult::Success:
				UE_LOGF(LogTmvMedia, Log, "Applied CLI property %ls.%ls = \"%ls\"", *InStructType->GetName(), *Param.Key, *Param.Value);
				++Stats.NumSet;
				break;
			case EApplyParamResult::ImportFailed:
				UE_LOGF(LogTmvMedia, Error, "Failed to import value \"%ls\" for property %ls.%ls.", *Param.Value, *InStructType->GetName(), *Param.Key);
				++Stats.NumImportFailed;
				break;
			case EApplyParamResult::NotFound:
				// Param key doesn't map to this struct; let another struct (or nothing) handle it.
				break;
			}
		}
		return Stats;
	}

	struct FCommandLineArguments
	{
		TArray<FString> Tokens;
		TArray<FString> Switches;
		TMap<FString, FString> Params;

		explicit FCommandLineArguments(const FString& InParameters)
		{
			UCommandlet::ParseCommandLine(*InParameters, Tokens, Switches, Params);
		}
	};

	/** Look up an encoder factory by exact name (case-insensitive), then by unique case-insensitive substring. */
	TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> FindEncoderFactoryByNameOrSubstring(ITmvMediaModule& InModule, const FString& InNameOrSubstring)
	{
		// Exact match (FName comparison is case-insensitive).
		if (TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> Exact = InModule.FindEncoderFactory(FName(*InNameOrSubstring)))
		{
			return Exact;
		}

		// Substring (case-insensitive) match over all registered factories.
		TArray<TWeakPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe>> Factories;
		InModule.GetEncoderFactories(Factories);

		TArray<TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe>> Matches;
		for (const TWeakPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe>& FactoryWeak : Factories)
		{
			if (TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> Factory = FactoryWeak.Pin())
			{
				if (Factory->GetName().ToString().Contains(InNameOrSubstring, ESearchCase::IgnoreCase))
				{
					Matches.Add(Factory);
				}
			}
		}

		if (Matches.Num() == 1)
		{
			// Surface fuzzy resolution so the user can confirm it picked what they meant.
			UE_LOGF(LogTmvMedia, Display, "Encoder \"%ls\" resolved to \"%ls\" via substring match.",
				*InNameOrSubstring, *Matches[0]->GetName().ToString());
			return Matches[0];
		}

		if (Matches.IsEmpty())
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't find an encoder factory matching \"%ls\".", *InNameOrSubstring);
		}
		else
		{
			TArray<FString> MatchNames;
			MatchNames.Reserve(Matches.Num());
			for (const TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe>& Factory : Matches)
			{
				MatchNames.Add(Factory->GetName().ToString());
			}
			UE_LOGF(LogTmvMedia, Error, "Ambiguous encoder name \"%ls\": matches [%ls]. Please disambiguate.", *InNameOrSubstring, *FString::Join(MatchNames, TEXT(", ")));
		}
		return nullptr;
	}

	/**
	 * Resolve default encoder options via the encoder factory for the given encoder name.
	 * Tries an exact (case-insensitive) match first, then falls back to a single substring match
	 * across registered factories (so "apv" or "exr" resolves even when the registered name has
	 * a version suffix or different casing).
	 */
	bool GetDefaultEncoderOptions(const FString& InEncoderName, TInstancedStruct<FTmvMediaEncoderOptions>& OutEncoderOptions)
	{
		ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get();
		if (!TmvMediaModule)
		{
			UE_LOGF(LogTmvMedia, Error, "TmvMediaModule Pointer is null.");
			return false;
		}

		TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> EncoderFactory = FindEncoderFactoryByNameOrSubstring(*TmvMediaModule, InEncoderName);
		if (!EncoderFactory)
		{
			return false;
		}

		EncoderFactory->GetEncoderOptions(OutEncoderOptions);
		if (!OutEncoderOptions.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't get encoder options from factory \"%ls\".", *EncoderFactory->GetName().ToString());
			return false;
		}
		UE_LOGF(LogTmvMedia, Log, "Transcode Job Default Encoder Options \"%ls\" selected.", *EncoderFactory->GetName().ToString());
		return true;
	}

	/** Mode 1: deserialize a full transcode list from a json file. */
	bool LoadJobListFromJson(const FString& InPath, UTmvMediaTranscodeList& OutList)
	{
		const TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InPath));
		if (!FileReader)
		{
			UE_LOGF(LogTmvMedia, Error, "Failed to open job list file \"%ls\".", *InPath);
			return false;
		}

		using namespace UE::TmvMedia::TranscodeSerialization;
		if (!DeserializeTranscodeListFromJson(*FileReader, OutList))
		{
			UE_LOGF(LogTmvMedia, Error, "Failed to deserialize job list from \"%ls\".", *InPath);
			return false;
		}

		UE_LOGF(LogTmvMedia, Log, "Transcode Job List loaded from \"%ls\".", *InPath);
		return true;
	}

	/** Mode 2: deserialize a single item from a json file and add it to the (empty) list. */
	bool LoadJobItemFromJson(const FString& InPath, UTmvMediaTranscodeList& OutList)
	{
		const TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InPath));
		if (!FileReader)
		{
			UE_LOGF(LogTmvMedia, Error, "Failed to open job item file \"%ls\".", *InPath);
			return false;
		}

		using namespace UE::TmvMedia::TranscodeSerialization;
		FTmvMediaTranscodeJobSettings JobSettings;
		TInstancedStruct<FTmvMediaEncoderOptions> EncoderOptions;	// Deserializer initializes.

		if (!DeserializeTranscodeJobSettingsFromJson(*FileReader, &JobSettings, EncoderOptions))
		{
			UE_LOGF(LogTmvMedia, Error, "Failed to deserialize job item from \"%ls\".", *InPath);
			return false;
		}

		OutList.InsertItemAt(0);
		if (FTmvMediaTranscodeListItem* Item = OutList.GetItemMutable(0))
		{
			Item->Name = FPaths::GetBaseFilename(InPath);
			Item->Settings = MoveTemp(JobSettings);
			Item->EncoderOptions = MoveTemp(EncoderOptions);
		}

		UE_LOGF(LogTmvMedia, Log, "Transcode Job Item loaded from \"%ls\".", *InPath);
		return true;
	}

	/** Mode 3: create a single item from command line parameters using property reflection. */
	bool BuildJobItemFromCommandLine(const TMap<FString, FString>& InParams, UTmvMediaTranscodeList& OutList)
	{
		static const FString EncoderParam = TEXT("Encoder");
		static const FString JobNameParam = TEXT("JobName");

		const FString* EncoderValue = InParams.Find(EncoderParam);
		if (!EncoderValue)
		{
			UE_LOGF(LogTmvMedia, Error, "Required parameter: \"%ls\" not specified.", *EncoderParam);
			return false;
		}

		// Resolve the encoder options via the factory (without any explicit dependency on derived types).
		TInstancedStruct<FTmvMediaEncoderOptions> EncoderOptions;
		if (!GetDefaultEncoderOptions(*EncoderValue, EncoderOptions))
		{
			UE_LOGF(LogTmvMedia, Error, "Specified Encoder: \"%ls\" not found.", *(*EncoderValue));
			return false;
		}

		// Apply reflection-based overrides to the job settings (e.g. -InputPath=..., -OutputPath=..., -bUseMediaPlayer=...).
		FTmvMediaTranscodeJobSettings JobSettings;
		const FApplyParamsStats SettingsStats = ApplyParamsToStructProperties(InParams, FTmvMediaTranscodeJobSettings::StaticStruct(), &JobSettings);

		// Default the muxer to the Tmv container muxer unless the user provided an explicit override.
		if (JobSettings.OutputFormat == ETmvMediaTranscodeOutputFormat::Container && JobSettings.Muxer.Name.IsNone())
		{
			static const FName DefaultContainerMuxerName(TEXT("Tmv"));
			JobSettings.Muxer.Name = DefaultContainerMuxerName;
		}

		// Apply reflection-based overrides to the encoder options (derived type accessed only by reflection).
		FApplyParamsStats EncoderStats;
		if (const UScriptStruct* EncoderStruct = EncoderOptions.GetScriptStruct())
		{
			EncoderStats = ApplyParamsToStructProperties(InParams, EncoderStruct, EncoderOptions.GetMutableMemory());
		}

		// Refuse to run with user typos: an unparseable -<Key>=<Value> would otherwise silently
		// fall back to the default, which is surprising for a batch tool driven by scripts.
		if (const int32 NumImportFailed = SettingsStats.NumImportFailed + EncoderStats.NumImportFailed; NumImportFailed > 0)
		{
			UE_LOGF(LogTmvMedia, Error, "Refusing to build job from command line: %d parameter value(s) failed to import.", NumImportFailed);
			return false;
		}

		OutList.InsertItemAt(0);
		FTmvMediaTranscodeListItem* Item = OutList.GetItemMutable(0);
		if (!Item)
		{
			UE_LOGF(LogTmvMedia, Error, "Failed to insert new job item in the transcode list.");
			return false;
		}

		if (const FString* JobNameValue = InParams.Find(JobNameParam))
		{
			Item->Name = *JobNameValue;
		}
		else
		{
			Item->Name = TEXT("CommandLineJob");
		}
		Item->Settings = MoveTemp(JobSettings);
		Item->EncoderOptions = MoveTemp(EncoderOptions);

		FString ValidationError;
		if (!UTmvMediaTranscodeList::ValidateJobItem(*Item, &ValidationError))
		{
			UE_LOGF(LogTmvMedia, Error, "Job item \"%ls\" is not configured to run:\n%ls", *Item->Name, *ValidationError);
			return false;
		}
		return true;
	}

	/** Populate the transcode list based on command line parameters. */
	bool BuildTranscodeList(const FCommandLineArguments& InArgs, UTmvMediaTranscodeList& OutList)
	{
		static const FString JobListParam = TEXT("JobList");
		static const FString JobItemParam = TEXT("JobItem");

		if (const FString* JobListPath = InArgs.Params.Find(JobListParam))
		{
			return LoadJobListFromJson(*JobListPath, OutList);
		}

		if (const FString* JobItemPath = InArgs.Params.Find(JobItemParam))
		{
			return LoadJobItemFromJson(*JobItemPath, OutList);
		}

		return BuildJobItemFromCommandLine(InArgs.Params, OutList);
	}
}

UTmvMediaTranscodeCommandlet::UTmvMediaTranscodeCommandlet()
{
	// Help metadata surfaced by -run=help <Commandlet>.
	HelpDescription = TEXT("Execute one or more TmvMedia transcode jobs from a json list, a json item, or command line parameters.");
	HelpUsage = TEXT("UnrealEditor.exe <Project.uproject> -run=TmvMediaTranscode -AllowCommandletRendering (-JobList=<path> | -JobItem=<path> | -Encoder=<name> [-<Property>=<Value> ...])");

	HelpParamNames.Add(TEXT("JobList"));
	HelpParamDescriptions.Add(TEXT("Path to a json file containing a serialized UTmvMediaTranscodeList (Using Tmv Transcoder UI). All items in the list are executed sequentially. Mutually exclusive with -JobItem and the command line job mode."));

	HelpParamNames.Add(TEXT("JobItem"));
	HelpParamDescriptions.Add(TEXT("Path to a json file containing a single serialized job item (settings + encoder options, using Tmv Transcoder UI, export job item). The item is wrapped in a one-item list and executed. Mutually exclusive with -JobList and the command line job mode."));

	HelpParamNames.Add(TEXT("Encoder"));
	HelpParamDescriptions.Add(TEXT("[Command line mode] Name or unique case-insensitive substring of a registered encoder factory (e.g. \"apv\", \"exr\"). Used to resolve the default encoder options. Required in command line mode."));

	HelpParamNames.Add(TEXT("JobName"));
	HelpParamDescriptions.Add(TEXT("[Command line mode] Optional display name assigned to the created job item. Defaults to \"CommandLineJob\"."));

	HelpParamNames.Add(TEXT("JobTimeoutSeconds"));
	HelpParamDescriptions.Add(TEXT("Optional per-job wall-clock timeout in seconds. When exceeded the job is cancelled and the runner proceeds to the next item. A non-zero value is required to enable the watchdog; 0 (default) disables it."));

	HelpParamNames.Add(TEXT("Debug"));
	HelpParamDescriptions.Add(TEXT("Raise the verbosity of selected media log categories (LogMediaAssets, LogMediaUtils) to VeryVerbose to aid debugging."));

	HelpParamNames.Add(TEXT("LoadModule"));
	HelpParamDescriptions.Add(TEXT("Comma-separated list of extra modules to load before executing the job (e.g. -LoadModule=\"Foo,Bar\"). Useful for pulling in additional encoder/muxer plugins that are not part of the default commandlet module set."));

	HelpParamNames.Add(TEXT("<Property>"));
	HelpParamDescriptions.Add(TEXT("[Command line mode] Any switch matching a property name of FTmvMediaTranscodeJobSettings or of the selected encoder options struct is imported via property reflection (e.g. -InputPath=<path>, -OutputPath=<path>, -OutputFormat=Container, -bUseMediaPlayer=false). Nested sub-fields are accessed with dot or slash notation (e.g. -Muxer.Name=Tmv, -StartTimecodeOverride.Hours=1). FFilePath and FDirectoryPath accept a bare string as a convenience."));

	// Start off with GIsClient set to false, so we don't init viewport etc., but turn it on later for netcode
	IsClient = false;
	IsServer = false;

	LogToConsole = true;
	ShowErrorCount = true;
	ShowProgress = true;
}

// Remark: We need to get the renderer going and modules we need have to be manually loaded.
// See UUnitTestCommandlet::Main
// This is called in EnginePreInit
int32 UTmvMediaTranscodeCommandlet::Main(const FString& InParams)
{
	if (IsEngineExitRequested())
	{
		UE_LOGF(LogTmvMedia, Log, "Transcode Commandlet: Engine exit requested.");
		return 0;
	}

	const UE::TmvMedia::FCommandLineArguments Arguments(InParams);

	if (Arguments.Switches.Contains(TEXT("help")) || Arguments.Switches.Contains(TEXT("?")))
	{
		PrintUsage();
		return 0;
	}


	// Required for the media player. Note: might be too late. This commandlet should be run with -AllowCommandletRendering.
	PRIVATE_GAllowCommandletRendering = true;
	//PRIVATE_GAllowCommandletAudio = true;	// todo: Might need audio for audio transcoding?

	// ---
	// Load selected modules that don't load for commandlets (RuntimeNoCommandlet).
	// LoadModule returns nullptr on failure; surface that at Warning so a missing plugin
	// doesn't cause a later cryptic failure during job construction.
	// ---
	auto LoadModuleOrWarn = [](const TCHAR* InModuleName)
	{
		if (!FModuleManager::Get().LoadModule(InModuleName))
		{
			UE_LOGF(LogTmvMedia, Warning, "Transcode Commandlet: Failed to load required module \"%ls\".", InModuleName);
		}
	};

	// Plugin: ElectraUtil
	LoadModuleOrWarn(TEXT("ElectraBase"));
	LoadModuleOrWarn(TEXT("ElectraSamples"));

	// Plugin: ElectraCodecs
	LoadModuleOrWarn(TEXT("ElectraDecoders"));
	LoadModuleOrWarn(TEXT("ElectraCodecFactory"));

	// Plugin: D3D12VideoDecodersElectra
	//LoadModuleOrWarn(TEXT("D3D12VideoDecodersElectra")); // already runtime.

	// Plugin: ElectraPlayer
	LoadModuleOrWarn(TEXT("ElectraPlayerRuntime"));
	LoadModuleOrWarn(TEXT("ElectraPlayerPlugin"));
	LoadModuleOrWarn(TEXT("ElectraPlayerFactory"));
	LoadModuleOrWarn(TEXT("ElectraProtron"));
	LoadModuleOrWarn(TEXT("ElectraProtronFactory"));

	// Plugin: TmvMedia
	LoadModuleOrWarn(TEXT("TmvMedia"));
	LoadModuleOrWarn(TEXT("ApvMedia"));
	LoadModuleOrWarn(TEXT("TmvMediaMp4Utils"));

	// Plugin: ImgMedia
	LoadModuleOrWarn(TEXT("ImgMedia"));
	LoadModuleOrWarn(TEXT("ImgMediaFactory"));
	LoadModuleOrWarn(TEXT("OpenExrWrapper"));

	// Optional -LoadModule="Foo, Bar, Baz" allows the user to pull in additional modules
	// (e.g. experimental encoder plugins) without having to recompile this commandlet.
	if (const FString* ExtraModulesValue = Arguments.Params.Find(TEXT("LoadModule")))
	{
		TArray<FString> ExtraModules;
		ExtraModulesValue->ParseIntoArray(ExtraModules, TEXT(","), /*InCullEmpty*/ true);
		for (FString& ModuleName : ExtraModules)
		{
			ModuleName.TrimStartAndEndInline();
			if (ModuleName.IsEmpty())
			{
				continue;
			}
			UE_LOGF(LogTmvMedia, Log, "Transcode Commandlet: Loading extra module \"%ls\".", *ModuleName);
			if (!FModuleManager::Get().LoadModule(FName(*ModuleName)))
			{
				UE_LOGF(LogTmvMedia, Warning, "Transcode Commandlet: Failed to load module \"%ls\".", *ModuleName);
			}
		}
	}

	if (IsEngineExitRequested())
	{
		UE_LOGF(LogTmvMedia, Log, "Transcode Commandlet: Engine exit requested.");
		return 0;
	}

	if (Arguments.Switches.Contains(TEXT("Debug")))
	{
		if (GEngine)
		{
			GEngine->Exec(nullptr, TEXT("log LogMediaAssets VeryVerbose"));
			GEngine->Exec(nullptr, TEXT("log LogMediaUtils VeryVerbose"));
		}
		else
		{
			UE_LOGF(LogTmvMedia, Warning, "Transcode Commandlet: GEngine is null; -Debug verbosity overrides will not be applied.");
		}
	}

#if !UE_SERVER // not sure why this is compiled out on server.
	static const FName MediaModuleName("Media");
	IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>(MediaModuleName);
	if (MediaModule != nullptr)
	{
		MediaModule->SetTimeSource(MakeShareable(new FAppMediaTimeSource));
	}
#else
	IMediaModule* MediaModule = nullptr;
#endif

	using namespace UE::TmvMedia;

	// Build the transcode list from the command line (file, item, or inline parameters).
	TStrongObjectPtr<UTmvMediaTranscodeList> TranscodeList(NewObject<UTmvMediaTranscodeList>(GetTransientPackage(), NAME_None));
	if (!BuildTranscodeList(Arguments, *TranscodeList))
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Commandlet: Failed to build job list.");
		return 1;
	}

	if (TranscodeList->GetNumItems() == 0)
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Commandlet: No job items to execute.");
		return 1;
	}

	ITmvMediaTranscodeJobRunner* Runner = ITmvMediaTranscodeJobRunner::Get();
	if (!Runner)
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Commandlet: Job runner is unavailable.");
		return 1;
	}

	// Optional per-job wall-clock timeout (off unless the caller specifies -JobTimeoutSeconds).
	FTmvMediaTranscodeJobRunOptions RunOptions;
	if (const FString* TimeoutValue = Arguments.Params.Find(TEXT("JobTimeoutSeconds")))
	{
		RunOptions.TimeoutSeconds = FCString::Atod(**TimeoutValue);
		if (RunOptions.TimeoutSeconds > 0.0)
		{
			UE_LOGF(LogTmvMedia, Display, "Transcode Commandlet: per-job timeout set to %.1fs.", RunOptions.TimeoutSeconds);
		}
	}

	for (int32 Index = 0; Index < TranscodeList->GetNumItems(); ++Index)
	{
		const FTmvMediaTranscodeListItem& Item = TranscodeList->GetItem(Index);
		UTmvMediaTranscodeJob* NewJob = FTmvMediaTranscodeJobBuilder(Item).Build();
		if (!NewJob)
		{
			UE_LOGF(LogTmvMedia, Error, "Transcode Commandlet: Failed to build job from item \"%ls\".", *Item.Name);
			continue;
		}
		Runner->EnqueueJob(NewJob, RunOptions);
	}

	// If every Start() failed synchronously (or no items were built), bail out before entering the loop.
	if (!Runner->HasActiveOrPendingJobs())
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Commandlet: No jobs successfully started - exiting.");
		return 1;
	}

	UE_LOGF(LogTmvMedia, Log, "Transcode Commandlet: %d job(s) enqueued.", TranscodeList->GetNumItems());

	// Subscribed after the enqueue loop so a future change that broadcasts OnAllJobsFinished from
	// a synchronous EnqueueJob path can't cause RequestEngineExit to fire mid-submission. The
	// queue can only drain via Tick from this point onwards, which only runs after we enter the
	// main loop below.
	const FDelegateHandle OnAllDoneHandle = Runner->GetOnAllJobsFinished().AddLambda([]()
	{
		UE_LOGF(LogTmvMedia, Display, "Transcode Job Runner: All jobs done - exiting.");
		RequestEngineExit(TEXT("TmvMediaTranscodeCommandlet: All jobs done."));
	});

	GIsRunning = true;

	// We shouldn't need a game viewport (?), nor world for media transcoding jobs.
	// This is taken from UUnitTestCommandlet, I don't know if it is needed.
	GIsClient = true;	// todo: try to remove this and see if it still works...

	double LastTime = FPlatformTime::Seconds();

	// Main thread throttling is probably not desired for transcode jobs.
	constexpr float IdealFrameRate = 120.0;	// We are unlikely to transcode faster than 120 fps. It would be pretty good!
	constexpr float IdealFrameTime = 1.0f / IdealFrameRate;

	// Inspired from: LiveLinkHubLoop and UFileServerCommandlet
	while (GIsRunning && !IsEngineExitRequested())
	{
		const double CurrentTime = FPlatformTime::Seconds();
		const double DeltaTime = CurrentTime - LastTime;

		if (MediaModule != nullptr)
		{
			MediaModule->TickPreEngine();
		}

		FApp::SetDeltaTime(DeltaTime);
		if (GEngine)
		{
			GEngine->UpdateTimeAndHandleMaxTickRate();
		}

		CommandletHelpers::TickEngine(nullptr, DeltaTime);

		// Run garbage collection for the UObjects for the rest of the frame or at least to 2 ms
		IncrementalPurgeGarbage(true, FMath::Max<float>(0.002f, IdealFrameTime - (FPlatformTime::Seconds() - LastTime)));

		if (MediaModule != nullptr)
		{
			MediaModule->TickPostEngine();
			MediaModule->TickPostRender();
		}

		FCoreDelegates::OnEndFrame.Broadcast();	// From LiveLinkHubLoop, don't know if necessary.

		// Throttle main thread main fps by sleeping if we still have time
		FPlatformProcess::Sleep(FMath::Max<float>(0.0f, IdealFrameTime - (FPlatformTime::Seconds() - LastTime)));

		LastTime = CurrentTime;
	}

	GIsRunning = false;
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

	Runner->GetOnAllJobsFinished().Remove(OnAllDoneHandle);

	return GWarn->GetNumErrors() == 0 ? 0 : 1;
}

void UTmvMediaTranscodeCommandlet::PrintUsage() const
{
	UE_LOGF(LogTmvMedia, Display, "%ls", *HelpDescription);
	UE_LOGF(LogTmvMedia, Display, "%ls", *HelpUsage);
	const int32 NumOptions = HelpParamNames.Num();
	for (int32 Idx = 0; Idx < NumOptions; ++Idx)
	{
		UE_LOGF(LogTmvMedia, Display, "-%ls\t%ls", *HelpParamNames[Idx], *HelpParamDescriptions[Idx]);
	}
}
