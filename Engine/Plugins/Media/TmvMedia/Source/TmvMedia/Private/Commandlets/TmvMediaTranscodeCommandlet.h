// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Commandlets/Commandlet.h"
#include "TmvMediaTranscodeCommandlet.generated.h"

/**
 * Execute TmvMedia transcode jobs headlessly from a commandlet.
 *
 * Usage:
 *   UnrealEditor.exe <Project.uproject> -run=TmvMediaTranscode -AllowCommandletRendering [options]
 *
 * Job source (pick one):
 *   -JobList=<path>      Load a serialized UTmvMediaTranscodeList from json; runs every item.
 *   -JobItem=<path>      Load a single serialized job item from json; wrapped in a one-item list.
 *   (otherwise)          Build a single job item from the command line. See "CLI job" below.
 *
 * CLI job:
 *   -Encoder=<name>      Required. Name or unique case-insensitive substring of a registered
 *                        encoder factory (e.g. "apv", "exr"). Selects the encoder options struct.
 *   -JobName=<name>      Optional display name for the item. Defaults to "CommandLineJob".
 *   -<Property>=<Value>  Any property of FTmvMediaTranscodeJobSettings or of the selected encoder
 *                        options struct, set via reflection. Sub-fields use dot or slash notation.
 *                        Examples: -InputPath=<path>, -OutputPath=<path>, -OutputFormat=Container,
 *                        -Muxer.Name=Tmv, -bUseMediaPlayer=false. FFilePath and FDirectoryPath
 *                        properties accept a bare string.
 *   After parsing, the item is validated (input/output paths set, and OutputAssetDirectory
 *   consistent with bMakeOutputAsset).
 *
 * Other switches:
 *   -JobTimeoutSeconds=<s>  Optional per-job wall-clock timeout. When exceeded the job is
 *                           cancelled and the runner proceeds to the next item. 0 (default)
 *                           disables the watchdog.
 *   -Debug               Raise LogMediaAssets and LogMediaUtils verbosity to VeryVerbose.
 *   -LoadModule="A,B,…"  Comma-separated list of extra modules to load before running
 *                        (e.g. additional encoder / muxer plugins).
 *   -help, -?            Print usage and exit.
 */
UCLASS()
class UTmvMediaTranscodeCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	/** Default Constructor */
	UTmvMediaTranscodeCommandlet();

	//~ Begin UCommandlet
	virtual int32 Main(const FString& InParams) override;
	//~ End UCommandlet

private:
	void PrintUsage() const;
};
