// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvSparseOutputBackendD3D12.h"

#include "HAL/PlatformTime.h"
#include "Misc/OutputDevice.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "RHIResources.h"
#include "TmvMediaFrameInfo.h"
#include "TmvMediaLog.h"
#include "TmvMediaTuningCVars.h"

namespace
{
	struct FTmvSparseOutputBackendD3D12AutoRegister
	{
		FTmvSparseOutputBackendD3D12AutoRegister()
		{
			FTmvSparseOutputBackendFactory::Get()->RegisterBackendForRhi(
				ERHIInterfaceType::D3D12,
				&FTmvSparseOutputBackendD3D12::CreateInstance);
		}
	};

	// Static-init self-registration. The platform-folder UBT exclusion guarantees
	// this object only exists in modules built for Windows targets.
	static FTmvSparseOutputBackendD3D12AutoRegister GAutoRegister;

	// Exponential growth schedule for the heap pool, in MB. After the schedule
	// is exhausted we keep allocating 512 MB heaps up to the total budget. This
	// gives the cheap-start property of grow-on-demand without paying a heap
	// allocation on every commit, and keeps a hard ceiling on each heap's
	// blast radius.
	constexpr int32 GHeapGrowthStepsMB[] = { 64, 128, 256, 512 };
	constexpr int32 GMaxHeapSizeMB = 512;
}

TSharedPtr<FTmvSparseOutputBackend, ESPMode::ThreadSafe> FTmvSparseOutputBackendD3D12::CreateInstance()
{
	return MakeShared<FTmvSparseOutputBackendD3D12, ESPMode::ThreadSafe>();
}

FTmvSparseOutputBackendD3D12::FTmvSparseOutputBackendD3D12() = default;
FTmvSparseOutputBackendD3D12::~FTmvSparseOutputBackendD3D12() = default;

bool FTmvSparseOutputBackendD3D12::IsRenderTargetOrDepthStencil(FRHITexture* Texture) const
{
	if (!Texture)
	{
		return false;
	}
	const ETextureCreateFlags Flags = Texture->GetDesc().Flags;
	return EnumHasAnyFlags(Flags, ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::DepthStencilTargetable);
}

int32 FTmvSparseOutputBackendD3D12::AllocateHeap_NoLock(int32 MinTiles, bool bRenderTargetable, bool bForceAllocation)
{
	if (!IsRHID3D12())
	{
		return INDEX_NONE;
	}

	ID3D12Device* Device = GetID3D12DynamicRHI()->RHIGetDevice(0);
	if (!Device)
	{
		return INDEX_NONE;
	}

	// Pick heap size from growth schedule, capped at 512 MB. Must hold MinTiles tiles.
	const int32 StepIdx = FMath::Min(NextHeapGrowthStep, static_cast<int32>(UE_ARRAY_COUNT(GHeapGrowthStepsMB)) - 1);
	const int32 SizeMB = (NextHeapGrowthStep < UE_ARRAY_COUNT(GHeapGrowthStepsMB))
		? GHeapGrowthStepsMB[StepIdx]
		: GMaxHeapSizeMB;
	int64 HeapBytes = static_cast<int64>(SizeMB) * 1024ll * 1024ll;

	// Ensure heap is large enough to satisfy this request even if MinTiles
	// is unusually large (e.g. mip 0 of a 16K texture needs many tiles).
	const int64 RequiredBytes = static_cast<int64>(MinTiles) * TileSizeInBytes;
	HeapBytes = FMath::Max(HeapBytes, RequiredBytes);

	// Budget gate: refuse to grow if it would push the pool over budget, so
	// CommitMip can fall through to force-steal. Always-resident pre-commits
	// at RegisterOutput pass bForceAllocation=true since they have no
	// stealable fallback.
	if (!bForceAllocation)
	{
		const int64 BudgetBytes = static_cast<int64>(UE::TmvMedia::GetSparseOutputMipBudgetMB()) * 1024ll * 1024ll;
		if (BudgetBytes > 0)
		{
			int64 ExistingPoolBytes = 0;
			for (const FHeapEntry& H : HeapPool)
			{
				ExistingPoolBytes += static_cast<int64>(H.TilesInHeap) * TileSizeInBytes;
			}
			if (ExistingPoolBytes + HeapBytes > BudgetBytes)
			{
				return INDEX_NONE;
			}
		}
	}
	const int32 TilesInHeap = static_cast<int32>(HeapBytes / TileSizeInBytes);

	D3D12_HEAP_PROPERTIES Props = {};
	Props.Type = D3D12_HEAP_TYPE_DEFAULT;
	Props.CPUPageProperty = D3D12_CPU_PAGE_PROPERTY_UNKNOWN;
	Props.MemoryPoolPreference = D3D12_MEMORY_POOL_UNKNOWN;
	Props.CreationNodeMask = 0;
	Props.VisibleNodeMask = 0;

	D3D12_HEAP_DESC Desc = {};
	Desc.SizeInBytes = static_cast<UINT64>(HeapBytes);
	Desc.Properties = Props;
	Desc.Alignment = D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT;
	// Heap kind. bRenderTargetable is always true in this backend: RegisterOutput
	// rejects non-RT/DS textures precisely so the shared pool stays a single heap
	// kind and tiles remain interchangeable across textures (see
	// AllocateSpans_NoLock). The non-RT/DS branch is retained only to keep this
	// helper self-contained; it is unreachable given that guard.
	Desc.Flags = bRenderTargetable
		? D3D12_HEAP_FLAG_ALLOW_ONLY_RT_DS_TEXTURES
		: D3D12_HEAP_FLAG_ALLOW_ONLY_NON_RT_DS_TEXTURES;

	ID3D12Heap* RawHeap = nullptr;
	const HRESULT Hr = Device->CreateHeap(&Desc, IID_PPV_ARGS(&RawHeap));
	if (FAILED(Hr) || !RawHeap)
	{
		UE_LOGF(LogTmvMedia, Warning,
			"FTmvSparseOutputBackendD3D12: CreateHeap failed (size=%lld bytes, hr=0x%08x)",
			HeapBytes, static_cast<uint32>(Hr));
		return INDEX_NONE;
	}

	FHeapEntry Entry;
	Entry.Heap = RawHeap;
	RawHeap->Release(); // TRefCountPtr holds its own reference.
	Entry.TilesInHeap = TilesInHeap;
	Entry.NumFreeTiles = TilesInHeap;
	Entry.FreeTiles.Init(true, TilesInHeap);
	const int32 NewIdx = HeapPool.Add(MoveTemp(Entry));

	++NextHeapGrowthStep;
	return NewIdx;
}

bool FTmvSparseOutputBackendD3D12::AllocateSpans_NoLock(int32 NumTiles, bool bRenderTargetable, TArray<FMipHeapSpan>& OutSpans, bool bForceAllocation)
{
	if (NumTiles <= 0)
	{
		return true;
	}

	// Heap-kind safety: tiles are assigned (and later stolen/rebound) from any heap
	// to any texture without per-heap kind tracking. This is valid only because
	// RegisterOutput rejects non-RT/DS textures, so every heap in the pool is created
	// ALLOW_ONLY_RT_DS_TEXTURES and is compatible with every registered resource. If
	// a non-RT/DS reserved resource is ever allowed, this walk (and FindStealableMip)
	// must filter heaps by kind.
	int32 Remaining = NumTiles;
	while (Remaining > 0)
	{
		// Walk existing heaps first; take the longest contiguous run we can find.
		int32 BestHeap = INDEX_NONE;
		int32 BestStart = 0;
		int32 BestLen = 0;
		for (int32 hi = 0; hi < HeapPool.Num(); ++hi)
		{
			FHeapEntry& Heap = HeapPool[hi];
			if (Heap.NumFreeTiles == 0)
			{
				continue;
			}
			int32 BestRunStart = 0;
			int32 BestRunLen = 0;
			int32 RunStart = -1;
			for (int32 ti = 0; ti < Heap.TilesInHeap; ++ti)
			{
				if (Heap.FreeTiles[ti])
				{
					if (RunStart < 0)
					{
						RunStart = ti;
					}
					const int32 RunLen = ti - RunStart + 1;
					if (RunLen > BestRunLen)
					{
						BestRunLen = RunLen;
						BestRunStart = RunStart;
						if (BestRunLen >= Remaining)
						{
							break;
						}
					}
				}
				else
				{
					RunStart = -1;
				}
			}
			if (BestRunLen > BestLen)
			{
				BestLen = BestRunLen;
				BestStart = BestRunStart;
				BestHeap = hi;
				if (BestLen >= Remaining)
				{
					break;
				}
			}
		}

		if (BestHeap == INDEX_NONE)
		{
			// No free space anywhere; grow.
			const int32 NewHeapIdx = AllocateHeap_NoLock(Remaining, bRenderTargetable, bForceAllocation);
			if (NewHeapIdx == INDEX_NONE)
			{
				// Rewind partial allocations (caller will see false and clean up its own spans, but
				// we already pushed into OutSpans). Free what we did.
				for (const FMipHeapSpan& Span : OutSpans)
				{
					FHeapEntry& H = HeapPool[Span.HeapIndex];
					for (int32 ti = Span.HeapStartTile; ti < Span.HeapStartTile + Span.TileCount; ++ti)
					{
						H.FreeTiles[ti] = true;
					}
					H.NumFreeTiles += Span.TileCount;
				}
				OutSpans.Reset();
				return false;
			}
			continue;
		}

		const int32 TakeCount = FMath::Min(BestLen, Remaining);
		FHeapEntry& Heap = HeapPool[BestHeap];
		for (int32 ti = BestStart; ti < BestStart + TakeCount; ++ti)
		{
			Heap.FreeTiles[ti] = false;
		}
		Heap.NumFreeTiles -= TakeCount;

		FMipHeapSpan Span;
		Span.HeapIndex = BestHeap;
		Span.HeapStartTile = BestStart;
		Span.TileCount = TakeCount;
		OutSpans.Add(Span);
		Remaining -= TakeCount;
	}
	return true;
}

void FTmvSparseOutputBackendD3D12::FreeSpans_NoLock(TConstArrayView<FMipHeapSpan> Spans)
{
	for (const FMipHeapSpan& Span : Spans)
	{
		if (!HeapPool.IsValidIndex(Span.HeapIndex))
		{
			continue;
		}
		FHeapEntry& Heap = HeapPool[Span.HeapIndex];
		for (int32 ti = Span.HeapStartTile; ti < Span.HeapStartTile + Span.TileCount; ++ti)
		{
			if (Heap.FreeTiles.IsValidIndex(ti))
			{
				Heap.FreeTiles[ti] = true;
			}
		}
		Heap.NumFreeTiles += Span.TileCount;
	}
	// Note: heap release moved to UnregisterOutput. Spans freed mid-session
	// (e.g. when an always-resident mip is unmapped during teardown) stay
	// inside the pool until the next unregister sweep.
}

void FTmvSparseOutputBackendD3D12::ReleaseEmptyTrailingHeaps_NoLock()
{
	while (HeapPool.Num() > 0 && HeapPool.Last().NumFreeTiles == HeapPool.Last().TilesInHeap)
	{
		HeapPool.Pop();
		if (NextHeapGrowthStep > 0)
		{
			--NextHeapGrowthStep;
		}
	}
}

void FTmvSparseOutputBackendD3D12::SubmitMappingForSpan(
	ID3D12Resource* Resource,
	uint32 Subresource,
	uint32 MipStartTileOffset,
	uint32 MipWidthInTiles,
	uint32 MipHeightInTiles,
	ID3D12Heap* Heap,
	uint32 HeapStartTile,
	uint32 TileCount,
	bool bCommit)
{
	if (!IsRHID3D12() || !Resource || TileCount == 0)
	{
		return;
	}

	D3D12_TILED_RESOURCE_COORDINATE Coord = {};
	Coord.Subresource = Subresource;
	// Linear (UseBox=false) addressing within the subresource: starting tile in row-major order.
	if (MipWidthInTiles > 0)
	{
		Coord.X = MipStartTileOffset % MipWidthInTiles;
		const uint32 RowsTraversed = MipStartTileOffset / MipWidthInTiles;
		Coord.Y = (MipHeightInTiles > 0) ? (RowsTraversed % MipHeightInTiles) : 0;
		Coord.Z = (MipHeightInTiles > 0) ? (RowsTraversed / MipHeightInTiles) : 0;
	}
	else
	{
		// Packed-mip block addressing: origin (0,0,0) is the entire packed region.
		Coord.X = 0;
		Coord.Y = 0;
		Coord.Z = 0;
	}

	D3D12_TILE_REGION_SIZE RegionSize = {};
	RegionSize.NumTiles = TileCount;
	RegionSize.UseBox = false;

	const D3D12_TILE_RANGE_FLAGS Flag = bCommit ? D3D12_TILE_RANGE_FLAG_NONE : D3D12_TILE_RANGE_FLAG_NULL;
	const UINT HeapOffset = HeapStartTile;
	const UINT RangeCount = TileCount;

	// AddRef both COM objects so the captured lambda extends their lifetime past
	// any caller-side release: RHIRunOnQueue is async and the lambda fires on the
	// D3D12 submission thread, possibly long after the caller's stack returns.
	TRefCountPtr<ID3D12Resource> ResourceRef(Resource);
	TRefCountPtr<ID3D12Heap>     HeapRef(Heap);

	GetID3D12DynamicRHI()->RHIRunOnQueue(
		ED3D12RHIRunOnQueueType::Graphics,
		[ResourceRef, Coord, RegionSize, HeapRef, Flag, HeapOffset, RangeCount, bCommit](ID3D12CommandQueue* Queue)
		{
			if (!Queue)
			{
				return;
			}
			Queue->UpdateTileMappings(
				ResourceRef,
				1, /*NumResourceRegions*/
				&Coord,
				&RegionSize,
				bCommit ? HeapRef.GetReference() : nullptr,
				1, /*NumRanges*/
				&Flag,
				&HeapOffset,
				&RangeCount,
				D3D12_TILE_MAPPING_FLAG_NONE);
		},
		/*bWaitForSubmission*/ false);
}

bool FTmvSparseOutputBackendD3D12::RegisterOutput(FRHITexture* Texture, const FTmvSparseOutputLayout& Layout)
{
	if (!Texture || !IsRHID3D12())
	{
		return false;
	}

	// Heap-kind invariant. The shared heap pool assigns and steals tiles across
	// textures with no per-heap kind tracking (see AllocateSpans_NoLock /
	// FindStealableMip_NoLock), which is only valid if every heap permits the same
	// resource kind. All sparse-output textures are the media intermediate render
	// target (RenderTargetable + UAV + ShaderResource + ReservedResource), so every
	// heap is created ALLOW_ONLY_RT_DS_TEXTURES and interchange is trivially
	// compatible. Reject a non-RT/DS reserved texture rather than silently mixing it
	// into RT/DS heaps and issuing invalid UpdateTileMappings; the caller keeps it
	// non-sparse (the documented registration-failure fallback).
	if (!IsRenderTargetOrDepthStencil(Texture))
	{
		UE_LOGF(LogTmvMedia, Warning,
			"FTmvSparseOutputBackendD3D12: refusing to register non-RT/DS reserved texture; "
			"the shared heap pool assumes a single (RT/DS) heap kind. Texture stays non-sparse.");
		return false;
	}

	ID3D12DynamicRHI* DynRHI = GetID3D12DynamicRHI();
	ID3D12Device* Device = DynRHI->RHIGetDevice(0);
	ID3D12Resource* Resource = DynRHI->RHIGetResource(Texture);
	if (!Device || !Resource)
	{
		return false;
	}

	// Query D3D12 tiling layout for this resource.
	UINT D3DResourceNumTiles = 0;
	D3D12_PACKED_MIP_INFO PackedMipInfo = {};
	D3D12_TILE_SHAPE TileShape = {};
	UINT NumSubresourceTilings = static_cast<UINT>(Layout.NumMips);
	TArray<D3D12_SUBRESOURCE_TILING, TInlineAllocator<16>> SubTilings;
	SubTilings.SetNum(NumSubresourceTilings);

	Device->GetResourceTiling(
		Resource,
		&D3DResourceNumTiles,
		&PackedMipInfo,
		&TileShape,
		&NumSubresourceTilings,
		0,
		SubTilings.GetData());

	if (NumSubresourceTilings == 0)
	{
		return false;
	}

	FScopeLock ScopeLock(&Lock);
	FTextureEntry Entry;
	Entry.Texture = Texture;
	Entry.Resource = Resource;
	Entry.Layout = Layout;
	Entry.NumStandardMips = PackedMipInfo.NumStandardMips;
	Entry.NumPackedMips = PackedMipInfo.NumPackedMips;
	Entry.NumTilesForPackedMips = PackedMipInfo.NumTilesForPackedMips;
	Entry.PackedMipStartTileIndex = PackedMipInfo.StartTileIndexInOverallResource;
	Entry.Mips.SetNum(Layout.NumMips);

	for (int32 MipIndex = 0; MipIndex < Layout.NumMips; ++MipIndex)
	{
		FMipMapping& Mip = Entry.Mips[MipIndex];
		if (static_cast<uint32>(MipIndex) < PackedMipInfo.NumStandardMips)
		{
			const D3D12_SUBRESOURCE_TILING& Tiling = SubTilings[MipIndex];
			Mip.WidthInTiles = static_cast<int32>(Tiling.WidthInTiles);
			Mip.HeightInTiles = static_cast<int32>(Tiling.HeightInTiles);
			Mip.DepthInTiles = static_cast<int32>(Tiling.DepthInTiles);
			Mip.NumTiles = Mip.WidthInTiles * Mip.HeightInTiles * Mip.DepthInTiles;
			Mip.bPackedMip = false;
		}
		else
		{
			Mip.bPackedMip = true;
			// All packed mips share one block; we represent them via the first packed-mip entry.
		}
	}

	// Pre-commit packed mip tail: a single block, always-resident, never decommitted.
	const bool bRenderTargetable = IsRenderTargetOrDepthStencil(Texture);
	if (PackedMipInfo.NumPackedMips > 0 && PackedMipInfo.NumTilesForPackedMips > 0)
	{
		FMipMapping& FirstPacked = Entry.Mips[PackedMipInfo.NumStandardMips];
		FirstPacked.NumTiles = static_cast<int32>(PackedMipInfo.NumTilesForPackedMips);
		if (!AllocateSpans_NoLock(FirstPacked.NumTiles, bRenderTargetable, FirstPacked.Spans, /*bForceAllocation=*/true))
		{
			UE_LOGF(LogTmvMedia, Warning,
				"FTmvSparseOutputBackendD3D12: failed to allocate packed-mip backing (%d tiles).",
				FirstPacked.NumTiles);
			return false;
		}
		FirstPacked.bCommitted = true;
		FirstPacked.bAlwaysResident = true; // packed tail is never stealable
		FirstPacked.CommittedBytes = static_cast<int64>(FirstPacked.NumTiles) * TileSizeInBytes;
		TotalCommittedBytes += FirstPacked.CommittedBytes;

		// Issue mappings for the packed-mip block. For the packed tail, D3D12
		// addresses the whole packed region with subresource = first packed-mip
		// subresource and Coord (0,0,0) - see D3D12 docs and the reference
		// implementation in D3D12Resources.cpp:373.
		uint32 Offset = 0;
		for (const FMipHeapSpan& Span : FirstPacked.Spans)
		{
			const FHeapEntry& Heap = HeapPool[Span.HeapIndex];
			SubmitMappingForSpan(
				Resource,
				PackedMipInfo.NumStandardMips,
				Offset,
				/*MipWidthInTiles=*/0,
				/*MipHeightInTiles=*/0,
				Heap.Heap.GetReference(),
				static_cast<uint32>(Span.HeapStartTile),
				static_cast<uint32>(Span.TileCount),
				/*bCommit=*/true);
			Offset += static_cast<uint32>(Span.TileCount);
		}
	}

	// Pre-commit always-resident standard mips: anything below the byte
	// threshold stays committed for the lifetime of the resource. This
	// matches the manager-side eligibility computation so the sparse
	// path only manages mips that meaningfully save VRAM.
	for (int32 MipIndex = 0; MipIndex < static_cast<int32>(PackedMipInfo.NumStandardMips); ++MipIndex)
	{
		if (!Layout.IsMipSparseEligible(MipIndex))
		{
			FMipMapping& Mip = Entry.Mips[MipIndex];
			if (Mip.NumTiles == 0)
			{
				continue;
			}
			if (!AllocateSpans_NoLock(Mip.NumTiles, bRenderTargetable, Mip.Spans, /*bForceAllocation=*/true))
			{
				UE_LOGF(LogTmvMedia, Warning,
					"FTmvSparseOutputBackendD3D12: failed to allocate always-resident mip %d (%d tiles).",
					MipIndex, Mip.NumTiles);
				continue;
			}
			Mip.bCommitted = true;
			Mip.bAlwaysResident = true; // below-threshold mip: never stealable
			Mip.CommittedBytes = static_cast<int64>(Mip.NumTiles) * TileSizeInBytes;
			TotalCommittedBytes += Mip.CommittedBytes;

			uint32 Offset = 0;
			for (const FMipHeapSpan& Span : Mip.Spans)
			{
				const FHeapEntry& Heap = HeapPool[Span.HeapIndex];
				SubmitMappingForSpan(
					Resource,
					static_cast<uint32>(MipIndex),
					Offset,
					static_cast<uint32>(Mip.WidthInTiles),
					static_cast<uint32>(Mip.HeightInTiles),
					Heap.Heap.GetReference(),
					static_cast<uint32>(Span.HeapStartTile),
					static_cast<uint32>(Span.TileCount),
					/*bCommit=*/true);
				Offset += static_cast<uint32>(Span.TileCount);
			}
		}
	}

	Entries.Add(Texture, MoveTemp(Entry));
	return true;
}

void FTmvSparseOutputBackendD3D12::UnregisterOutput(FRHITexture* Texture)
{
	if (!Texture)
	{
		return;
	}

	FScopeLock ScopeLock(&Lock);
	FTextureEntry* Entry = Entries.Find(Texture);
	if (!Entry)
	{
		return;
	}

	// Walk every committed mip (owned chunks + always-resident packed tail and
	// below-threshold mips). Issue unmap to the GPU queue and release each
	// chunk's spans back to the heap pool. UE will tear down the reserved
	// resource shortly after; the manager waits for GPU idle around this on
	// process exit so the unmap ExecuteCommandLists drains before device
	// teardown.
	for (int32 MipIndex = 0; MipIndex < Entry->Mips.Num(); ++MipIndex)
	{
		FMipMapping& Mip = Entry->Mips[MipIndex];
		if (!Mip.bCommitted)
		{
			continue;
		}
		uint32 Offset = 0;
		for (const FMipHeapSpan& Span : Mip.Spans)
		{
			if (!HeapPool.IsValidIndex(Span.HeapIndex))
			{
				continue;
			}
			const FHeapEntry& Heap = HeapPool[Span.HeapIndex];
			SubmitMappingForSpan(
				Entry->Resource,
				static_cast<uint32>(MipIndex),
				Offset,
				static_cast<uint32>(Mip.WidthInTiles),
				static_cast<uint32>(Mip.HeightInTiles),
				Heap.Heap.GetReference(),
				static_cast<uint32>(Span.HeapStartTile),
				static_cast<uint32>(Span.TileCount),
				/*bCommit=*/false);
			Offset += static_cast<uint32>(Span.TileCount);
		}
		FreeSpans_NoLock(Mip.Spans);
		TotalCommittedBytes -= Mip.CommittedBytes;
		Mip.Spans.Reset();
		Mip.CommittedBytes = 0;
		Mip.bCommitted = false;
	}

	Entries.Remove(Texture);

	// Drop heaps that drained as a result of this teardown. This is the only
	// place where physical heap memory is released back to D3D12; the
	// demand-driven pool never shrinks otherwise.
	ReleaseEmptyTrailingHeaps_NoLock();
}

FTmvSparseOutputBackendD3D12::FStealTarget FTmvSparseOutputBackendD3D12::FindStealableMip_NoLock(
	FRHITexture* RequestingTexture,
	int32 RequestingMipIndex,
	int32 RequiredTiles,
	double NowSeconds,
	bool bIgnoreHysteresis) const
{
	const double HysteresisSeconds = static_cast<double>(UE::TmvMedia::GetSparseOutputMipHysteresisSeconds());

	FStealTarget Best;
	double OldestUsed = TNumericLimits<double>::Max();

	for (const TPair<FRHITexture*, FTextureEntry>& Pair : Entries)
	{
		const FTextureEntry& CandEntry = Pair.Value;
		if (CandEntry.bPinned)
		{
			continue;
		}
		for (int32 CandMipIdx = 0; CandMipIdx < CandEntry.Mips.Num(); ++CandMipIdx)
		{
			if (Pair.Key == RequestingTexture && CandMipIdx == RequestingMipIndex)
			{
				continue;
			}
			const FMipMapping& Cand = CandEntry.Mips[CandMipIdx];
			if (!Cand.bCommitted || Cand.bAlwaysResident || Cand.bPackedMip)
			{
				continue;
			}
			if (Cand.NumTiles != RequiredTiles)
			{
				continue;
			}
			if (!bIgnoreHysteresis && (NowSeconds - Cand.LastUsedTime) < HysteresisSeconds)
			{
				continue;
			}
			if (Cand.LastUsedTime < OldestUsed)
			{
				OldestUsed = Cand.LastUsedTime;
				Best.OwnerTexture = Pair.Key;
				Best.MipIndex = CandMipIdx;
			}
		}
	}
	return Best;
}

void FTmvSparseOutputBackendD3D12::RebindChunk_NoLock(
	FTextureEntry& FromEntry,
	int32 FromMipIndex,
	FTextureEntry& ToEntry,
	int32 ToMipIndex)
{
	FMipMapping& FromMip = FromEntry.Mips[FromMipIndex];
	FMipMapping& ToMip = ToEntry.Mips[ToMipIndex];

	// Step 1: unmap from the previous owner. Step 2: map to the new owner.
	// Both go on the graphics queue via RHIRunOnQueue; queue-serial ordering
	// guarantees the mapping is live by the time the consuming
	// ExecuteCommandLists for ToEntry's draw executes. The manager is
	// responsible for raising FromEntry's ResidentFloor and republishing
	// its SRV so any draw that samples FromEntry post-steal clamps to a
	// still-mapped mip.
	{
		uint32 Offset = 0;
		for (const FMipHeapSpan& Span : FromMip.Spans)
		{
			if (!HeapPool.IsValidIndex(Span.HeapIndex))
			{
				continue;
			}
			const FHeapEntry& Heap = HeapPool[Span.HeapIndex];
			SubmitMappingForSpan(
				FromEntry.Resource,
				static_cast<uint32>(FromMipIndex),
				Offset,
				static_cast<uint32>(FromMip.WidthInTiles),
				static_cast<uint32>(FromMip.HeightInTiles),
				Heap.Heap.GetReference(),
				static_cast<uint32>(Span.HeapStartTile),
				static_cast<uint32>(Span.TileCount),
				/*bCommit=*/false);
			Offset += static_cast<uint32>(Span.TileCount);
		}
	}
	{
		uint32 Offset = 0;
		for (const FMipHeapSpan& Span : FromMip.Spans)
		{
			if (!HeapPool.IsValidIndex(Span.HeapIndex))
			{
				continue;
			}
			const FHeapEntry& Heap = HeapPool[Span.HeapIndex];
			SubmitMappingForSpan(
				ToEntry.Resource,
				static_cast<uint32>(ToMipIndex),
				Offset,
				static_cast<uint32>(ToMip.WidthInTiles),
				static_cast<uint32>(ToMip.HeightInTiles),
				Heap.Heap.GetReference(),
				static_cast<uint32>(Span.HeapStartTile),
				static_cast<uint32>(Span.TileCount),
				/*bCommit=*/true);
			Offset += static_cast<uint32>(Span.TileCount);
		}
	}

	// Transfer span ownership: same physical tiles, new logical mip.
	ToMip.Spans = MoveTemp(FromMip.Spans);
	ToMip.bCommitted = true;
	ToMip.CommittedBytes = static_cast<int64>(ToMip.NumTiles) * TileSizeInBytes;
	ToMip.LastUsedTime = FPlatformTime::Seconds();

	// Bytes accounting: pool size is unchanged (same tiles, different mapping);
	// add what ToMip claims and subtract what FromMip released. For
	// equal-tile-count chunks these are equal, so the net is zero, but the
	// math is honest about the moving parts.
	FromMip.Spans.Reset();
	TotalCommittedBytes -= FromMip.CommittedBytes;
	FromMip.CommittedBytes = 0;
	FromMip.bCommitted = false;
	FromMip.LastUsedTime = 0.0;
	TotalCommittedBytes += ToMip.CommittedBytes;
}

FTmvSparseMipCommitResult FTmvSparseOutputBackendD3D12::CommitMip(FRHITexture* Texture, int32 MipIndex)
{
	FTmvSparseMipCommitResult Result;
	if (!Texture)
	{
		return Result;
	}

	FScopeLock ScopeLock(&Lock);
	FTextureEntry* Entry = Entries.Find(Texture);
	if (!Entry || !Entry->Mips.IsValidIndex(MipIndex))
	{
		return Result;
	}
	FMipMapping& Mip = Entry->Mips[MipIndex];
	if (Mip.bPackedMip || Mip.NumTiles == 0)
	{
		return Result;
	}

	const double NowSeconds = FPlatformTime::Seconds();

	// Touch path: already committed. Refresh LastUsedTime and return.
	if (Mip.bCommitted)
	{
		Mip.LastUsedTime = NowSeconds;
		return Result;
	}

	// Steal path: search for an oldest-LRU chunk past hysteresis with matching
	// tile count. On hit we rebind the physical tiles to this mip without
	// growing the pool.
	{
		const FStealTarget Steal = FindStealableMip_NoLock(
			Texture, MipIndex, Mip.NumTiles, NowSeconds, /*bIgnoreHysteresis=*/false);
		if (Steal.OwnerTexture && Steal.OwnerTexture != Texture)
		{
			if (FTextureEntry* FromEntry = Entries.Find(Steal.OwnerTexture))
			{
				RebindChunk_NoLock(*FromEntry, Steal.MipIndex, *Entry, MipIndex);
				Result.StolenFromTexture = Steal.OwnerTexture;
				Result.StolenFromMip = Steal.MipIndex;
				return Result;
			}
		}
		// Same-texture steal: a hand-off between mips of the same texture
		// would re-bind tiles the texture itself depends on. Skip and
		// fall through to fresh-alloc.
	}

	// Fresh allocation path: walk free space in existing heaps, grow on demand
	// (subject to budget gate). On failure, fall through to force-steal.
	const bool bRenderTargetable = IsRenderTargetOrDepthStencil(Texture);
	if (AllocateSpans_NoLock(Mip.NumTiles, bRenderTargetable, Mip.Spans))
	{
		uint32 Offset = 0;
		for (const FMipHeapSpan& Span : Mip.Spans)
		{
			const FHeapEntry& Heap = HeapPool[Span.HeapIndex];
			SubmitMappingForSpan(
				Entry->Resource,
				static_cast<uint32>(MipIndex),
				Offset,
				static_cast<uint32>(Mip.WidthInTiles),
				static_cast<uint32>(Mip.HeightInTiles),
				Heap.Heap.GetReference(),
				static_cast<uint32>(Span.HeapStartTile),
				static_cast<uint32>(Span.TileCount),
				/*bCommit=*/true);
			Offset += static_cast<uint32>(Span.TileCount);
		}
		Mip.bCommitted = true;
		Mip.CommittedBytes = static_cast<int64>(Mip.NumTiles) * TileSizeInBytes;
		Mip.LastUsedTime = NowSeconds;
		TotalCommittedBytes += Mip.CommittedBytes;
		return Result;
	}

	// Force-steal fallback: pool is at the budget cap. Steal regardless of
	// hysteresis so the commit succeeds; manager raises the victim's floor.
	{
		const FStealTarget Steal = FindStealableMip_NoLock(
			Texture, MipIndex, Mip.NumTiles, NowSeconds, /*bIgnoreHysteresis=*/true);
		if (Steal.OwnerTexture && Steal.OwnerTexture != Texture)
		{
			if (FTextureEntry* FromEntry = Entries.Find(Steal.OwnerTexture))
			{
				RebindChunk_NoLock(*FromEntry, Steal.MipIndex, *Entry, MipIndex);
				Result.StolenFromTexture = Steal.OwnerTexture;
				Result.StolenFromMip = Steal.MipIndex;
				return Result;
			}
		}
	}

	UE_LOGF(LogTmvMedia, Warning,
		"FTmvSparseOutputBackendD3D12: CommitMip(%d) failed: pool full, no stealable chunk (%d tiles requested).",
		MipIndex, Mip.NumTiles);
	return Result;
}

void FTmvSparseOutputBackendD3D12::SetTexturePinned(FRHITexture* Texture, bool bPinned)
{
	if (!Texture)
	{
		return;
	}
	FScopeLock ScopeLock(&Lock);
	if (FTextureEntry* Entry = Entries.Find(Texture))
	{
		Entry->bPinned = bPinned;
	}
}

bool FTmvSparseOutputBackendD3D12::IsMipCommitted(FRHITexture* Texture, int32 MipIndex) const
{
	FScopeLock ScopeLock(&Lock);
	if (const FTextureEntry* Entry = Entries.Find(Texture))
	{
		if (Entry->Mips.IsValidIndex(MipIndex))
		{
			return Entry->Mips[MipIndex].bCommitted;
		}
	}
	return false;
}

int64 FTmvSparseOutputBackendD3D12::GetCommittedBytes(FRHITexture* Texture, int32 MipIndex) const
{
	FScopeLock ScopeLock(&Lock);
	if (const FTextureEntry* Entry = Entries.Find(Texture))
	{
		if (Entry->Mips.IsValidIndex(MipIndex))
		{
			return Entry->Mips[MipIndex].CommittedBytes;
		}
	}
	return 0;
}

int64 FTmvSparseOutputBackendD3D12::GetTotalCommittedBytes() const
{
	FScopeLock ScopeLock(&Lock);
	return TotalCommittedBytes;
}

void FTmvSparseOutputBackendD3D12::Dump(FOutputDevice& Ar) const
{
	FScopeLock ScopeLock(&Lock);
	int64 PoolBytes = 0;
	for (const FHeapEntry& H : HeapPool)
	{
		PoolBytes += static_cast<int64>(H.TilesInHeap) * TileSizeInBytes;
	}
	Ar.Logf(TEXT("FTmvSparseOutputBackendD3D12: %d textures, %lld committed bytes, %lld pool bytes across %d heaps."),
		Entries.Num(), TotalCommittedBytes, PoolBytes, HeapPool.Num());
	for (int32 hi = 0; hi < HeapPool.Num(); ++hi)
	{
		const FHeapEntry& Heap = HeapPool[hi];
		const int64 HeapBytes = static_cast<int64>(Heap.TilesInHeap) * TileSizeInBytes;
		const int64 UsedBytes = static_cast<int64>(Heap.TilesInHeap - Heap.NumFreeTiles) * TileSizeInBytes;
		Ar.Logf(TEXT("  Heap[%d]: %lld bytes total, %lld used (%d/%d tiles)"),
			hi, HeapBytes, UsedBytes, Heap.TilesInHeap - Heap.NumFreeTiles, Heap.TilesInHeap);
	}
	const double NowSeconds = FPlatformTime::Seconds();
	for (const TPair<FRHITexture*, FTextureEntry>& Pair : Entries)
	{
		const FTextureEntry& Entry = Pair.Value;
		Ar.Logf(TEXT("  Texture %p: %dx%d %dmips, packed=%u standard=%u, pinned=%s"),
			Pair.Key, Entry.Layout.Dim.X, Entry.Layout.Dim.Y, Entry.Layout.NumMips,
			Entry.NumPackedMips, Entry.NumStandardMips,
			Entry.bPinned ? TEXT("yes") : TEXT("no"));
		for (int32 MipIndex = 0; MipIndex < Entry.Mips.Num(); ++MipIndex)
		{
			const FMipMapping& Mip = Entry.Mips[MipIndex];
			const double AgeSeconds = (Mip.bCommitted && !Mip.bAlwaysResident && Mip.LastUsedTime > 0.0)
				? (NowSeconds - Mip.LastUsedTime)
				: 0.0;
			Ar.Logf(TEXT("    mip %d: %dx%d tiles, committed=%s, bytes=%lld, packed=%s, alwaysResident=%s, ageSec=%.2f"),
				MipIndex, Mip.WidthInTiles, Mip.HeightInTiles,
				Mip.bCommitted ? TEXT("yes") : TEXT("no"),
				Mip.CommittedBytes,
				Mip.bPackedMip ? TEXT("yes") : TEXT("no"),
				Mip.bAlwaysResident ? TEXT("yes") : TEXT("no"),
				AgeSeconds);
		}
	}
}
