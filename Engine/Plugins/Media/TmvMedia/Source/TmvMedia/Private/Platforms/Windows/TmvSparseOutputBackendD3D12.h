// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"

#include "SampleConverter/SparseOutputMip/TmvSparseOutputBackend.h"

#include "ID3D12DynamicRHI.h"

// <d3d12.h> (pulled in by ID3D12DynamicRHI.h) includes <windows.h>, whose
// <WinUser.h> defines LoadImage as LoadImageW. The engine's
// WindowsD3D12ThirdParty.h undefs DrawText but not LoadImage, so without this the
// macro leaks across the TmvMedia unity blob and rewrites FImageUtils::LoadImage
// (in the file-sequence frame producer) into an unresolved LoadImageW symbol.
// Undo it at the boundary where this module pulls in the D3D12 headers; TmvMedia
// never uses the Win32 LoadImage API.
#ifdef LoadImage
#undef LoadImage
#endif

/**
 * D3D12 backend for the TMV sparse-output media-texture path. Uses
 * ID3D12DynamicRHI (the public extension surface DLSS/FSR/RTX plugins use)
 * to issue per-mip UpdateTileMappings on a reserved (tiled) texture without
 * extending the engine RHI. See plan
 * `we-ve-improved-performance-reduced-agile-journal.md` and memory
 * `tmv-sparse-output-mip-rhi-gap` for why the public RHI is insufficient
 * on this branch.
 *
 * Demand-driven chunk pool. Physical tile-spans stay mapped to whoever last
 * owned them; CommitMip prefers to steal an eligible chunk (oldest LRU,
 * matching tile count, past hysteresis) and rebind it via UpdateTileMappings
 * rather than free + re-alloc. Memory is released only at UnregisterOutput,
 * which walks the texture's owned chunks, unmaps them, and drops trailing
 * empty heaps from the pool. Heap pool grows exponentially (64 -> 128 ->
 * 256 -> 512 MB, capped) up to the configured budget; over-budget pressure
 * force-steals the oldest chunk regardless of hysteresis as a safety valve.
 */
class FTmvSparseOutputBackendD3D12 : public FTmvSparseOutputBackend
{
public:
	FTmvSparseOutputBackendD3D12();
	virtual ~FTmvSparseOutputBackendD3D12() override;

	//~ FTmvSparseOutputBackend interface
	virtual bool RegisterOutput(FRHITexture* Texture, const FTmvSparseOutputLayout& Layout) override;
	virtual void UnregisterOutput(FRHITexture* Texture) override;
	virtual FTmvSparseMipCommitResult CommitMip(FRHITexture* Texture, int32 MipIndex) override;
	virtual void SetTexturePinned(FRHITexture* Texture, bool bPinned) override;
	virtual bool IsMipCommitted(FRHITexture* Texture, int32 MipIndex) const override;
	virtual int64 GetCommittedBytes(FRHITexture* Texture, int32 MipIndex) const override;
	virtual int64 GetTotalCommittedBytes() const override;
	virtual void Dump(FOutputDevice& Ar) const override;
	//~ End FTmvSparseOutputBackend interface

	/** Creator registered with FTmvSparseOutputBackendFactory at static init. */
	static TSharedPtr<FTmvSparseOutputBackend, ESPMode::ThreadSafe> CreateInstance();

private:
	static constexpr int64 TileSizeInBytes = 65536; // D3D12 standard tile size

	struct FHeapEntry
	{
		TRefCountPtr<ID3D12Heap> Heap;
		int32 TilesInHeap = 0;
		int32 NumFreeTiles = 0;
		TBitArray<> FreeTiles; // bit set = free
	};

	/** One contiguous range of tiles allocated to a single mip from one heap. */
	struct FMipHeapSpan
	{
		int32 HeapIndex = INDEX_NONE;
		int32 HeapStartTile = 0;
		int32 TileCount = 0;
	};

	struct FMipMapping
	{
		int32 WidthInTiles = 0;
		int32 HeightInTiles = 0;
		int32 DepthInTiles = 1;
		int32 NumTiles = 0;
		bool bCommitted = false;
		bool bPackedMip = false;
		bool bAlwaysResident = false; // packed tail or below-threshold: never stealable, never decommitted
		TArray<FMipHeapSpan> Spans;
		int64 CommittedBytes = 0;
		double LastUsedTime = 0.0; // wall-clock seconds; refreshed by CommitMip touch path
	};

	struct FTextureEntry
	{
		// Holds the FRHITexture refcount so the underlying ID3D12Resource stays alive
		// from RegisterOutput until UnregisterOutput, regardless of when external owners
		// (FMediaTextureResource) drop their refs. Without this, the engine-exit drain
		// path dereferences a Resource that's already been freed by deferred-delete.
		FTextureRHIRef Texture;
		ID3D12Resource* Resource = nullptr; // observer of Texture's underlying D3D12 object
		FTmvSparseOutputLayout Layout;
		TArray<FMipMapping> Mips;
		uint32 NumStandardMips = 0;
		uint32 NumPackedMips = 0;
		uint32 NumTilesForPackedMips = 0;
		uint32 PackedMipStartTileIndex = 0;
		bool bPinned = false; // manager-side FullResidencyRefCount > 0: chunks not stealable
	};

	// Returns true if the entry's referenced texture has render-target or
	// depth-stencil flags; controls heap-flag selection.
	bool IsRenderTargetOrDepthStencil(FRHITexture* Texture) const;

	// Allocate K tiles, possibly across multiple heaps. Appends spans to OutSpans
	// and returns true on success. Holds Lock; assumes caller already holds it.
	// bForceAllocation = true bypasses the budget cap - used for always-resident
	// commits at RegisterOutput that have no stealable fallback. Sparse-mip
	// commits leave it false so the budget cap forces a steal-or-fail.
	bool AllocateSpans_NoLock(int32 NumTiles, bool bRenderTargetable, TArray<FMipHeapSpan>& OutSpans, bool bForceAllocation = false);

	// Mark spans free. Holds Lock; assumes caller already holds it.
	void FreeSpans_NoLock(TConstArrayView<FMipHeapSpan> Spans);

	// Allocate a new heap large enough to hold at least MinTiles tiles, capped
	// at MaxHeapSizeInBytes. Returns new heap index, or INDEX_NONE on failure
	// (CreateHeap failed, or would exceed budget). bForceAllocation = true
	// bypasses the budget cap for callers that must succeed.
	int32 AllocateHeap_NoLock(int32 MinTiles, bool bRenderTargetable, bool bForceAllocation = false);

	// Drop trailing fully-empty heaps. Holds Lock; assumes caller already holds it.
	void ReleaseEmptyTrailingHeaps_NoLock();

	// Find an existing committed mip elsewhere in Entries that can give up its
	// chunk to satisfy a NumTiles request. Returns {nullptr, INDEX_NONE} on miss.
	// LRU among eligible chunks: oldest LastUsedTime wins. Skips pinned textures,
	// skips always-resident mips (packed tail + below-threshold), skips the
	// requesting (Texture, MipIndex) itself. If bIgnoreHysteresis is false the
	// chunk must be at least HysteresisSeconds wall-clock old.
	struct FStealTarget
	{
		FRHITexture* OwnerTexture = nullptr;
		int32 MipIndex = INDEX_NONE;
	};
	FStealTarget FindStealableMip_NoLock(
		FRHITexture* RequestingTexture,
		int32 RequestingMipIndex,
		int32 RequiredTiles,
		double NowSeconds,
		bool bIgnoreHysteresis) const;

	// Move the existing chunk owned by (FromTex, FromMip) to back (ToTex, ToMip).
	// Issues unmap from FromTex's VA then map to ToTex's VA on the GPU queue;
	// the ExecuteCommandLists ordering against subsequent draws ensures mappings
	// are live before sampling. Caller must have raised FromTex's residency
	// floor (or be about to) before any draw that samples FromTex's now-stale mip.
	void RebindChunk_NoLock(
		FTextureEntry& FromEntry,
		int32 FromMipIndex,
		FTextureEntry& ToEntry,
		int32 ToMipIndex);

	// Submit one UpdateTileMappings call for a single span. Issued via RHIRunOnQueue.
	void SubmitMappingForSpan(
		ID3D12Resource* Resource,
		uint32 Subresource,
		uint32 MipStartTileOffset,
		uint32 MipWidthInTiles,
		uint32 MipHeightInTiles,
		ID3D12Heap* Heap,
		uint32 HeapStartTile,
		uint32 TileCount,
		bool bCommit);

	mutable FCriticalSection Lock;
	TMap<FRHITexture*, FTextureEntry> Entries;
	TArray<FHeapEntry> HeapPool;
	int64 TotalCommittedBytes = 0;

	// Heap growth schedule: 64, 128, 256, 512 MB (cap). Index = next-heap-to-allocate position.
	int32 NextHeapGrowthStep = 0;
};
