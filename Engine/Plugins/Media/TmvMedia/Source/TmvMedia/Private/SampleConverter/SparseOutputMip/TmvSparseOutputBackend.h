// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Math/IntPoint.h"
#include "PixelFormat.h"
#include "RHIDefinitions.h"
#include "Templates/SharedPointer.h"

class FOutputDevice;
class FRHITexture;

/**
 * Per-output-texture layout snapshot passed to the backend at registration
 * time. The manager fills this from the texture create description; the
 * backend uses it to decide which mips are sparse-eligible, query tile counts
 * from the underlying API, and pre-commit always-resident mips (those below
 * the byte threshold or in a hardware-packed mip tail).
 */
struct FTmvSparseOutputLayout
{
	FIntPoint Dim = FIntPoint::ZeroValue;
	EPixelFormat Format = PF_Unknown;
	int32 NumMips = 0;
	int32 BytesPerPixel = 0;
	int64 MinSparseBytes = 0; // bytes; mips with W*H*BytesPerPixel below this are not sparse-managed.

	int64 ComputeMipBytes(int32 MipIndex) const
	{
		if (MipIndex < 0 || MipIndex >= NumMips)
		{
			return 0;
		}
		const int64 W = FMath::Max<int64>(1, static_cast<int64>(Dim.X) >> MipIndex);
		const int64 H = FMath::Max<int64>(1, static_cast<int64>(Dim.Y) >> MipIndex);
		return W * H * static_cast<int64>(BytesPerPixel);
	}

	bool IsMipSparseEligible(int32 MipIndex) const
	{
		return ComputeMipBytes(MipIndex) >= MinSparseBytes;
	}
};

/**
 * Result of a CommitMip call. If a fresh chunk was allocated, both fields
 * are zero/null. If an existing chunk was stolen from another texture/mip
 * to satisfy the commit, StolenFromTexture/StolenFromMip identify the
 * previous owner so the manager can raise its ResidentFloor and republish
 * its SRV before any draw samples the now-unmapped subresource.
 */
struct FTmvSparseMipCommitResult
{
	FRHITexture* StolenFromTexture = nullptr;
	int32 StolenFromMip = INDEX_NONE;
};

/**
 * Platform/RHI-agnostic interface for the sparse output media-texture backend.
 * Each backend implements per-mip commit on a reserved (tiled) resource using
 * its API of choice (D3D12 UpdateTileMappings, Vulkan vkBindSparseImageMemory,
 * etc). Backends self-register with the factory during static init; the
 * platform-specific file lives under Private/Platforms/<Platform>/ and is
 * automatically excluded by UBT on non-matching targets.
 *
 * Memory model: demand-driven chunk pool. Physical tile-spans stay mapped to
 * whoever last owned them; a new commit prefers to steal a chunk past its
 * hysteresis age via UpdateTileMappings rebind rather than free + re-alloc.
 * Memory is physically released only at UnregisterOutput (and the stale-GC
 * path that calls it). No timer ever decommits.
 *
 * Threading: all methods are called from the render thread. Backends may
 * dispatch actual GPU/queue work asynchronously (RHIRunOnQueue or
 * equivalent) but must guarantee that a CommitMip issued before a converter
 * pass has produced a valid mapping by the time that pass executes on the
 * GPU. UpdateTileMappings + ExecuteCommandLists ordering on D3D12 gives
 * this for free; other backends must reproduce the equivalent.
 */
class FTmvSparseOutputBackend
{
public:
	virtual ~FTmvSparseOutputBackend() = default;

	/**
	 * One-time per-resource setup: caches the underlying API handle, queries
	 * per-mip tile counts, pre-commits always-resident mips. Returns false
	 * on any failure; manager will keep the texture non-sparse in that case
	 * and the per-clip cost reverts to baseline.
	 */
	virtual bool RegisterOutput(FRHITexture* Texture, const FTmvSparseOutputLayout& Layout) = 0;

	/** Tear down per-resource state. Waits for any in-flight GPU work that references the resource. */
	virtual void UnregisterOutput(FRHITexture* Texture) = 0;

	/**
	 * Commit physical backing for the given mip subresource. Idempotent: a
	 * second call when the mip is already committed refreshes the chunk's
	 * last-used timestamp and returns an empty result. On fresh commit the
	 * backend prefers to steal an eligible owned chunk (returns StolenFrom*
	 * identifying the prior owner) rather than allocate; the manager must
	 * raise the prior owner's ResidentFloor and republish its SRV before
	 * the next draw.
	 */
	virtual FTmvSparseMipCommitResult CommitMip(FRHITexture* Texture, int32 MipIndex) = 0;

	/**
	 * Mark all of a texture's owned chunks as pinned (not eligible for steal)
	 * or unpinned. Set true when a manager-side FullResidencyRefCount goes
	 * 0 -> 1, false on the 1 -> 0 transition. Backends with no pinning
	 * concept may treat as a no-op.
	 */
	virtual void SetTexturePinned(FRHITexture* Texture, bool bPinned) = 0;

	/** Returns true iff the named mip currently has physical backing. */
	virtual bool IsMipCommitted(FRHITexture* Texture, int32 MipIndex) const = 0;

	/**
	 * Committed bytes for a specific mip, summed across all tiles that
	 * actually have physical backing right now (NOT the nominal mip size).
	 * Used by tmvmedia.dumpmipbuffers to surface real VRAM impact.
	 */
	virtual int64 GetCommittedBytes(FRHITexture* Texture, int32 MipIndex) const = 0;

	/** Sum of committed bytes across all sparse output textures the backend is managing. */
	virtual int64 GetTotalCommittedBytes() const = 0;

	/** Free-form dump for tmvmedia.dumpmipbuffers; includes per-texture, per-mip rows. */
	virtual void Dump(FOutputDevice& Ar) const = 0;
};

/**
 * Singleton registry: each platform/RHI backend self-registers its creator
 * at static-init time, keyed by ERHIInterfaceType. The manager queries this
 * at startup and resolves to whichever backend matches the running RHI; on
 * an RHI with no registered backend, manager startup logs once and the
 * sparse-output path stays disabled regardless of CVar.
 */
class FTmvSparseOutputBackendFactory
{
public:
	using FCreator = TSharedPtr<FTmvSparseOutputBackend, ESPMode::ThreadSafe>(*)();

	bool RegisterBackendForRhi(ERHIInterfaceType RhiType, FCreator Creator);
	TSharedPtr<FTmvSparseOutputBackend, ESPMode::ThreadSafe> CreateInstanceForRhi(ERHIInterfaceType RhiType);
	bool HasBackendForRhi(ERHIInterfaceType RhiType) const;

	static FTmvSparseOutputBackendFactory* Get();

private:
	TMap<ERHIInterfaceType, FCreator> BackendCreators;
};
