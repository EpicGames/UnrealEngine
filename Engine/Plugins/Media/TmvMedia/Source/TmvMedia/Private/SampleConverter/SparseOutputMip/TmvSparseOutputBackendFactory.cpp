// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvSparseOutputBackend.h"

bool FTmvSparseOutputBackendFactory::RegisterBackendForRhi(ERHIInterfaceType RhiType, FCreator Creator)
{
	if (!Creator)
	{
		return false;
	}
	BackendCreators.Add(RhiType, Creator);
	return true;
}

TSharedPtr<FTmvSparseOutputBackend, ESPMode::ThreadSafe> FTmvSparseOutputBackendFactory::CreateInstanceForRhi(ERHIInterfaceType RhiType)
{
	if (const FCreator* Creator = BackendCreators.Find(RhiType))
	{
		return (*Creator)();
	}
	return nullptr;
}

bool FTmvSparseOutputBackendFactory::HasBackendForRhi(ERHIInterfaceType RhiType) const
{
	return BackendCreators.Contains(RhiType);
}

FTmvSparseOutputBackendFactory* FTmvSparseOutputBackendFactory::Get()
{
	static FTmvSparseOutputBackendFactory Instance;
	return &Instance;
}
