// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleConverter/SparseOutputMip/TmvSparseOutputManager.h"

#include "TmvSparseOutputBackend.h"
#include "TmvMediaFrameInfo.h"
#include "TmvMediaTuningCVars.h"

#include "DynamicRHI.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "MediaDelegates.h"
#include "Misc/CoreDelegates.h"
#include "Misc/OutputDevice.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeLock.h"
#include "RHI.h"
#include "RenderingThread.h"

FTmvSparseOutputManager& FTmvSparseOutputManager::Get()
{
	static FTmvSparseOutputManager Instance;
	return Instance;
}

FTmvSparseOutputManager::FTmvSparseOutputManager()
{
	if (GDynamicRHI)
	{
		const ERHIInterfaceType RhiType = GDynamicRHI->GetInterfaceType();
		Backend = FTmvSparseOutputBackendFactory::Get()->CreateInstanceForRhi(RhiType);
	}

	// Drain everything on shutdown so heap refs are released before the
	// D3D12 device is torn down. There is no in-session GC tick: an
	// FRHITexture can still be bound to material SRVs long after its
	// converter stops calling RequestFloor, and unmapping under a live SRV
	// faults the GPU (NVIDIA MMU fault -> device removal).
	//
	// UpdateTileMappings(unmap) lands on the GPU queue while the GPU may
	// still have draws in flight that sample these reserved textures. The GPU
	// then faults reading an unmapped VA, surfacing as "D3D Device Removed" /
	// Aftermath page-fault on quit. Flushing the render thread and then
	// blocking until the GPU is idle on the render thread guarantees the
	// pipeline is fully drained before we tear down heap mappings.
	OnEnginePreExitHandle = FCoreDelegates::OnEnginePreExit.AddLambda([this]()
	{
		FlushRenderingCommands();
		ENQUEUE_RENDER_COMMAND(TmvSparseOutputManagerShutdownGpuSync)(
			[](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.SubmitAndBlockUntilGPUIdle();
			});
		FlushRenderingCommands();
		FScopeLock ScopeLock(&Lock);
		DrainAll_NoLock();
	});

	// Unregister per-texture state on the render thread the moment
	// FMediaTextureResource drops its intermediate target, while the underlying
	// ID3D12Resource is still alive. Without this, the engine-exit drain runs
	// against a Resource that's already been freed by deferred-delete.
	OnMediaTextureReleasedHandle = FMediaDelegates::OnMediaTextureResourceReleased_RenderThread.AddRaw(
		this, &FTmvSparseOutputManager::HandleMediaTextureReleased);
}

FTmvSparseOutputManager::~FTmvSparseOutputManager()
{
	if (OnMediaTextureReleasedHandle.IsValid())
	{
		FMediaDelegates::OnMediaTextureResourceReleased_RenderThread.Remove(OnMediaTextureReleasedHandle);
	}
	if (OnEnginePreExitHandle.IsValid())
	{
		FCoreDelegates::OnEnginePreExit.Remove(OnEnginePreExitHandle);
	}
	FScopeLock ScopeLock(&Lock);
	DrainAll_NoLock();
}

void FTmvSparseOutputManager::HandleMediaTextureReleased(FRHITexture* Texture)
{
	UnregisterOutput(Texture);
}

void FTmvSparseOutputManager::DrainAll_NoLock()
{
	if (!Backend.IsValid())
	{
		States.Reset();
		Lru.Reset();
		return;
	}
	// Make a copy of keys - UnregisterOutput mutates States/Lru.
	TArray<FRHITexture*> AllKeys;
	AllKeys.Reserve(States.Num());
	for (const TPair<FRHITexture*, FTextureState>& Pair : States)
	{
		AllKeys.Add(Pair.Key);
	}
	for (FRHITexture* Key : AllKeys)
	{
		Backend->UnregisterOutput(Key);
	}
	States.Reset();
	Lru.Reset();
}

bool FTmvSparseOutputManager::RegisterOutput(FRHITexture* Texture, const FTmvSparseOutputLayout& Layout)
{
	if (!Backend.IsValid() || !Texture)
	{
		return false;
	}

	// Idempotent: the converter calls this on every frame at Convert() entry
	// because the converter is per-frame but the destination texture is
	// per-plate (longer-lived). A second call with the same shape is a no-op;
	// a call with a different shape would indicate the upstream texture was
	// rebuilt under us and is treated as a fresh registration after
	// unregistering the stale state.
	{
		FScopeLock ScopeLock(&Lock);
		if (FTextureState* Existing = States.Find(Texture))
		{
			if (Existing->NumMips == Layout.NumMips)
			{
				return true;
			}
		}
	}
	UnregisterOutput(Texture);

	if (!Backend->RegisterOutput(Texture, Layout))
	{
		return false;
	}

	FScopeLock ScopeLock(&Lock);

	FTextureState State;
	State.NumMips = Layout.NumMips;
	State.SparseEligible.Init(false, Layout.NumMips);

	int32 MaxSparse = -1;
	for (int32 MipIndex = 0; MipIndex < Layout.NumMips; ++MipIndex)
	{
		if (Layout.IsMipSparseEligible(MipIndex))
		{
			State.SparseEligible[MipIndex] = true;
			MaxSparse = FMath::Max(MaxSparse, MipIndex);
		}
	}
	State.MaxSparseMipIndex = MaxSparse;
	State.ResidentFloor = (MaxSparse < 0) ? 0 : MaxSparse + 1; // nothing sparse-committed yet

	States.Add(Texture, MoveTemp(State));
	Lru.Add(Texture);
	return true;
}

void FTmvSparseOutputManager::UnregisterOutput(FRHITexture* Texture)
{
	if (!Texture)
	{
		return;
	}

	if (Backend.IsValid())
	{
		Backend->UnregisterOutput(Texture);
	}

	FScopeLock ScopeLock(&Lock);
	States.Remove(Texture);
	Lru.RemoveSingle(Texture);
}

void FTmvSparseOutputManager::RequestFloor(const FTmvSparseOutputFrameRequest& Request)
{
	if (!Backend.IsValid() || !Request.Texture)
	{
		return;
	}

	FScopeLock ScopeLock(&Lock);
	FTextureState* State = States.Find(Request.Texture);
	if (!State || State->MaxSparseMipIndex < 0)
	{
		return;
	}

	// Bump LRU - the converter is actively using this texture this frame.
	Lru.RemoveSingle(Request.Texture);
	Lru.Add(Request.Texture);

	const int32 DebugForceFloor = UE::TmvMedia::GetSparseOutputMipDebugForceFloor();
	const int32 EffectiveTarget = (DebugForceFloor >= 0)
		? FMath::Clamp(DebugForceFloor, 0, State->NumMips - 1)
		: FMath::Clamp(Request.TargetFloor, 0, State->MaxSparseMipIndex + 1);

	// Eager commit: anything in [EffectiveTarget, MaxSparse] that's eligible.
	// Touch-on-already-committed refreshes the chunk's LastUsedTime in the
	// backend; fresh commit either allocates or steals an eligible chunk.
	// On steal, the backend reports the prior owner so we can raise its
	// ResidentFloor below.
	for (int32 MipIndex = EffectiveTarget; MipIndex <= State->MaxSparseMipIndex; ++MipIndex)
	{
		if (!State->SparseEligible[MipIndex])
		{
			continue;
		}
		const FTmvSparseMipCommitResult CommitResult = Backend->CommitMip(Request.Texture, MipIndex);
		if (CommitResult.StolenFromTexture && CommitResult.StolenFromTexture != Request.Texture)
		{
			if (FTextureState* VictimState = States.Find(CommitResult.StolenFromTexture))
			{
				// Victim lost mip CommitResult.StolenFromMip. Raise its floor
				// to one above that mip so the converter clamps SRVs and skips
				// any draw pass that would have sampled the now-unmapped mip.
				VictimState->ResidentFloor = FMath::Max(VictimState->ResidentFloor, CommitResult.StolenFromMip + 1);
			}
		}
	}

	// Republish resident_floor: smallest mip that is currently committed.
	int32 NewFloor = State->NumMips;
	for (int32 MipIndex = 0; MipIndex < State->NumMips; ++MipIndex)
	{
		if (Backend->IsMipCommitted(Request.Texture, MipIndex))
		{
			NewFloor = MipIndex;
			break;
		}
	}
	State->ResidentFloor = NewFloor;
}

void FTmvSparseOutputManager::RequestFullResidency(FRHITexture* Texture)
{
	if (!Backend.IsValid() || !Texture)
	{
		return;
	}
	FScopeLock ScopeLock(&Lock);
	FTextureState* State = States.Find(Texture);
	if (!State)
	{
		return;
	}
	const bool bWasPinned = (State->FullResidencyRefCount > 0);
	++State->FullResidencyRefCount;
	if (!bWasPinned)
	{
		// 0 -> 1: protect this texture's chunks from being stolen for the
		// duration of the MRQ / capture pass.
		Backend->SetTexturePinned(Texture, true);
	}
	for (int32 MipIndex = 0; MipIndex <= State->MaxSparseMipIndex; ++MipIndex)
	{
		if (State->SparseEligible[MipIndex])
		{
			Backend->CommitMip(Texture, MipIndex);
		}
	}
	State->ResidentFloor = 0;
}

void FTmvSparseOutputManager::ReleaseFullResidency(FRHITexture* Texture)
{
	if (!Texture)
	{
		return;
	}
	FScopeLock ScopeLock(&Lock);
	FTextureState* State = States.Find(Texture);
	if (!State || State->FullResidencyRefCount == 0)
	{
		return;
	}
	--State->FullResidencyRefCount;
	if (State->FullResidencyRefCount == 0 && Backend.IsValid())
	{
		// 1 -> 0: chunks become stealable again on the next CommitMip from
		// another texture.
		Backend->SetTexturePinned(Texture, false);
	}
}

int32 FTmvSparseOutputManager::GetResidentFloor(FRHITexture* Texture) const
{
	FScopeLock ScopeLock(&Lock);
	if (const FTextureState* State = States.Find(Texture))
	{
		return State->ResidentFloor;
	}
	return 0;
}

void FTmvSparseOutputManager::Dump(FOutputDevice& Ar) const
{
	FScopeLock ScopeLock(&Lock);
	const TCHAR* RhiName = GDynamicRHI ? GDynamicRHI->GetName() : TEXT("(null)");
	Ar.Logf(TEXT("FTmvSparseOutputManager: RHI=%s, IsSparseOutputMipEnabled=%s, BudgetMB=%d, MinBytesMB=%d, HysteresisSeconds=%d, DebugForceFloor=%d"),
		RhiName,
		UE::TmvMedia::IsSparseOutputMipEnabled() ? TEXT("true") : TEXT("false"),
		UE::TmvMedia::GetSparseOutputMipBudgetMB(),
		UE::TmvMedia::GetSparseOutputMipMinBytesMB(),
		UE::TmvMedia::GetSparseOutputMipHysteresisSeconds(),
		UE::TmvMedia::GetSparseOutputMipDebugForceFloor());

	if (!Backend.IsValid())
	{
		Ar.Logf(TEXT("FTmvSparseOutputManager: no backend registered for running RHI; sparse-output path disabled."));
		return;
	}
	Ar.Logf(TEXT("FTmvSparseOutputManager: %d tracked textures, total committed bytes = %lld."),
		States.Num(), Backend->GetTotalCommittedBytes());
	for (const TPair<FRHITexture*, FTextureState>& Pair : States)
	{
		const FTextureState& State = Pair.Value;
		Ar.Logf(TEXT("  Texture %p: floor=%d, maxSparse=%d, fullResidencyRefs=%d"),
			Pair.Key, State.ResidentFloor, State.MaxSparseMipIndex, State.FullResidencyRefCount);
	}
	Backend->Dump(Ar);
}

// End-to-end verification: confirms CVars + factory + manager + backend all wire
// up correctly when the editor is launched against the running RHI. Prints to the
// console output device so it's visible in the editor's output log and in any
// `-log` console window.
static FAutoConsoleCommand GTmvSparseOutputStatusCmd(
	TEXT("tmvmedia.sparseoutput.status"),
	TEXT("Print TMV sparse-output manager state: resolved RHI, backend registration, CVar values, per-texture residency."),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FTmvSparseOutputManager::Get().Dump(*GLog);
	}));
