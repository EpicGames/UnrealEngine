// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleConverter/TmvMediaFrameMipBufferPool.h"

#include "HAL/IConsoleManager.h"
#include "Misc/OutputDeviceRedirector.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderingThread.h"
#include "SampleConverter/SparseOutputMip/TmvSparseOutputManager.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "SampleConverter/TmvMediaTileResidency.h"
#include "TmvMediaFrameMipTextureBuffer.h"
#include "TmvMediaFrameMipStructuredBuffer.h"
#include "TmvMediaLog.h"
#include "TmvMediaTuningCVars.h"

// Route scanline-layout content through FTmvMediaFrameMipStructuredBuffer
// instead of FTmvMediaFrameMipTextureBuffer. The structured-buffer path
// uses a persistent-mapped upload heap + RHICopyBufferRegion, avoiding the
// per-region RHIUpdateTexture2D allocations that dominate render-thread
// CPU time at 16K. The structured buffer's GetPlaneBufferForComponent
// already handles scanline (line 137 of TmvMediaFrameMipStructuredBuffer.cpp);
// the rest of the path (sample converter, PlanarFull shader permutation)
// already handles scanline structured buffers.
static int32 GTmvUseStructuredBufferForScanline = 0;
static FAutoConsoleVariableRef CVarTmvUseStructuredBufferForScanline(
	TEXT("r.TmvMedia.UseStructuredBufferForScanline"),
	GTmvUseStructuredBufferForScanline,
	TEXT("If 1, route scanline-layout TMV mips through FTmvMediaFrameMipStructuredBuffer ")
	TEXT("(persistent upload heap + per-row CopyBufferRegion) instead of FTmvMediaFrameMipTextureBuffer ")
	TEXT("(per-tile UpdateTexture2D). Default 0 because the trade is not a clear win on ")
	TEXT("single-instance playback: render-thread CPU drops slightly but GPU cost rises (more ")
	TEXT("bytes copied per frame). Recommended for multi-instance scenes where render-thread ")
	TEXT("cost scales with instance count. CVar evaluation is dynamic - flipping at runtime ")
	TEXT("will discard mismatched pooled buffers on next acquire."),
	ECVF_Default);

// Phase 0 of mip-buffer virtualization: per-allocation logging + on-demand
// `tmvmedia.dumpmipbuffers` console command so we can confirm where VRAM goes
// before changing allocator behavior. Tracks every live FTmvMediaFrameMipBuffer
// (acquired or pooled) by listening at the only two sites that own the GPU
// resource: the `new FTmvMediaFrameMip{Structured,Texture}Buffer()` in
// AcquireBuffer and the `delete` calls in the deleter closures / pool teardown.
// Pool-release does NOT deregister - pooled-but-idle buffers still occupy VRAM.
static int32 GTmvLogMipBufferStats = 0;
static FAutoConsoleVariableRef CVarTmvLogMipBufferStats(
	TEXT("r.TmvMedia.LogMipBufferStats"),
	GTmvLogMipBufferStats,
	TEXT("If 1, log every Tmv mip buffer allocate/free with size, mip level, layout and ")
	TEXT("the running live count. Use 'tmvmedia.dumpmipbuffers' to print a per-mip ")
	TEXT("breakdown of currently-live (acquired or pooled) buffers on demand. Default 0."),
	ECVF_Default);

// Pool retention cap (slice A of mip-buffer virtualization). Without a cap, the
// pool grows monotonically as frames cycle through sample converters - we
// measured 36 mip-5 buffers retained for 9 active clips at 16K. Each
// pool-pinned buffer holds its full RHI allocation in VRAM, so a runaway pool
// is a direct VRAM cost. ReleaseBuffer consults this CVar and drops (rather
// than pools) any buffer arriving at a size class that already has this many
// pooled entries. Pool churn cost is bounded by RHI buffer creation, which is
// negligible compared to the per-frame upload-buffer copy this code lives
// behind, so default-on at a small value is safe.
static int32 GTmvMaxPooledBuffersPerSize = 4;
static FAutoConsoleVariableRef CVarTmvMaxPooledBuffersPerSize(
	TEXT("r.TmvMedia.MaxPooledBuffersPerSize"),
	GTmvMaxPooledBuffersPerSize,
	TEXT("Maximum number of free mip buffers retained per size class in the buffer pool. ")
	TEXT("Surplus buffers returning to the pool are released to the RHI instead of pooled. ")
	TEXT("Lower values reduce VRAM held in the idle pool at the cost of occasionally re-allocating ")
	TEXT("a mip buffer on cycle. Set to 0 to disable pool retention entirely (always free). ")
	TEXT("Set to INT_MAX to disable the cap (legacy unbounded behavior). Default 4."),
	ECVF_Default);

// Virtualized mip buffers (slice B+C of mip-buffer virtualization).
// Primary switch + budget formula. When VirtualizeMipBuffers != 0, mip buffers
// are sized for a bounded slot count instead of full worst-case tile count.
// budget(mip) = max(MipBudgetMin, MipBudgetBase >> mip), clamped to the
// actual worst-case tile count of the mip (so high-mip behaviour is identical
// to today when worst-case is already below budget). The runtime allocator
// and per-frame slot assignment honour this; the sample-side TileMappingBuffer
// rewires offsets per-frame from the resident set. Default ON; set to 0 to
// fall back to full-worst-case allocation if the bounded path causes issues.
static int32 GTmvVirtualizeMipBuffers = 1;
static FAutoConsoleVariableRef CVarTmvVirtualizeMipBuffers(
	TEXT("r.TmvMedia.VirtualizeMipBuffers"),
	GTmvVirtualizeMipBuffers,
	TEXT("Primary switch for mip-buffer virtualization. When 1, mip buffers allocate at a ")
	TEXT("bounded slot budget (see r.TmvMedia.MipBudgetBase/Min) and the decode + sample paths ")
	TEXT("use a logical-to-physical tile mapping table. Default 1; set 0 to fall back to ")
	TEXT("full-worst-case allocation."),
	ECVF_Default);

static int32 GTmvMipBudgetBase = 0;
static FAutoConsoleVariableRef CVarTmvMipBudgetBase(
	TEXT("r.TmvMedia.MipBudgetBase"),
	GTmvMipBudgetBase,
	TEXT("Optional explicit override for the per-mip slot budget. 0 (default) means ")
	TEXT("'use the UHD-viewport heuristic' (see r.TmvMedia.MipBudgetK). >0 forces the ")
	TEXT("legacy fixed formula: budget(mip) = max(MipBudgetMin, MipBudgetBase >> mip), ")
	TEXT("clamped to worst-case tiles. Use this only when you need a deterministic budget ")
	TEXT("that ignores tile size - typically for A/B testing."),
	ECVF_Default);

// Safety factor for the UHD-viewport heuristic. The peak visible-tile count
// for a single UHD camera at a mip transition is ~4x ViewportArea / TileArea
// (the higher-detail mip covers (TileSize/2) screen pixels per tile when the
// camera is about to fall back to the next-coarser mip). K=5 gives ~1.25x
// safety margin over that ceiling. Increase if you run multi-viewport
// scenarios (per-camera visibilities union, so budget scales linearly with
// active cameras looking at the same plate).
static int32 GTmvMipBudgetK = 6;
static FAutoConsoleVariableRef CVarTmvMipBudgetK(
	TEXT("r.TmvMedia.MipBudgetK"),
	GTmvMipBudgetK,
	TEXT("Multiplier in the UHD heuristic for the per-mip slot budget: ")
	TEXT("budget = K * (3840 * 2160) / (TileWidth * TileHeight), clamped per-mip to the ")
	TEXT("source's worst-case tile count and floored by MipBudgetMin. K=6 covers the 4x ")
	TEXT("mip-transition peak for a single UHD camera + ~50%% safety; scale linearly with ")
	TEXT("simultaneous-camera count if multi-viewport. Ignored when MipBudgetBase > 0. Default 6."),
	ECVF_Default);

static int32 GTmvMipBudgetMin = 64;
static FAutoConsoleVariableRef CVarTmvMipBudgetMin(
	TEXT("r.TmvMedia.MipBudgetMin"),
	GTmvMipBudgetMin,
	TEXT("Minimum slot count for any mip in the virtualization formula (floor). ")
	TEXT("Higher mips naturally have few tiles and may not be virtualized in practice ")
	TEXT("(worst-case clamp kicks in). Default 64."),
	ECVF_Default);

// Grow-on-demand controls. When GrowMipBuffersOnDemand=1 (default), the
// per-frame slot-allocation path detects when the visible-tile count exceeds
// the current buffer's slot capacity and reallocates the buffer at a larger
// size (doubling, capped at MipBudgetMax). Set to 0 for deterministic memory:
// the buffer stays at MipBudgetBase x tile bytes for the clip's lifetime,
// and excess visible tiles fall back to the shader's lower-mip upscale pass.
static int32 GTmvGrowMipBuffersOnDemand = 1;
static FAutoConsoleVariableRef CVarTmvGrowMipBuffersOnDemand(
	TEXT("r.TmvMedia.GrowMipBuffersOnDemand"),
	GTmvGrowMipBuffersOnDemand,
	TEXT("When 1 (default), a virtualized mip buffer doubles its slot count ")
	TEXT("(up to r.TmvMedia.MipBudgetMax) the first time visible tiles exceed its ")
	TEXT("current capacity. Eliminates the lower-mip upscale fallback in exchange ")
	TEXT("for one buffer realloc + GPU copy when a clip's framing widens. Set to ")
	TEXT("0 to lock VRAM at the initial budget for deterministic memory usage."),
	ECVF_Default);

// Cap on how far a grown buffer can expand. 0 (default) means "no cap" - the
// buffer can grow up to the mip's worst-case tile count, at which point it's
// effectively un-virtualized for that clip. A positive value caps slot count
// explicitly; once reached, further visible-overflow falls back via shader
// discard. Useful for hard VRAM ceilings.
static int32 GTmvMipBudgetMax = 0;
static FAutoConsoleVariableRef CVarTmvMipBudgetMax(
	TEXT("r.TmvMedia.MipBudgetMax"),
	GTmvMipBudgetMax,
	TEXT("Per-mip slot ceiling for grow-on-demand. 0 = no cap (grow up to worst-case ")
	TEXT("tile count, returning the clip to its pre-virtualization VRAM footprint). ")
	TEXT(">0 caps slot growth; overflow falls back to the shader's lower-mip upscale ")
	TEXT("path. Default 0."),
	ECVF_Default);

// Primary switch for the cross-clip shared tile pool. When 1, virtualized
// (tiled) buffers route their slot storage through a global FTmvSharedTilePool
// keyed by tile format, instead of each clip having its own GPU buffer.
// Saves VRAM in multi-plate scenes where simultaneous worst-case visibility
// is rare. The pool's GPU alloc + grow are fully async (no decode-thread
// stalls) and grow takes an FRWLock write across the buffer swap so
// decoders' in-flight mapped-pointer writes can't race with the unlock.
static int32 GTmvUseSharedTilePool = 1;
static FAutoConsoleVariableRef CVarTmvUseSharedTilePool(
	TEXT("r.TmvMedia.UseSharedTilePool"),
	GTmvUseSharedTilePool,
	TEXT("If 1 (default), virtualized tiled mip buffers share a global slot pool ")
	TEXT("keyed by per-tile-bytes + plane layout. Multiple clips of the same APV ")
	TEXT("format share one GPU buffer; eviction is global LRU across owners. ")
	TEXT("Pool GPU allocation and grow happen asynchronously; tiles whose slot ")
	TEXT("allocation lands while the pool is pending fall back via shader discard ")
	TEXT("for that frame. Set to 0 to keep each clip's buffer private (legacy)."),
	ECVF_Default);

// Sparse output media texture (D3D12 reserved-resource per-mip residency).
// Cuts per-clip output media-texture VRAM by holding the large mips in a
// reserved (tiled) resource. Physical backing is committed/decommitted per-mip
// driven by the decoder's ConverterParams.Viewports keys. On by default;
// bypassed on non-D3D12 RHIs and for plates whose mips all fall below the
// size threshold.
static int32 GTmvSparseOutputMip = 1;
static FAutoConsoleVariableRef CVarTmvSparseOutputMip(
	TEXT("r.TmvMedia.SparseOutputMip"),
	GTmvSparseOutputMip,
	TEXT("Primary switch for sparse output-media-texture residency. When 1 (default), the ")
	TEXT("FMediaTextureResource intermediate target is created as a D3D12 reserved ")
	TEXT("resource; FTmvSparseOutputManager commits physical heap tiles per-mip ")
	TEXT("driven by the decoder's Viewports keys, and clamps the sampling SRV to the ")
	TEXT("resident floor. D3D12-only; on other RHIs the path is bypassed regardless ")
	TEXT("of this CVar."),
	ECVF_Default);

static int32 GTmvSparseOutputMipMinBytesMB = 128;
static FAutoConsoleVariableRef CVarTmvSparseOutputMipMinBytesMB(
	TEXT("r.TmvMedia.SparseOutputMipMinBytesMB"),
	GTmvSparseOutputMipMinBytesMB,
	TEXT("Per-mip byte-threshold for sparse eligibility. A mip joins the sparse set ")
	TEXT("only when its (W*H*BytesPerPixel) is at least this many MB. Adapts the ")
	TEXT("sparse mip set to plate resolution automatically: a 16K plate uses mips ")
	TEXT("0..3; HD plates skip the path entirely. Default 128."),
	ECVF_Default);

static int32 GTmvSparseOutputMipBudgetMB = 2048;
static FAutoConsoleVariableRef CVarTmvSparseOutputMipBudgetMB(
	TEXT("r.TmvMedia.SparseOutputMipBudgetMB"),
	GTmvSparseOutputMipBudgetMB,
	TEXT("Hard ceiling on heap-pool size across all sparse output mips. When a fresh ")
	TEXT("commit would push the pool past this, the backend force-steals the oldest ")
	TEXT("LRU chunk regardless of hysteresis (safety valve only). Default 2048 (=2 GB)."),
	ECVF_Default);

static int32 GTmvSparseOutputMipHysteresisSeconds = 2;
static FAutoConsoleVariableRef CVarTmvSparseOutputMipHysteresisSeconds(
	TEXT("r.TmvMedia.SparseOutputMipHysteresisSeconds"),
	GTmvSparseOutputMipHysteresisSeconds,
	TEXT("Minimum wall-clock age of an owned chunk before it can be stolen by a new ")
	TEXT("commit. Prevents ping-pong thrash; no decommit ever fires at this threshold ")
	TEXT("(memory is only released at UnregisterOutput). Frame-rate independent. ")
	TEXT("Default 2."),
	ECVF_Default);

static int32 GTmvSparseOutputMipDebugForceFloor = -1;
static FAutoConsoleVariableRef CVarTmvSparseOutputMipDebugForceFloor(
	TEXT("r.TmvMedia.SparseOutputMipDebugForceFloor"),
	GTmvSparseOutputMipDebugForceFloor,
	TEXT("Debug: pin the residency floor for all sparse output textures. -1 ")
	TEXT("(default) follows Viewports. 0 commits everything including mip 0; ")
	TEXT("N>0 forbids any mip finer than N. Used to A/B residency tradeoffs ")
	TEXT("without depending on camera motion."),
	ECVF_Default);

namespace UE::TmvMedia
{
	bool IsVirtualizationEnabled()
	{
		return GTmvVirtualizeMipBuffers != 0;
	}

	int32 GetMipBudget(int32 MipLevel)
	{
		const int32 Base = GTmvMipBudgetBase;
		// 0 sentinel = "no fixed budget configured; fall back to heuristic".
		// Callers (FTmvMediaFrameMipInfo::GetBudgetedSlotCount) branch on
		// this 0 to choose between fixed and heuristic mode, so do NOT
		// clamp to MipBudgetMin here when Base==0.
		if (Base <= 0)
		{
			return 0;
		}
		if (MipLevel < 0)
		{
			return FMath::Max(GTmvMipBudgetMin, Base);
		}
		// MipLevel >= 31 would shift the full int32 width - clamp.
		const int32 Shift = FMath::Min(MipLevel, 30);
		const int32 Computed = (Base >> Shift);
		return FMath::Max(GTmvMipBudgetMin, Computed);
	}

	int32 ComputeHeuristicMipBudget(int32 TileWidth, int32 TileHeight)
	{
		// UHD viewport hard-coded as the heuristic's reference output. The
		// editor's level-viewport setups vary widely (1080p inset to 4K
		// full-screen, plus multi-viewport layouts) and the "right" viewport
		// is ambiguous when several panels can look at the same plate, so
		// we pick UHD as a sensible upper bound for typical setups. The
		// shortfall when the actual viewport is larger gets absorbed by
		// grow-on-demand; the surplus when it's smaller is paid in extra
		// VRAM headroom that's still 4-8x smaller than worst-case
		// allocation.
		constexpr int64 UhdWidth  = 3840;
		constexpr int64 UhdHeight = 2160;
		if (TileWidth <= 0 || TileHeight <= 0)
		{
			return GTmvMipBudgetMin;
		}
		const int64 ViewportArea = UhdWidth * UhdHeight;
		const int64 TileArea     = static_cast<int64>(TileWidth) * static_cast<int64>(TileHeight);
		const int64 K            = static_cast<int64>(FMath::Max(1, GTmvMipBudgetK));
		// K * ViewportArea / TileArea. Floored at MipBudgetMin so very-large
		// tiles still get at least some slots; clamped to INT32_MAX so the
		// downstream slot count fits in int32.
		const int64 Computed = (K * ViewportArea) / TileArea;
		const int64 Clamped  = FMath::Clamp<int64>(Computed, GTmvMipBudgetMin, static_cast<int64>(MAX_int32));
		return static_cast<int32>(Clamped);
	}

	bool IsGrowOnDemandEnabled()
	{
		return GTmvGrowMipBuffersOnDemand != 0;
	}

	int32 GetMipBudgetMax()
	{
		// 0 sentinel = no cap; callers should clamp to worst-case tile count
		// themselves (we don't know the mip's tile count here).
		return GTmvMipBudgetMax;
	}

	bool IsSharedTilePoolEnabled()
	{
		return GTmvUseSharedTilePool != 0;
	}

	bool IsSparseOutputMipEnabled()
	{
		return GTmvSparseOutputMip != 0;
	}

	int32 GetSparseOutputMipMinBytesMB()
	{
		return FMath::Max(0, GTmvSparseOutputMipMinBytesMB);
	}

	int32 GetSparseOutputMipBudgetMB()
	{
		return FMath::Max(0, GTmvSparseOutputMipBudgetMB);
	}

	int32 GetSparseOutputMipHysteresisSeconds()
	{
		return FMath::Max(0, GTmvSparseOutputMipHysteresisSeconds);
	}

	int32 GetSparseOutputMipDebugForceFloor()
	{
		// Negative sentinel preserved (callers branch on <0 to mean "follow Viewports").
		return GTmvSparseOutputMipDebugForceFloor;
	}
}

namespace UE::TmvMediaMipBufferStats::Private
{
	struct FRegistry
	{
		FCriticalSection Cs;
		TSet<const FTmvMediaFrameMipBuffer*> Live;
	};

	static FRegistry& Get()
	{
		static FRegistry R;
		return R;
	}

	static void RegisterAllocation(const FTmvMediaFrameMipBuffer* InBuffer, const FTmvMediaFrameMipInfo& InMipInfo, SIZE_T AllocSize)
	{
		if (!InBuffer)
		{
			return;
		}
		FRegistry& R = Get();
		int32 NewCount = 0;
		{
			FScopeLock Lock(&R.Cs);
			R.Live.Add(InBuffer);
			NewCount = R.Live.Num();
		}
		if (GTmvLogMipBufferStats)
		{
			UE_LOGF(LogTmvMedia, Display,
				"TmvMipBuffer.Alloc layout=%ls mip=%d res=%dx%d tiles=%dx%d size=%.2fMB live=%d",
				InMipInfo.Layout == ETmvMediaFrameBufferLayout::Tiled ? TEXT("Tiled") : TEXT("ScanLine"),
				InMipInfo.MipLevel, InMipInfo.Width, InMipInfo.Height,
				InMipInfo.NumTiles.X, InMipInfo.NumTiles.Y,
				double(AllocSize) / (1024.0 * 1024.0),
				NewCount);
		}
	}

	static void DeregisterAllocation(const FTmvMediaFrameMipBuffer* InBuffer)
	{
		if (!InBuffer)
		{
			return;
		}
		const SIZE_T Size = InBuffer->GetAllocatedBufferSize();
		const int32 Mip = InBuffer->GetMipInfoRef().MipLevel;
		FRegistry& R = Get();
		int32 NewCount = 0;
		{
			FScopeLock Lock(&R.Cs);
			R.Live.Remove(InBuffer);
			NewCount = R.Live.Num();
		}
		if (GTmvLogMipBufferStats)
		{
			UE_LOGF(LogTmvMedia, Display,
				"TmvMipBuffer.Free mip=%d size=%.2fMB live=%d",
				Mip, double(Size) / (1024.0 * 1024.0), NewCount);
		}
	}

	static void DeletePooledBuffer(FTmvMediaFrameMipBuffer* InBuffer)
	{
		// Always deregister-then-delete; both sites that destroy a pooled buffer
		// (CVar mismatch and pool teardown) must funnel through here so the
		// registry stays balanced even when GTmvLogMipBufferStats is off.
		DeregisterAllocation(InBuffer);
		delete InBuffer;
	}

	static FAutoConsoleCommand CmdDumpMipBuffers(
		TEXT("tmvmedia.dumpmipbuffers"),
		TEXT("Dump live TMV mip buffer allocations (per-mip totals + grand total) to LogTmvMedia."),
		FConsoleCommandDelegate::CreateLambda([]()
		{
			FRegistry& R = Get();
			FScopeLock Lock(&R.Cs);

			int64 TotalBytes = 0;
			int64 SharedPoolBytes = 0;
			TMap<int32, int64> BytesByMip;
			TMap<int32, int32> CountByMip;
			TMap<ETmvMediaFrameBufferLayout, int64> BytesByLayout;
			TSet<const void*> CountedSharedPools;
			for (const FTmvMediaFrameMipBuffer* B : R.Live)
			{
				if (!B)
				{
					continue;
				}
				SIZE_T S = B->GetAllocatedBufferSize();
				const FTmvMediaFrameMipInfo& MI = B->GetMipInfoRef();

				// Shared-mode buffers have AllocatedBufferSize=0; their GPU
				// bytes live on the shared pool. Account each pool once.
				const FTmvMediaTileResidency* Residency = B->GetResidency();
				if (S == 0 && Residency && Residency->IsShared() && Residency->SharedPool.IsValid())
				{
					const void* PoolPtr = Residency->SharedPool.Get();
					if (!CountedSharedPools.Contains(PoolPtr))
					{
						CountedSharedPools.Add(PoolPtr);
						const SIZE_T PoolBytes = Residency->SharedPool->GetAllocatedBufferSize();
						SharedPoolBytes += PoolBytes;
						TotalBytes      += PoolBytes;
						BytesByMip.FindOrAdd(MI.MipLevel) += PoolBytes;
					}
					CountByMip.FindOrAdd(MI.MipLevel)++;
					BytesByLayout.FindOrAdd(MI.Layout) += 0;
					continue;
				}

				TotalBytes += S;
				BytesByMip.FindOrAdd(MI.MipLevel) += S;
				CountByMip.FindOrAdd(MI.MipLevel)++;
				BytesByLayout.FindOrAdd(MI.Layout) += S;
			}

			UE_LOGF(LogTmvMedia, Display, "=== TmvMedia mip buffer stats ===");
			UE_LOGF(LogTmvMedia, Display, "Live buffers: %d, total: %.2f MB (%.3f GB) [shared pools: %.2f MB]",
				R.Live.Num(),
				double(TotalBytes) / (1024.0 * 1024.0),
				double(TotalBytes) / (1024.0 * 1024.0 * 1024.0),
				double(SharedPoolBytes) / (1024.0 * 1024.0));

			const int64 TiledBytes = BytesByLayout.FindRef(ETmvMediaFrameBufferLayout::Tiled);
			const int64 ScanLineBytes = BytesByLayout.FindRef(ETmvMediaFrameBufferLayout::ScanLine);
			UE_LOGF(LogTmvMedia, Display, "  Tiled: %.2f MB, ScanLine: %.2f MB",
				double(TiledBytes) / (1024.0 * 1024.0),
				double(ScanLineBytes) / (1024.0 * 1024.0));

			TArray<int32> Mips;
			BytesByMip.GetKeys(Mips);
			Mips.Sort();
			for (int32 M : Mips)
			{
				const int64 Bytes = BytesByMip[M];
				const int32 Count = CountByMip[M];
				UE_LOGF(LogTmvMedia, Display,
					"  mip %d: %.2f MB across %d buffers (avg %.2f MB each)",
					M,
					double(Bytes) / (1024.0 * 1024.0),
					Count,
					double(Bytes) / FMath::Max(1, Count) / (1024.0 * 1024.0));
			}

			// Output-texture sparse-residency state. Lives in a separate
			// VRAM bucket (D3D12 reserved-resource heaps) from the input
			// pool above; the manager owns per-mip committed-byte tracking
			// and prints into the same log device so a single command shows
			// both sides of the VRAM picture.
			UE_LOGF(LogTmvMedia, Display, "=== TmvMedia sparse-output residency ===");
			FTmvSparseOutputManager::Get().Dump(*GLog);
		}));
}

FTmvMediaFrameMipBufferPool::~FTmvMediaFrameMipBufferPool()
{
	FScopeLock ScopeLock(&MemoryPoolCriticalSection);

	// Copy memory pool array to be released on render thread.
	ENQUEUE_RENDER_COMMAND(DeletePooledBuffers)([InMemoryPool = MemoryPool](FRHICommandListImmediate& RHICmdList)
	{
		SCOPED_DRAW_EVENT(RHICmdList, FTmvMediaFrameMipBuffer_ReleaseMemoryPool);
		TArray<SIZE_T> KeysForIteration;
		InMemoryPool.GetKeys(KeysForIteration);
		for (SIZE_T Key : KeysForIteration)
		{
			TArray<FTmvMediaFrameMipBuffer*> AllValues;
			InMemoryPool.MultiFind(Key, AllValues);
			for (FTmvMediaFrameMipBuffer* MemoryPoolItem : AllValues)
			{
				UE::TmvMediaMipBufferStats::Private::DeletePooledBuffer(MemoryPoolItem);
			}
		}
	});
}

FTmvMediaFrameMipBufferHandle FTmvMediaFrameMipBufferPool::AcquireBuffer(const FTmvMediaFrameMipInfo& InMipInfo)
{
	// Use the budgeted size so virtualized mips key off their slot-budget
	// footprint rather than worst-case, and so pool reuse coalesces across
	// clips that share the same budget/layout. Non-virtualized mips degrade
	// to GetMemorySizeInBytes() inside GetBudgetedMemorySizeInBytes().
	const SIZE_T AllocSize = InMipInfo.GetBudgetedMemorySizeInBytes();

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TmvConverter.AllocBuffer %llu"), static_cast<uint64>(AllocSize)));
	TWeakPtr<FTmvMediaFrameMipBufferPool, ESPMode::ThreadSafe> BufferPoolWeak = AsWeak();

	// This function is attached to the shared pointer and is used to return any allocated memory to buffer pool.
	// Query the buffer's current allocation size at release time (NOT the
	// captured value from acquire time) - grow-on-demand can change the size
	// during the buffer's lifetime, and re-pooling under the stale key would
	// hide the larger buffer from same-size reuse lookups.
	auto BufferDeleter = [BufferPoolWeak](FTmvMediaFrameMipBuffer* InBufferToRelease)
	{
		if (TSharedPtr<FTmvMediaFrameMipBufferPool, ESPMode::ThreadSafe> BufferPool = BufferPoolWeak.Pin())
		{
			const SIZE_T CurrentSize = InBufferToRelease ? InBufferToRelease->GetAllocatedBufferSize() : 0;
			BufferPool->ReleaseBuffer(CurrentSize, InBufferToRelease);
		}
		else
		{
			ENQUEUE_RENDER_COMMAND(DeletePooledBuffers)([InBufferToRelease](FRHICommandListImmediate& RHICmdList)
			{
				UE::TmvMediaMipBufferStats::Private::DeletePooledBuffer(InBufferToRelease);
			});
		}
	};

	// Choose the buffer variant we want for this layout. Tiled always uses
	// the structured buffer; scanline picks based on a runtime CVar so the
	// fix can be A/B tested.
	const bool bWantStructured =
		InMipInfo.Layout == ETmvMediaFrameBufferLayout::Tiled
		|| (InMipInfo.Layout == ETmvMediaFrameBufferLayout::ScanLine
			&& GTmvUseStructuredBufferForScanline != 0);

	// Buffer that ends up being returned out of this function.
	FTmvMediaFrameMipBufferHandle AllocatedBuffer;

	// Capture a pool buffer that needs to be discarded so the ENQUEUE_RENDER_COMMAND
	// runs OUTSIDE the pool lock. Enqueueing inside the lock measurably extended
	// hold time and contended ReleaseBuffer on decode-worker threads (traced as
	// a ~10x slowdown on TmvConverter.ReturnPoolItem in virtualized-mode runs).
	FTmvMediaFrameMipBuffer* BufferToDiscard = nullptr;
	{
		FScopeLock ScopeLock(&MemoryPoolCriticalSection);
		if (FTmvMediaFrameMipBuffer** FoundBuffer = MemoryPool.Find(AllocSize))
		{
			// Pool hits must satisfy both the layout-compatible check AND
			// match the variant we'd allocate today. If the CVar toggled
			// since the pooled buffer was created, discard the old variant
			// rather than reuse it (so the CVar is dynamically effective).
			const bool bVariantMatches = ((*FoundBuffer)->IsStructuredBuffer() == bWantStructured);
			if (bVariantMatches && (*FoundBuffer)->TryUpdateMipInfo(InMipInfo))
			{
				AllocatedBuffer = MakeShareable(*FoundBuffer, MoveTemp(BufferDeleter));
			}
			else
			{
				BufferToDiscard = *FoundBuffer;
			}
			MemoryPool.Remove(AllocSize, *FoundBuffer);
		}
	}

	if (BufferToDiscard)
	{
		ENQUEUE_RENDER_COMMAND(DeletePooledBuffers)([BufferToDiscard](FRHICommandListImmediate& RHICmdList)
		{
			UE::TmvMediaMipBufferStats::Private::DeletePooledBuffer(BufferToDiscard);
		});
	}

	if (!AllocatedBuffer)
	{
		FTmvMediaFrameMipBuffer* NewBuffer;

		if (bWantStructured)
		{
			NewBuffer = new FTmvMediaFrameMipStructuredBuffer();
		}
		else
		{
			NewBuffer = new FTmvMediaFrameMipTextureBuffer();
		}

		if (!NewBuffer->RequestAllocation(InMipInfo))
		{
			delete NewBuffer;
			return nullptr;
		}

		UE::TmvMediaMipBufferStats::Private::RegisterAllocation(NewBuffer, InMipInfo, AllocSize);

		// This buffer will be automatically processed and returned to BufferPool once nothing keeps reference to it.
		AllocatedBuffer = MakeShareable(NewBuffer, MoveTemp(BufferDeleter));
	}

	// A buffer handed out here is a fresh content epoch (new alloc, or a pool
	// buffer whose previous tiles belong to a different owner). Reset the upload
	// tracking so the converter's render gate doesn't see a stale frame id from
	// the previous epoch.
	if (AllocatedBuffer)
	{
		AllocatedBuffer->ResetUploadTracking();
	}

	return AllocatedBuffer;
}

void FTmvMediaFrameMipBufferPool::ReleaseBuffer(SIZE_T InAllocSize, FTmvMediaFrameMipBuffer* InBuffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvConverter.ReturnPoolItem");

	// Cap pool retention per size class. Above the cap, the surplus buffer
	// is destroyed on the render thread instead of being pooled, so its VRAM
	// is reclaimed immediately. The cap CVar is read live so it can be tuned
	// without restart.
	bool bAcceptIntoPool = false;
	{
		FScopeLock ScopeLock(&MemoryPoolCriticalSection);
		const int32 ExistingForSize = MemoryPool.Num(InAllocSize);
		if (GTmvMaxPooledBuffersPerSize > 0 && ExistingForSize < GTmvMaxPooledBuffersPerSize)
		{
			MemoryPool.Add(InAllocSize, InBuffer);
			bAcceptIntoPool = true;
		}
	}

	if (!bAcceptIntoPool)
	{
		ENQUEUE_RENDER_COMMAND(DeletePooledBuffers)([InBuffer](FRHICommandListImmediate& RHICmdList)
		{
			UE::TmvMediaMipBufferStats::Private::DeletePooledBuffer(InBuffer);
		});
	}
}
