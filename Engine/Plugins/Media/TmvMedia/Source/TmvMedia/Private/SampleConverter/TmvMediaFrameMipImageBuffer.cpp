// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleConverter/TmvMediaFrameMipImageBuffer.h"

#include "ImageCore.h"
#include "RHIResources.h"
#include "Utils/TmvMediaFrameUtils.h"

FTmvMediaFrameMipImageBuffer::FTmvMediaFrameMipImageBuffer() = default;

FTmvMediaFrameMipImageBuffer::FTmvMediaFrameMipImageBuffer(const TSharedPtr<FImage>& InImage, int32 InMipLevel)
{
	if (InImage)
	{
		AllocatedBufferSize = InImage->GetImageSizeBytes();
		UE::TmvMedia::FrameUtils::PopulateMipInfoFromImageInfo(InMipLevel, *InImage, MipInfo);
		PlaneImages.Add(InImage);
	}
}

FTmvMediaFrameMipImageBuffer::~FTmvMediaFrameMipImageBuffer() = default;

bool FTmvMediaFrameMipImageBuffer::RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo)
{
	// todo: currently not used. Implement if needed.
	ensureMsgf(false, TEXT("FTmvMediaFrameMipImageBuffer::RequestAllocation is not implemented."));
	return false;
}

void* FTmvMediaFrameMipImageBuffer::GetMappedBuffer()
{
	// Only return the first plane buffer if we have one plane to be compliant with
	// the api that should return one contiguous buffer.
	if (PlaneImages.Num() == 1 && PlaneImages.IsValidIndex(0) && PlaneImages[0])
	{
		return PlaneImages[0]->GetPixelPointer(0,0);
	}
	return nullptr;
}

void* FTmvMediaFrameMipImageBuffer::GetPlaneBufferForComponent(int32 InComponentIndex)
{
	// Use the mip info to find the plane index for this component
	const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(InComponentIndex);
	if (PlaneImages.IsValidIndex(PlaneIndex) && PlaneImages[PlaneIndex])
	{
		return PlaneImages[PlaneIndex]->GetPixelPointer(0,0);
	}
	return nullptr;
}

FShaderResourceViewRHIRef FTmvMediaFrameMipImageBuffer::GetShaderResourceView(int32 InComponentIndex) const
{
	return nullptr;
}