// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleConverter/TmvMediaFrameMipMemoryBuffer.h"

#include "RHIResources.h"
#include "TmvMediaLog.h"

bool FTmvMediaFrameMipMemoryBuffer::RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo)
{
	// Prepare the plane buffers
	TArray<TSharedPtr<FPlaneTextureBuffer>> NewPlaneBuffers;
	NewPlaneBuffers.Reserve(InMipInfo.Planes.Num());

	for (const FTmvMediaFramePlaneInfo& PlaneInfo : InMipInfo.Planes)
	{
		// @todo: support interleaved layouts.
		if (PlaneInfo.ComponentLayout == ETmvMediaFrameComponentLayout::Interleaved)
		{
			UE_LOGF(LogTmvMedia, Verbose, "Failed to allocate a buffer for plane: interleaved layouts are not implemented yet.");
			return false;
		}

		// Allocate cpu texture buffer - note that CPU buffer is the expected memory size for the decoder.
		const uint64 CpuBufferSize = static_cast<uint64>(PlaneInfo.Stride) * static_cast<uint64>(PlaneInfo.NumLines);

		TSharedPtr<FPlaneTextureBuffer> NewPlaneBuffer = MakeShared<FPlaneTextureBuffer>();
		NewPlaneBuffer->Buffer.SetNum(CpuBufferSize);
		NewPlaneBuffer->Pitch = PlaneInfo.Stride;
		NewPlaneBuffers.Add(NewPlaneBuffer);
	}

	// Commit new resource once everything is validated.
	MipInfo = InMipInfo;
	AllocatedBufferSize = InMipInfo.GetMemorySizeInBytes();
	PlaneBuffers = NewPlaneBuffers;
	return true;
}

void* FTmvMediaFrameMipMemoryBuffer::GetMappedBuffer()
{
	// Only return the first plane buffer if we have one plane to be compliant with
	// the api that should return one contiguous buffer.
	if (PlaneBuffers.Num() == 1 && PlaneBuffers.IsValidIndex(0) && PlaneBuffers[0])
	{
		return PlaneBuffers[0]->Buffer.GetData();
	}
	return nullptr;
}

 void* FTmvMediaFrameMipMemoryBuffer::GetPlaneBufferForComponent(int32 InComponentIndex)
{
	// Use the mip info to find the plane index for this component
	const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(InComponentIndex);
	if (PlaneBuffers.IsValidIndex(PlaneIndex) && PlaneBuffers[PlaneIndex])
	{
		return PlaneBuffers[PlaneIndex]->Buffer.GetData();
	}
	return nullptr;
}

FShaderResourceViewRHIRef FTmvMediaFrameMipMemoryBuffer::GetShaderResourceView(int32 InComponentIndex) const
{
	return nullptr;
}

