// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "Templates/SharedPointer.h"

/** 
 * Implementation of texture cpu buffering backed by a memory buffer.
 * Generic memory implementation to be used mostly to store the result of cpu conversion
 * sent to the encoder.
 */
class FTmvMediaFrameMipMemoryBuffer : public FTmvMediaFrameMipBuffer
{
public:
	//~ Begin FTmvMediaFrameMipBuffer
	virtual bool RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo) override;
	virtual void WaitAllocation() override {}
	virtual bool TryUpdateMipInfo(const FTmvMediaFrameMipInfo& InMipInfo) override { return false; }
	virtual void* GetMappedBuffer() override;
	virtual void* GetPlaneBufferForComponent(int32 InComponentIndex) override;
	virtual FShaderResourceViewRHIRef GetShaderResourceView(int32 InComponentIndex) const override;
	virtual void CopyTileRegions(int32 InFrameId, const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions, TConstArrayView<FTmvMediaShaderTileDesc> InTileInfos) override {}
	virtual bool IsValidForRendering() const override
	{
		return false;	// This buffer type can't be used for rendering.
	}
	//~ End FTmvMediaFrameMipBuffer

private:
	/** Plane Cpu buffer accessed by the encoder. */
	struct FPlaneTextureBuffer
	{
		int32 Pitch = 0;
		TArray64<uint8> Buffer;
	};

	/** Plane Buffer array. */
	TArray<TSharedPtr<FPlaneTextureBuffer>> PlaneBuffers;
};
