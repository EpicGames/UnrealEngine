// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaFrameMipStructuredBuffer.h"

#include "CoreGlobals.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RHICommandList.h"
#include "SampleConverter/TmvSharedTilePool.h"
#include "TmvMediaLog.h"
#include "TmvMediaTuningCVars.h"

DECLARE_GPU_STAT_NAMED(TmvConverter_AllocateBuffer, TEXT("TmvConverter.AllocateBuffer"));
DECLARE_GPU_STAT_NAMED(TmvConverter_UploadBufferRegions, TEXT("TmvConverter.UploadBufferRegions"));

/** Time spent uploading buffer regions in render thread. */
DECLARE_CYCLE_STAT(TEXT("TmvMedia Upload Buffer Regions"), STAT_TmvMedia_UploadBufferRegions, STATGROUP_Media);

static bool bTmvConverterUseUploadHeap = true;

static FAutoConsoleVariableRef CVarTmvConverterUseUploadHeap(
	TEXT("r.TmvConverter.UseUploadHeap"),
	bTmvConverterUseUploadHeap,
	TEXT("Utilizes upload heap and copies raw converter buffer asynchronously.\n")
	TEXT("Read-only and to be set in a config file (requires restart)."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

// Scanline X-clip: per-row regions clipped to the affected tile rect's X range,
// instead of one region per tile rect covering all Y rows at full plane stride.
// 0 = full-row regions (overcopies in X, fewer CopyBufferRegion calls) - DEFAULT
// 1 = per-row X-clipped regions (no X overcopy, many more calls)
// X-clip only applies to plane components with Packed layout (NumComponents
// packed per sample); Interleaved planes fall back to full-row regions because
// each component sub-segment within a row would need its own region.
//
// Empirically (2026-05-20 A/B on SouTmv 16K scanline content): X-clip=1
// produces ~11k CopyBufferRegion calls per frame median (vs ~21 with X-clip=0)
// and the per-call RHI overhead alone moves RenderThread p95 from 17.5 ms to
// 48.6 ms. Kept as opt-in for future investigation (e.g. with coalesced ranges
// or a compute-shader gather copy) but the default is the full-row variant.
static int32 GTmvScanlineXClipRegions = 0;
static FAutoConsoleVariableRef CVarTmvScanlineXClipRegions(
	TEXT("r.TmvMedia.ScanlineXClipRegions"),
	GTmvScanlineXClipRegions,
	TEXT("If 1, scanline structured-buffer mips emit per-row CopyBufferRegion entries clipped to ")
	TEXT("the affected tile rect's X range - minimises bytes copied across PCIe at the cost of many ")
	TEXT("more (small) CopyBufferRegion calls. If 0 (default), emit one region per tile rect ")
	TEXT("covering all affected Y rows at full plane stride (overcopies in X, fewer calls). ")
	TEXT("Default 0 because per-row call-count overhead dominates the bytes-saved win on the ")
	TEXT("SouTmv 16K workload (RenderThread p95 17.5 ms vs 48.6 ms in A/B). ")
	TEXT("Interleaved component layouts always use the full-row fallback regardless of this CVar."),
	ECVF_RenderThreadSafe);

FTmvMediaFrameMipStructuredBuffer::FTmvMediaFrameMipStructuredBuffer()
{
	constexpr bool bIsManualReset = true; // Manually reset events stay triggered until reset.
	AllocationReadyEvent = FPlatformProcess::GetSynchEventFromPool(bIsManualReset);
	check(AllocationReadyEvent);
}

FTmvMediaFrameMipStructuredBuffer::~FTmvMediaFrameMipStructuredBuffer()
{
	// Shared-mode buffers don't own GPU resources; release the owner handle
	// back to the pool and skip the local unlock path. Route through the manager
	// (not SharedPool->UnregisterOwner directly) so the pool is dropped from the
	// manager once its last owner leaves - otherwise it lingers for the whole
	// process. This buffer's own SharedPool / Residency.SharedPool references
	// then free the pool as this destructor completes.
	if (IsSharedMode())
	{
		if (SharedPool.IsValid() && SharedOwnerHandle.IsValid())
		{
			FTmvSharedTilePoolManager::Get().ReleaseOwner(SharedPool, SharedOwnerHandle);
		}
		FPlatformProcess::ReturnSynchEventToPool(AllocationReadyEvent);
		return;
	}

	// Check if the buffer was locked before unlocking.
	if (UploadBufferRef && UploadBufferMapped)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvConverter.ReleasePoolItem");
		FRHICommandListImmediate::Get().UnlockBuffer(UploadBufferRef);
		UploadBufferMapped = nullptr;
	}

	FPlatformProcess::ReturnSynchEventToPool(AllocationReadyEvent);
}

bool FTmvMediaFrameMipStructuredBuffer::RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo)
{
	MipInfo = InMipInfo;

	// Shared-pool path: virtualized tiled mips can opt into a global slot
	// pool keyed by tile format. No per-buffer GPU resources are allocated;
	// all reads/writes (and the SRV the shader binds) go through the pool.
	if (InMipInfo.IsVirtualized()
		&& InMipInfo.Layout == ETmvMediaFrameBufferLayout::Tiled
		&& UE::TmvMedia::IsSharedTilePoolEnabled())
	{
		const int32 InitialBudget = InMipInfo.GetBudgetedSlotCount();
		SharedPool = FTmvSharedTilePoolManager::Get().GetOrCreatePool(InMipInfo, InitialBudget);
		if (SharedPool.IsValid())
		{
			SharedOwnerHandle = SharedPool->RegisterOwner();
			Residency.InitializeShared(SharedPool, SharedOwnerHandle);
			AllocatedBufferSize = 0; // GPU bytes accounted on the pool, not here
			AllocationReadyEvent->Trigger(); // shared mode is always ready
			return true;
		}
		// Fallback to local path if pool creation failed.
	}

	// When virtualization is enabled, the buffer is sized for a bounded slot
	// count (budget) rather than the worst-case tile count. Slot routing is
	// driven by FTmvMediaTileResidency; the decoder receives per-tile slot
	// indices via oapv_mip_request::tile_dst_slots.
	AllocatedBufferSize = InMipInfo.GetBudgetedMemorySizeInBytes();
	if (InMipInfo.IsVirtualized())
	{
		Residency.Initialize(InMipInfo.GetBudgetedSlotCount());
	}
	else
	{
		Residency.Initialize(0);
	}

	// RHI buffer allocation api limitation.
	if (AllocatedBufferSize > MAX_uint32)
	{
		UE_LOGF(LogTmvMedia, Error,
			"Frame Mip Structured Buffers larger than 4 Gb are not supported. Requested size: %llu",
			static_cast<uint64>(AllocatedBufferSize));
		return false;
	}

	// Current shader implementation limitation. The parameters for buffer stride are int32.
	if (InMipInfo.Layout == ETmvMediaFrameBufferLayout::Tiled && InMipInfo.GetAllPlaneMemorySizeInBytes() > static_cast<SIZE_T>(MAX_int32))
	{
		UE_LOGF(LogTmvMedia, Error,
			"The tile buffer stride (%llu bytes) exceed 2 GB limit. This is not supported by the conversion shader.",
			static_cast<uint64>(InMipInfo.GetAllPlaneMemorySizeInBytes()));
		return false;
	}

	// Allocate and unlock the structured buffer on render thread.
	ENQUEUE_RENDER_COMMAND(CreatePooledBuffer)([this, AllocSize = AllocatedBufferSize](FRHICommandListImmediate& RHICmdList)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TmvConverter.AllocBuffer_RenderThread %llu"), static_cast<uint64>(AllocSize)));

		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TmvConverter_AllocateBuffer, "TmvConverter.MipRender.AllocateBuffer");

		constexpr uint32 Stride = sizeof(uint16) * 2; // Distance in bytes between the elements of the buffer.

		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("TmvConverter.UploadBuffer"), AllocSize, Stride)
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic | EBufferUsageFlags::FastVRAM)
				.DetermineInitialState();
			UploadBufferRef = RHICmdList.CreateBuffer(CreateDesc);
			UploadBufferMapped = RHICmdList.LockBuffer(UploadBufferRef, 0, AllocSize, RLM_WriteOnly);

			if (bTmvConverterUseUploadHeap)
			{
				// The upload buffer will remain in CopySrc state.
				RHICmdList.Transition(FRHITransitionInfo(UploadBufferRef, CreateDesc.InitialState, ERHIAccess::CopySrc));
			}
		}

		if (bTmvConverterUseUploadHeap)
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("TmvConverter.DestBuffer"), AllocSize, Stride)
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::FastVRAM)
				.DetermineInitialState();

			ShaderAccessBufferRef = RHICmdList.CreateBuffer(CreateDesc);
			ShaderResourceView = RHICmdList.CreateShaderResourceView(ShaderAccessBufferRef, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(ShaderAccessBufferRef));
		}
		else
		{
			// Creating a ShaderResourceView (SRV) from a buffer that remains locked (mapped) on the CPU can be invalid.
			// Buffers should typically be unlocked before being used by the GPU as SRVs to avoid synchronization and undefined behavior issues.
			ShaderResourceView = RHICmdList.CreateShaderResourceView(UploadBufferRef, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(UploadBufferRef));
		}

		AllocationReadyEvent->Trigger();
	});
	return true;
}

void FTmvMediaFrameMipStructuredBuffer::WaitAllocation()
{
	// Shared-pool path: the local AllocationReadyEvent is triggered eagerly
	// inside RequestAllocation (the buffer's own "owner registration" is
	// instant), but the underlying GPU resource on the pool side is allocated
	// asynchronously. Decoder threads that fetch GetMappedBuffer()/GetPlaneBufferForComponent
	// before the pool's bGpuReady transitions to true get a null pointer,
	// ApvSetupImageBuffer fails the ensure(OutFrame->a[i]) check, and the
	// decode silently bails - the media sample never lands, ImgMedia's
	// blocking-fetch deadline (10s) fires, blocking gets disabled for the
	// session, and the shader renders the stale slot bytes ("green frames").
	// Forwarding to the pool's WaitForReady closes that window: callers of
	// WaitAllocation see a real "ready" state, same as the per-clip path.
	if (SharedPool.IsValid())
	{
		SharedPool->WaitForReady();
		return;
	}
	AllocationReadyEvent->Wait();
}

bool FTmvMediaFrameMipStructuredBuffer::TryUpdateMipInfo(const FTmvMediaFrameMipInfo& InMipInfo)
{
	// Shared-mode buffers are not eligible for pool reuse - they bind to a
	// specific FTmvSharedTilePool owner handle that's not transferable.
	// Returning false here makes the buffer pool delete the instance; a
	// fresh one will register cleanly with the shared pool on next acquire.
	if (IsSharedMode())
	{
		return false;
	}

	// For the tiled layout, it is just a big memory buffer, we only care for a compatible size.
	// Use budgeted size: virtualized buffers are smaller than worst-case, so a
	// pool-keyed lookup by AllocSize will already match same-budget mips of the
	// same dimensions, and TryUpdateMipInfo just confirms the new request also
	// fits the existing allocation.
	if (MipInfo.Layout != InMipInfo.Layout || InMipInfo.GetBudgetedMemorySizeInBytes() > AllocatedBufferSize)
	{
		return false;
	}

	const bool bWasVirtualized = MipInfo.IsVirtualized();
	const bool bNowVirtualized = InMipInfo.IsVirtualized();
	MipInfo = InMipInfo;

	// Pool reuse must reset residency: the buffer's bytes are from a
	// different TMV frame / clip, so existing tile->slot mappings are stale.
	if (bNowVirtualized)
	{
		const int32 NewCapacity = InMipInfo.GetBudgetedSlotCount();
		if (!bWasVirtualized || Residency.Capacity != NewCapacity)
		{
			Residency.Initialize(NewCapacity);
		}
		else
		{
			Residency.Reset();
		}
	}
	else if (bWasVirtualized)
	{
		Residency.Initialize(0);
	}
	return true;
}

void* FTmvMediaFrameMipStructuredBuffer::GetMappedBuffer()
{
	if (IsSharedMode())
	{
		return SharedPool.IsValid() ? SharedPool->GetMappedBuffer() : nullptr;
	}
	return UploadBufferMapped;
}

void* FTmvMediaFrameMipStructuredBuffer::GetPlaneBufferForComponent(int32 InComponentIndex)
{
	// Both layouts use the same in-tile / in-buffer plane offsets:
	//  - Scanline: Planes[] describe the full picture, PlaneBufferOffset is the absolute
	//    byte offset of plane c in the buffer (start of plane c's data).
	//  - Tiled: Planes[] describe per-tile geometry, PlaneBufferOffset is the intra-tile
	//    byte offset of plane c. The decoder uses it as a per-tile base bias and addresses
	//    tile (tx, ty) via base + (ty * NumTilesX + tx) * AllPlaneMemorySizeInBytes.
	void* MappedBase = IsSharedMode()
		? (SharedPool.IsValid() ? SharedPool->GetMappedBuffer() : nullptr)
		: UploadBufferMapped;
	if (!MappedBase)
	{
		// Diagnostic - this branch should never fire after WaitAllocation, but
		// if it does we lose this mip's decode silently (ApvSetupImageBuffer
		// will return nullptr, decoder skips, sample is empty, blocking-media
		// fetcher times out at 10s, shader renders stale slot bytes a.k.a.
		// "green frames").
		UE_LOGF(LogTmvMedia, Warning,
			"FTmvMediaFrameMipStructuredBuffer.GetPlaneBufferForComponent[%d] mip=%d: MappedBase=null shared=%d poolReady=%d",
			InComponentIndex, MipInfo.MipLevel, (int32)IsSharedMode(),
			IsSharedMode() && SharedPool.IsValid() ? (int32)SharedPool->IsReady() : -1);
		return nullptr;
	}
	SIZE_T PlaneBufferOffset = 0;
	const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(InComponentIndex);
	if (MipInfo.GetPlaneBufferOffset(PlaneIndex, PlaneBufferOffset))
	{
		return static_cast<uint8*>(MappedBase) + PlaneBufferOffset;
	}
	return nullptr;
}

FShaderResourceViewRHIRef FTmvMediaFrameMipStructuredBuffer::GetShaderResourceView(int32 InComponentIndex) const
{
	if (IsSharedMode())
	{
		return SharedPool.IsValid() ? SharedPool->GetShaderResourceView() : FShaderResourceViewRHIRef();
	}
	return ShaderResourceView;
}

namespace UE::TmvMediaFrameMipStructuredBuffer::Private
{
	struct FBufferRegion
	{
		int64 Offset;
		int64 Size;
	};

	// For scanline layout, each "tile rect" in InTileRegions describes a
	// rectangle of tiles in pixel space. Two strategies, selected by the
	// r.TmvMedia.ScanlineXClipRegions CVar:
	//
	// XClip OFF (CVar=0, default): for each plane x tile rect, emit ONE byte range
	// covering the affected Y rows at full plane stride. Few calls,
	// overcopies in X (full plane width instead of the tile rect's X range).
	//
	// XClip ON (CVar=1): for each plane x tile rect, emit one
	// byte range PER affected Y row, clipped to the tile rect's X range.
	// Many more calls, but transfers only the bytes that actually changed.
	// Only safe for Packed component layouts - Interleaved planes have
	// per-component sub-strides within a row, so they fall back to the
	// full-row strategy regardless of CVar.
	TArray<FBufferRegion> CalculateScanLineBufferRegionsToCopy(
		const FTmvMediaFrameMipInfo& InTmvMipInfo,
		TConstArrayView<FIntRect> InTileRegions,
		bool bXClipPerRow)
	{
		TArray<FBufferRegion> BufferRegionsToCopy;
		if (InTileRegions.IsEmpty() || InTmvMipInfo.Planes.IsEmpty())
		{
			return BufferRegionsToCopy;
		}

		const int32 TileWidth = InTmvMipInfo.TileWidth;
		const int32 TileHeight = InTmvMipInfo.TileHeight;
		if (TileHeight <= 0 || TileWidth <= 0)
		{
			return BufferRegionsToCopy;
		}

		// Rough capacity hint: full-row path is ~Planes * Rects regions,
		// X-clip path can be Planes * Rects * (rows-per-rect) regions.
		BufferRegionsToCopy.Reserve(InTileRegions.Num() * InTmvMipInfo.Planes.Num());

		for (int32 PlaneIndex = 0; PlaneIndex < InTmvMipInfo.Planes.Num(); ++PlaneIndex)
		{
			const FTmvMediaFramePlaneInfo& PlaneInfo = InTmvMipInfo.Planes[PlaneIndex];
			SIZE_T PlaneBufferOffset = 0;
			if (!InTmvMipInfo.GetPlaneBufferOffset(PlaneIndex, PlaneBufferOffset))
			{
				continue;
			}
			const int32 WidthRatio = PlaneInfo.WidthRatio > 0 ? PlaneInfo.WidthRatio : 1;
			const int32 HeightRatio = PlaneInfo.HeightRatio > 0 ? PlaneInfo.HeightRatio : 1;
			const int64 Stride = PlaneInfo.Stride;
			const int32 NumLines = PlaneInfo.NumLines;
			const int32 PlaneWidth = static_cast<int32>(PlaneInfo.Width);

			// X-clip is only safe for Packed layouts. Interleaved planes
			// pack per-component sub-segments within each row, so X-clipping
			// would have to emit one region per component per row.
			const bool bPlaneCanXClip =
				bXClipPerRow
				&& PlaneInfo.ComponentLayout == ETmvMediaFrameComponentLayout::Packed
				&& PlaneInfo.NumComponents > 0
				&& PlaneWidth > 0;

			const int64 BytesPerSample = bPlaneCanXClip
				? static_cast<int64>(PlaneInfo.NumComponents) * static_cast<int64>(PlaneInfo.GetBytesPerComponent())
				: 0;

			for (const FIntRect& TileRegion : InTileRegions)
			{
				const int32 PixelYMin = TileRegion.Min.Y * TileHeight;
				const int32 PixelYMax = TileRegion.Max.Y * TileHeight;

				const int32 PlaneYMin = FMath::Clamp(PixelYMin / HeightRatio, 0, NumLines);
				const int32 PlaneYMax = FMath::Clamp(PixelYMax / HeightRatio, 0, NumLines);
				if (PlaneYMax <= PlaneYMin)
				{
					continue;
				}

				if (bPlaneCanXClip)
				{
					const int32 PixelXMin = TileRegion.Min.X * TileWidth;
					const int32 PixelXMax = TileRegion.Max.X * TileWidth;
					const int32 PlaneXMin = FMath::Clamp(PixelXMin / WidthRatio, 0, PlaneWidth);
					const int32 PlaneXMax = FMath::Clamp(PixelXMax / WidthRatio, 0, PlaneWidth);
					if (PlaneXMax <= PlaneXMin)
					{
						continue;
					}

					const int64 RowStartByte = static_cast<int64>(PlaneXMin) * BytesPerSample;
					const int64 RowSize = static_cast<int64>(PlaneXMax - PlaneXMin) * BytesPerSample;
					if (RowSize <= 0)
					{
						continue;
					}

					for (int32 Y = PlaneYMin; Y < PlaneYMax; ++Y)
					{
						const int64 RegionOffset = static_cast<int64>(PlaneBufferOffset)
							+ static_cast<int64>(Y) * Stride
							+ RowStartByte;
						BufferRegionsToCopy.Add({RegionOffset, RowSize});
					}
				}
				else
				{
					const int64 RegionOffset = static_cast<int64>(PlaneBufferOffset) + static_cast<int64>(PlaneYMin) * Stride;
					const int64 RegionSize = static_cast<int64>(PlaneYMax - PlaneYMin) * Stride;
					if (RegionSize > 0)
					{
						BufferRegionsToCopy.Add({RegionOffset, RegionSize});
					}
				}
			}
		}
		return BufferRegionsToCopy;
	}

	// For the tiled layout, each tile's planes are laid out contiguously
	// (FTmvMediaFrameMipInfo: planes describe per-tile layout in tiled mode), and
	// FTmvMediaShaderTileDesc.Offsets[0] is the absolute buffer offset of the first
	// plane of that tile - i.e. the start of the tile's data. The tile is exactly
	// AllPlaneMemorySize bytes long. We emit one FBufferRegion per tile and then
	// coalesce contiguous byte ranges so the render-side issues a small number of
	// CopyBufferRegion calls (a row of row-major tiles collapses to a single copy).
	// Coalescing works by sorted byte offset rather than tile coordinates, so it is
	// independent of the decoder's tile-ordering convention.
	TArray<FBufferRegion> CalculateBufferRegionsToCopy(const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions, TConstArrayView<FTmvMediaShaderTileDesc> InTileInfos)
	{
		TArray<FBufferRegion> BufferRegionsToCopy;

		if (InTmvMipInfo.Layout != ETmvMediaFrameBufferLayout::Tiled)
		{
			return BufferRegionsToCopy;
		}

		const int32 NumTilesX = InTmvMipInfo.NumTiles.X;
		const int32 NumTilesY = InTmvMipInfo.NumTiles.Y;
		if (NumTilesX <= 0 || NumTilesY <= 0 || InTileInfos.Num() == 0 || InTileRegions.Num() == 0)
		{
			// An empty result falls through CopyTileRegions to the whole-buffer-copy path,
			// which is the correct default when we have no usable tile information.
			return BufferRegionsToCopy;
		}

		const int64 TileSizeBytes = static_cast<int64>(InTmvMipInfo.GetAllPlaneMemorySizeInBytes());
		if (TileSizeBytes <= 0)
		{
			return BufferRegionsToCopy;
		}

		// Pass 1: collect per-tile byte regions from each input tile-coordinate rect.
		int32 ReserveCount = 0;
		for (const FIntRect& Region : InTileRegions)
		{
			const int32 W = FMath::Max(0, FMath::Min(Region.Max.X, NumTilesX) - FMath::Max(Region.Min.X, 0));
			const int32 H = FMath::Max(0, FMath::Min(Region.Max.Y, NumTilesY) - FMath::Max(Region.Min.Y, 0));
			ReserveCount += W * H;
		}
		BufferRegionsToCopy.Reserve(ReserveCount);

		for (const FIntRect& RawRegion : InTileRegions)
		{
			const int32 MinX = FMath::Max(RawRegion.Min.X, 0);
			const int32 MaxX = FMath::Min(RawRegion.Max.X, NumTilesX);
			const int32 MinY = FMath::Max(RawRegion.Min.Y, 0);
			const int32 MaxY = FMath::Min(RawRegion.Max.Y, NumTilesY);

			for (int32 TileY = MinY; TileY < MaxY; ++TileY)
			{
				for (int32 TileX = MinX; TileX < MaxX; ++TileX)
				{
					const int32 TileIdx = TileY * NumTilesX + TileX;
					if (!InTileInfos.IsValidIndex(TileIdx))
					{
						continue;
					}
					// Offsets[] in the descriptor are stored as 16-bit-short indices for the
					// swizzle shader (StructuredBuffer<uint> is uint16-packed). Convert back
					// to bytes for the CPU-side copy region.
					const int64 TileStart = static_cast<int64>(InTileInfos[TileIdx].Offsets[0]) * 2;
					BufferRegionsToCopy.Add({TileStart, TileSizeBytes});
				}
			}
		}

		if (BufferRegionsToCopy.Num() <= 1)
		{
			return BufferRegionsToCopy;
		}

		// Pass 2: sort by byte offset and merge contiguous (or overlapping) regions.
		BufferRegionsToCopy.Sort([](const FBufferRegion& A, const FBufferRegion& B)
		{
			return A.Offset < B.Offset;
		});

		TArray<FBufferRegion> Coalesced;
		Coalesced.Reserve(BufferRegionsToCopy.Num());
		Coalesced.Add(BufferRegionsToCopy[0]);
		for (int32 i = 1; i < BufferRegionsToCopy.Num(); ++i)
		{
			FBufferRegion& Last = Coalesced.Last();
			const FBufferRegion& Next = BufferRegionsToCopy[i];
			const int64 LastEnd = Last.Offset + Last.Size;
			if (Next.Offset <= LastEnd)
			{
				// Contiguous or overlapping: extend if needed, otherwise drop (duplicate).
				const int64 NextEnd = Next.Offset + Next.Size;
				if (NextEnd > LastEnd)
				{
					Last.Size = NextEnd - Last.Offset;
				}
			}
			else
			{
				Coalesced.Add(Next);
			}
		}

		return Coalesced;
	}
}

void FTmvMediaFrameMipStructuredBuffer::CopyTileRegions(int32 InFrameId, const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions, TConstArrayView<FTmvMediaShaderTileDesc> InTileInfos)
{
	using namespace UE::TmvMediaFrameMipStructuredBuffer::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvCopyTileRegions.Structured");

	// If not using the upload buffer, no need to copy.
	if (!bTmvConverterUseUploadHeap)
	{
	 	return;
	}

	// Pick a per-region strategy based on layout. Tiled uses tile-keyed
	// byte ranges (still TODO). Scanline uses per-plane Y-row byte ranges
	// so we don't copy the whole 16K mip when only a few rows changed.
	TArray<FBufferRegion> TmpBufferRegionsToCopy;
	if (InTmvMipInfo.Layout == ETmvMediaFrameBufferLayout::Tiled)
	{
		TmpBufferRegionsToCopy = CalculateBufferRegionsToCopy(InTmvMipInfo, InTileRegions, InTileInfos);
	}
	else if (InTmvMipInfo.Layout == ETmvMediaFrameBufferLayout::ScanLine)
	{
		const bool bXClip = (GTmvScanlineXClipRegions != 0);
		TmpBufferRegionsToCopy = CalculateScanLineBufferRegionsToCopy(InTmvMipInfo, InTileRegions, bXClip);
	}

	const int32 NumRegionsIn  = InTileRegions.Num();
	const int32 NumRegionsOut = TmpBufferRegionsToCopy.Num();
	int64 TotalBytes = 0;
	for (const FBufferRegion& R : TmpBufferRegionsToCopy)
	{
		TotalBytes += R.Size;
	}
	const uint32 PlayerId = static_cast<uint32>(reinterpret_cast<UPTRINT>(this) >> 4) & 0xFFFFu;
	const TCHAR* Path = TmpBufferRegionsToCopy.IsEmpty() ? TEXT("WHOLE") : TEXT("PARTIAL");
	UE_LOGF(LogTmvMedia, Verbose,
		"TmvCopyTileRegions.Structured %ls player=%04x mip=%d frame=%d engFrame=%llu xclip=%d inRegs=%d outRegs=%d bytesKB=%lld",
		Path, PlayerId, InTmvMipInfo.MipLevel, InFrameId,
		static_cast<unsigned long long>(GFrameCounter),
		GTmvScanlineXClipRegions,
		NumRegionsIn, NumRegionsOut, TotalBytes / 1024);

	TSharedPtr<FTmvMediaFrameMipBuffer> BufferData = AsShared();

	// Prefer the member-captured refs that CaptureWriteState() snapshotted at
	// *decode* time, under the decoder's read lock. They name the SAME
	// upload+SA pair the decoder wrote into - even if a concurrent grow has
	// since swapped the pool's current refs to a fresh empty buffer (the
	// green-frame race). When no captures were taken, fall back to the pool's
	// current refs in shared mode, or the buffer's own refs otherwise. RHI ref
	// counting keeps the captured buffers alive past any swap until the render
	// command completes.
	FBufferRHIRef SrcUploadAtQueue = CapturedUploadBuffer.IsValid()
		? CapturedUploadBuffer
		: (IsSharedMode() && SharedPool.IsValid()
			? SharedPool->GetUploadBuffer()        : UploadBufferRef);
	FBufferRHIRef DstShaderAccessAtQueue = CapturedShaderAccessBuffer.IsValid()
		? CapturedShaderAccessBuffer
		: (IsSharedMode() && SharedPool.IsValid()
			? SharedPool->GetShaderAccessBuffer() : ShaderAccessBufferRef);

	// One-shot: clear captures so a stale snapshot is never reused for the
	// next decode -> CopyTileRegions pair. CaptureWriteState() repopulates
	// them before the next decode begins.
	CapturedUploadBuffer.SafeRelease();
	CapturedShaderAccessBuffer.SafeRelease();

	// Render gate: claim a monotonic upload sequence for this enqueue (worker
	// thread, before the command). The command records it once the GPU copy is
	// laid down, so IsUploadPending() stays true for THIS refresh until then -
	// correct even under paused-refresh where the media frame id is constant.
	const uint64 UploadSeq = BeginUpload();

	ENQUEUE_RENDER_COMMAND(CopyFromUploadBuffer)([BufferData, UploadSeq, SrcUpload = MoveTemp(SrcUploadAtQueue), DstShaderAccess = MoveTemp(DstShaderAccessAtQueue), BufferRegionsToCopy = MoveTemp(TmpBufferRegionsToCopy), FrameId = InFrameId, MipLevel = InTmvMipInfo.MipLevel, PlayerId](FRHICommandListImmediate& InRHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT_F(InRHICmdList, TmvConverter_UploadBufferRegions, "TmvMediaConverter.StartCopy", "TmvMediaConverter.StartCopy %d", FrameId);
		SCOPE_CYCLE_COUNTER(STAT_TmvMedia_UploadBufferRegions);
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvConverter.RecordCopyBufferRegion");

		if (!SrcUpload.IsValid() || !DstShaderAccess.IsValid())
		{
			// No copy possible - still mark the seq recorded so the gate doesn't
			// pin this mip as permanently pending.
			BufferData->RecordUploadExecuted(UploadSeq);
			return;
		}

		InRHICmdList.Transition(FRHITransitionInfo(DstShaderAccess, ERHIAccess::SRVMask, ERHIAccess::CopyDest));

		int32 NumCopies = 0;
		if (!BufferRegionsToCopy.IsEmpty())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvConverter.CopyBufferRegion.Partial");
			for (const FBufferRegion& Region : BufferRegionsToCopy)
			{
				InRHICmdList.CopyBufferRegion(DstShaderAccess, Region.Offset, SrcUpload, Region.Offset, Region.Size);
				NumCopies++;
			}
		}
		else
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvConverter.CopyBufferRegion.Whole");
			// Copy the whole buffer.
			InRHICmdList.CopyBufferRegion(DstShaderAccess, 0, SrcUpload, 0, DstShaderAccess->GetSize());
			NumCopies++;
		}

		// Make sure resource is in SRV mode again.
		InRHICmdList.Transition(FRHITransitionInfo(DstShaderAccess, ERHIAccess::CopyDest, ERHIAccess::SRVMask));

		// Render gate: this upload's GPU copy is now recorded on the render thread,
		// so the converter knows it ran before it samples the buffer.
		BufferData->RecordUploadExecuted(UploadSeq);

		UE_LOGF(LogTmvMedia, Verbose,
			"TmvCopyTileRegions.Structured.Recorded player=%04x mip=%d frame=%d engFrameRT=%llu copies=%d",
			PlayerId, MipLevel, FrameId,
			static_cast<unsigned long long>(GFrameCounterRenderThread), NumCopies);
	});
}

bool FTmvMediaFrameMipStructuredBuffer::IsValidForRendering() const
{
	if (IsSharedMode())
	{
		const FShaderResourceViewRHIRef SRV = SharedPool.IsValid() ? SharedPool->GetShaderResourceView() : FShaderResourceViewRHIRef();
		return SRV.IsValid();
	}
	if (!UploadBufferRef.IsValid())
	{
		return false;
	}
	if (bTmvConverterUseUploadHeap && (!ShaderAccessBufferRef.IsValid() || !ShaderAccessBufferRef->IsValid()))
	{
		return false;
	}
	return true;
}

void FTmvMediaFrameMipStructuredBuffer::CaptureWriteState()
{
	if (!IsSharedMode() || !SharedPool.IsValid())
	{
		return;
	}
	// Caller (decoder) holds the pool's read lock - safe to snapshot the
	// pool's current refs. They get consumed (and cleared) by the next
	// CopyTileRegions, ensuring the GPU per-tile copy targets the pair the
	// decoder actually wrote into even if a concurrent grow swaps the pool's
	// current refs to fresh empty buffers.
	CapturedUploadBuffer       = SharedPool->GetUploadBuffer();
	CapturedShaderAccessBuffer = SharedPool->GetShaderAccessBuffer();
}

bool FTmvMediaFrameMipStructuredBuffer::EnsureCapacityFor(int32 InMinCapacity)
{
	// Shared-mode path: delegate to the pool. The pool manages its own
	// allocation/copy/swap and updates Capacity on success.
	if (IsSharedMode())
	{
		if (!SharedPool.IsValid())
		{
			return false;
		}
		const bool bGrew = SharedPool->EnsureCapacityFor(InMinCapacity);
		// Mirror pool's new capacity into the local Residency.Capacity so
		// callers querying Residency see the up-to-date number.
		Residency.Capacity = SharedPool->GetCapacity();
		return bGrew;
	}

	if (InMinCapacity <= Residency.Capacity)
	{
		return true; // Already big enough.
	}

	if (!MipInfo.IsVirtualized())
	{
		// Non-virtualized: buffer is sized at worst case; we should never
		// hit this path (Residency.Capacity == WorstCase). Be defensive.
		return false;
	}

	// Compute target capacity: at least double, at least the requested
	// minimum, capped at the configured grow ceiling. Doubling is the
	// standard dynamic-array growth strategy and bounds re-allocations to
	// O(log N) across a clip's lifetime.
	const int32 MaxCapacity = MipInfo.GetMaxGrowSlotCount();
	if (Residency.Capacity >= MaxCapacity)
	{
		// Already at the cap (or grow-on-demand disabled). Caller falls
		// back via the shader's lower-mip upscale path.
		return false;
	}

	const int32 Doubled = FMath::Min(MaxCapacity, FMath::Max(1, Residency.Capacity) * 2);
	const int32 NewCapacity = FMath::Min(MaxCapacity, FMath::Max(Doubled, InMinCapacity));
	if (NewCapacity <= Residency.Capacity)
	{
		return false;
	}

	const SIZE_T PerTileBytes = static_cast<SIZE_T>(MipInfo.GetAllPlaneMemorySizeInBytes());
	const SIZE_T NewAllocSize = PerTileBytes * static_cast<SIZE_T>(NewCapacity);
	const SIZE_T OldAllocSize = AllocatedBufferSize;

	if (NewAllocSize > MAX_uint32)
	{
		UE_LOGF(LogTmvMedia, Error,
			"FTmvMediaFrameMipStructuredBuffer::EnsureCapacityFor: grown buffer (%llu bytes) exceeds 4 GB RHI limit; staying at current capacity %d.",
			static_cast<uint64>(NewAllocSize), Residency.Capacity);
		return false;
	}

	UE_LOGF(LogTmvMedia, Verbose,
		"TmvMipBuffer.Grow mip=%d %d -> %d slots (%.2f MB -> %.2f MB)",
		MipInfo.MipLevel, Residency.Capacity, NewCapacity,
		double(OldAllocSize) / (1024.0 * 1024.0),
		double(NewAllocSize) / (1024.0 * 1024.0));

	// Render-thread work: allocate new buffers, copy existing slot contents
	// over, unlock the old upload buffer, hand the new resources back to the
	// calling thread. We block the calling thread until the render thread
	// finishes this work; the structured buffer's RHI resources can only be
	// created / locked on the render thread, and decode must not begin
	// writing into the new buffer until the swap is complete.
	struct FGrowResult
	{
		FBufferRHIRef NewUpload;
		void* NewMapped = nullptr;
		FBufferRHIRef NewShaderAccess;
		FShaderResourceViewRHIRef NewSRV;
	};
	FGrowResult Result;

	FBufferRHIRef  OldUploadRef       = UploadBufferRef;
	void*          OldUploadMapped    = UploadBufferMapped;
	FBufferRHIRef  OldShaderAccessRef = ShaderAccessBufferRef;

	// Synchronization: this method runs on the media decode thread (not the
	// game thread), so FRenderCommandFence is unavailable (it asserts
	// IsInGameThread). Use a manual-reset FEvent triggered by the render
	// command - same pattern as RequestAllocation/WaitAllocation.
	FEventRef GrowDoneEvent(EEventMode::ManualReset);
	FEvent* const GrowDoneEventPtr = GrowDoneEvent.Get();

	ENQUEUE_RENDER_COMMAND(GrowMipBuffer)(
		[OldUploadRef, OldUploadMapped, OldShaderAccessRef, OldAllocSize, NewAllocSize, &Result, GrowDoneEventPtr]
		(FRHICommandListImmediate& RHICmdList)
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TmvConverter_AllocateBuffer, "TmvConverter.MipRender.Grow");
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvConverter.GrowBuffer_RenderThread");

			constexpr uint32 Stride = sizeof(uint16) * 2;

			// Allocate the new upload heap (mirrors RequestAllocation).
			const FRHIBufferCreateDesc UploadDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("TmvConverter.UploadBuffer"), NewAllocSize, Stride)
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic | EBufferUsageFlags::FastVRAM)
				.DetermineInitialState();
			Result.NewUpload = RHICmdList.CreateBuffer(UploadDesc);
			Result.NewMapped = RHICmdList.LockBuffer(Result.NewUpload, 0, NewAllocSize, RLM_WriteOnly);

			if (bTmvConverterUseUploadHeap)
			{
				RHICmdList.Transition(FRHITransitionInfo(Result.NewUpload, UploadDesc.InitialState, ERHIAccess::CopySrc));

				// Allocate the new shader-access buffer.
				const FRHIBufferCreateDesc DestDesc =
					FRHIBufferCreateDesc::CreateStructured(TEXT("TmvConverter.DestBuffer"), NewAllocSize, Stride)
					.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::FastVRAM)
					.DetermineInitialState();
				Result.NewShaderAccess = RHICmdList.CreateBuffer(DestDesc);
				Result.NewSRV = RHICmdList.CreateShaderResourceView(Result.NewShaderAccess, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(Result.NewShaderAccess));

				// Copy the previously-decoded slot bytes from the old shader-access
				// buffer to the new one. Only the first OldAllocSize bytes hold
				// valid data; the remainder is uninitialized and will be filled
				// by future per-tile decodes.
				if (OldShaderAccessRef.IsValid() && OldAllocSize > 0)
				{
					RHICmdList.Transition(FRHITransitionInfo(Result.NewShaderAccess, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
					RHICmdList.CopyBufferRegion(Result.NewShaderAccess, 0, OldShaderAccessRef, 0, OldAllocSize);
					RHICmdList.Transition(FRHITransitionInfo(Result.NewShaderAccess, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
				}
			}
			else
			{
				// No upload heap: SRV comes off the upload buffer directly.
				Result.NewSRV = RHICmdList.CreateShaderResourceView(Result.NewUpload, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(Result.NewUpload));
			}

			// Mirror the still-locked CPU bytes from old upload heap into the
			// new one so newly-decoded tiles writing past OldAllocSize coexist
			// with the previously-decoded slots that callers will retain via
			// the Residency map.
			if (OldUploadMapped && Result.NewMapped && OldAllocSize > 0)
			{
				FMemory::Memcpy(Result.NewMapped, OldUploadMapped, OldAllocSize);
			}

			// Unlock the old upload buffer; we hold the last ref via
			// OldUploadRef in this lambda, so the buffer is released when this
			// lambda returns.
			if (OldUploadRef && OldUploadMapped)
			{
				RHICmdList.UnlockBuffer(OldUploadRef);
			}

			GrowDoneEventPtr->Trigger();
		});

	// Block the calling (decode) thread until the render command completes.
	GrowDoneEvent->Wait();

	// Hot-swap to the new buffers. Old RHI refs drop here (the lambda
	// captured them by value, so the lambda's exit also drops its refs and
	// the RHI starts deferred deletion).
	UploadBufferRef        = Result.NewUpload;
	UploadBufferMapped     = Result.NewMapped;
	ShaderAccessBufferRef  = Result.NewShaderAccess;
	ShaderResourceView     = Result.NewSRV;
	AllocatedBufferSize    = NewAllocSize;

	// Expand the residency map. SlotToTile grows with InvalidTile sentinels
	// in the new slots; existing entries (slots 0..OldCapacity) are
	// preserved so already-resident tiles keep their slot indices.
	const int32 OldCapacity = Residency.Capacity;
	Residency.SlotToTile.SetNum(NewCapacity);
	for (int32 i = OldCapacity; i < NewCapacity; ++i)
	{
		Residency.SlotToTile[i] = FTmvMediaTileResidency::InvalidTile();
	}
	Residency.LruOrder.Reserve(NewCapacity);
	Residency.Capacity = NewCapacity;

	return true;
}