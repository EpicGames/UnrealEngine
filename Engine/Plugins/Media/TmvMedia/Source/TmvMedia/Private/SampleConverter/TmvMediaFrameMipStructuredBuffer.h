// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "SampleConverter/TmvMediaTileResidency.h"

/**
 * Implementation of a structured buffer that can be memory mapped and partially updated.
 * This is best suited for tiled memory layouts.
 */
class FTmvMediaFrameMipStructuredBuffer : public FTmvMediaFrameMipBuffer
{
public:
	/* Constructor */
	FTmvMediaFrameMipStructuredBuffer();

	/** Destructor to clean up render resources */
	virtual ~FTmvMediaFrameMipStructuredBuffer() override;

	//~ Begin FTmvMediaFrameMipBuffer
	virtual bool RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo) override;	
	virtual void WaitAllocation() override;
	virtual bool TryUpdateMipInfo(const FTmvMediaFrameMipInfo& InMipInfo) override;
	virtual void* GetMappedBuffer() override;
	virtual void* GetPlaneBufferForComponent(int32 InComponentIndex) override;
	virtual FShaderResourceViewRHIRef GetShaderResourceView(int32 InComponentIndex) const override;
	virtual void CopyTileRegions(int32 InFrameId, const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions, TConstArrayView<FTmvMediaShaderTileDesc> InTileInfos) override;
	virtual void CaptureWriteState() override;
	virtual bool IsValidForRendering() const override;
	virtual bool IsStructuredBuffer() const override
	{
		return true;
	}
	virtual bool EnsureCapacityFor(int32 InMinCapacity) override;
	virtual FTmvMediaTileResidency* GetResidency() override { return &Residency; }
	virtual const FTmvMediaTileResidency* GetResidency() const override { return &Residency; }
	//~ End FTmvMediaFrameMipBuffer

	/** True when this buffer is in shared-pool mode (no per-buffer GPU resources). */
	bool IsSharedMode() const { return SharedPool.IsValid(); }

private:
	/**
	 * Logical-to-physical tile mapping for virtualized mips. Only the structured
	 * buffer virtualizes, so the residency lives here rather than on the base
	 * (see FTmvMediaFrameMipBuffer::GetResidency). Populated in RequestAllocation
	 * when MipInfo.IsVirtualized(); reset on pool reuse / fresh allocation.
	 */
	FTmvMediaTileResidency Residency;

	/** Shared-pool reference (set when in shared mode). */
	TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe> SharedPool;
	FTmvSharedTilePool::FOwnerHandle SharedOwnerHandle;

	/**
	* This is the actual buffer reference that we need to keep after it is locked and until it is unlocked.
	* The buffer is used as an upload heap and will not be accessed by shader if CVar "r.TmvConverter.UseUploadHeap" is set.
	*/
	FBufferRHIRef UploadBufferRef;

	/** 
	* A pointer to mapped GPU memory.
	*/
	void* UploadBufferMapped = nullptr;

	/**
	* This buffer is used by the swizzling shader if CVar "r.TmvConverter.UseUploadHeap" is set and UploadBufferRef contents are copied into it.
	*/
	FBufferRHIRef ShaderAccessBufferRef;

	/** 
	* Resource View used by swizzling shader.
	*/
	FShaderResourceViewRHIRef ShaderResourceView;

	/** Event used to wait for completed buffer allocations. */
	FEvent* AllocationReadyEvent;

	/**
	 * Shared-pool snapshot captured by CaptureWriteState() under the pool's
	 * read lock just before decode writes. Consumed (and cleared) by the
	 * following CopyTileRegions so the GPU per-tile copy targets the same
	 * pair the decoder wrote into - even if a concurrent grow has since
	 * swapped the pool's current refs to fresh buffers. Null outside the
	 * decode -> CopyTileRegions pair and on non-shared buffers.
	 */
	FBufferRHIRef CapturedUploadBuffer;
	FBufferRHIRef CapturedShaderAccessBuffer;
};