// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaFrameMipTextureBuffer.h"

#include "CoreGlobals.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RHICommandList.h"
#include "TmvMediaLog.h"
#include "Utils/TmvMediaFrameUtils.h"

DECLARE_GPU_STAT_NAMED(TmvConverter_AllocateTexture, TEXT("TmvConverter.AllocateTexture"));
DECLARE_GPU_STAT_NAMED(TmvConverter_UploadTextureRegions, TEXT("TmvConverter.UploadTextureRegions"));

/** Time spent uploading texture regions in render thread. */
DECLARE_CYCLE_STAT(TEXT("TmvMedia Upload Texture Regions"), STAT_TmvMedia_UploadTextureRegions, STATGROUP_Media);

FTmvMediaFrameMipTextureBuffer::FTmvMediaFrameMipTextureBuffer()
{
}

FTmvMediaFrameMipTextureBuffer::~FTmvMediaFrameMipTextureBuffer()
{
}

namespace UE::TmvMedia
{
	/**
	 * Calculate the extent of a texture for the given plane.
	 * @remark supports interleaved layouts through emulation with a single component texture.
	 * @param InPlaneInfo Plane info 
	 * @return For packed planes, it returns directly the plane extent. For interleaved layouts, it returns
	 * an adjusted extent for a single component texture that emulated an interleaved layout.
	 */
	FIntPoint GetPlaneTextureExtent(const FTmvMediaFramePlaneInfo& InPlaneInfo)
	{
		if (InPlaneInfo.ComponentLayout == ETmvMediaFrameComponentLayout::Packed)
		{
			return FIntPoint(InPlaneInfo.Width, InPlaneInfo.Height);
		}

		// For interleaved component layouts, we need to do a bit of math to
		// figure out what should be the extent of the texture. (WIP)
		if (InPlaneInfo.NumComponents == 0) 
		{ 
			UE_LOGF(LogTmvMedia, Verbose, "GetPlaneTextureExtent: Invalid NumComponents=%d for interleaved plane.", InPlaneInfo.NumComponents); 
			return FIntPoint::ZeroValue; 
		} 

		const int32 BytesPerComponent = InPlaneInfo.GetBytesPerComponent();
		if (BytesPerComponent <= 0) 
		{ 
			UE_LOGF(LogTmvMedia, Verbose, "GetPlaneTextureExtent: Invalid BytesPerComponent=%d for interleaved plane (BitDepth=%d).", BytesPerComponent, InPlaneInfo.BitDepth); 
			return FIntPoint::ZeroValue; 
		} 
		
		int32 ComponentStride = InPlaneInfo.Width * BytesPerComponent;	// no padding
		int32 ComponentStrideWithPadding = InPlaneInfo.NumComponents ? InPlaneInfo.Stride / InPlaneInfo.NumComponents : InPlaneInfo.Stride;
		int32 ComponentPadding = ComponentStrideWithPadding - ComponentStride;
		
		// To match the memory layout, we have to include the horizontal padding between all the components but the last.
		// The shader will have to account for the padding offset when sampling the components.
		return FIntPoint((InPlaneInfo.Stride - ComponentPadding) / BytesPerComponent, InPlaneInfo.Height);
	}

	/**
	 * Compares the plane descriptors and determines if they would produce the same texture format.
	 * This is used for recycling buffers.
	 */
	bool WouldPlaneTexturesBeCompatible(const FTmvMediaFramePlaneInfo& InPlaneInfo0, const FTmvMediaFramePlaneInfo& InPlaneInfo1)
	{
		if (GetPlaneTextureExtent(InPlaneInfo0) != GetPlaneTextureExtent(InPlaneInfo1))
		{
			return false;
		}
		if (FrameUtils::GetPlanePixelFormat(InPlaneInfo0, /*Normalized*/ true) != FrameUtils::GetPlanePixelFormat(InPlaneInfo1, /*Normalized*/ true))
		{
			return false;
		}
		return true;
	}

	/**
	 * Indicate if the format is out of order, GR instead of RG, indicate shader should swizzle
	 * the components back in the correct order.
	 */
	bool ShouldShaderSwizzleComponents(EPixelFormat InPixelFormat)
	{
		PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
		switch (InPixelFormat)
		{
		case PF_G16R16:
		case PF_G16R16F:
		case PF_G32R32F:
			return true;
		}
		PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
		return false;
	}

	const TCHAR* ToString(ETmvMediaFrameComponentLayout InLayout)
	{
		switch (InLayout)
		{
		case ETmvMediaFrameComponentLayout::Packed:
			return TEXT("Packed");
		case ETmvMediaFrameComponentLayout::Interleaved:
			return TEXT("Interleaved");
		default:
			return TEXT("Unknown");
		}
	}

	FString GetFormatInfoForLogs(const FTmvMediaFramePlaneInfo& InPlaneInfo)
	{
		return FString::Printf(TEXT("Components: %d, Component Size: %d bytes (%d bits), Layout: %s"),
			InPlaneInfo.NumComponents, InPlaneInfo.GetBytesPerComponent(), InPlaneInfo.BitDepth, ToString(InPlaneInfo.ComponentLayout));
	}

	struct FBufferRegion
	{
		FIntPoint Offset;
		FIntPoint Size;
	};
	
	TArray<FBufferRegion> CalculateBufferRegionsToCopy(const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions)
	{
		FIntPoint TileSize(InTmvMipInfo.TileWidth, InTmvMipInfo.TileHeight);
		TArray<FBufferRegion> BufferRegionsToCopy;
		BufferRegionsToCopy.Reserve(InTileRegions.Num());
		for (const FIntRect& TileRegion : InTileRegions)
		{
			// For a planar buffer, the regions are rectangles.
			FIntPoint StartRegion = TileRegion.Min * TileSize;
			FIntPoint EndRegion = TileRegion.Max * TileSize;
			StartRegion.X = FMath::Clamp(StartRegion.X, 0, InTmvMipInfo.Width);
			StartRegion.Y = FMath::Clamp(StartRegion.Y, 0, InTmvMipInfo.Height);
			EndRegion.X = FMath::Clamp(EndRegion.X, 0, InTmvMipInfo.Width);
			EndRegion.Y = FMath::Clamp(EndRegion.Y, 0, InTmvMipInfo.Height);
			BufferRegionsToCopy.Add({StartRegion, EndRegion - StartRegion});
		}
		return BufferRegionsToCopy;
	}

	FUpdateTextureRegion2D CalculateTextureRegion(const FIntPoint& InOffset, const FIntPoint& InSize, const FTmvMediaFramePlaneInfo& InPlaneInfo)
	{
		// Convert in the size for the current plane.
		const int32 WidthRatio = InPlaneInfo.WidthRatio != 0 ? InPlaneInfo.WidthRatio : 1;
		const int32 HeightRatio = InPlaneInfo.HeightRatio != 0 ? InPlaneInfo.HeightRatio : 1;
		const int32 StartX = FMath::Clamp(InOffset.X / WidthRatio, 0, InPlaneInfo.Width);
		const int32 StartY = FMath::Clamp(InOffset.Y / HeightRatio, 0, InPlaneInfo.Height);
		const int32 EndX = FMath::Clamp((InOffset.X + InSize.X) / WidthRatio, 0, InPlaneInfo.Width);
		const int32 EndY = FMath::Clamp((InOffset.Y + InSize.Y) / HeightRatio, 0, InPlaneInfo.Height);
		return FUpdateTextureRegion2D(StartX, StartY, StartX, StartY, EndX - StartX, EndY - StartY);
	}
}

bool FTmvMediaFrameMipTextureBuffer::RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo)
{
	using namespace UE::TmvMedia;

	MipInfo = InMipInfo;
	AllocatedBufferSize = InMipInfo.GetMemorySizeInBytes();

	PlaneBuffers.Reset();
	PlaneBuffers.Reserve(InMipInfo.Planes.Num());

	TSharedPtr<FPlaneTextureResources> NewPlaneTextureResources = MakeShared<FPlaneTextureResources>();
	NewPlaneTextureResources->PlaneTextures.Reserve(InMipInfo.Planes.Num());

	for (const FTmvMediaFramePlaneInfo& PlaneInfo : InMipInfo.Planes)
	{
		// @todo: support interleaved layouts.
		if (PlaneInfo.ComponentLayout == ETmvMediaFrameComponentLayout::Interleaved)
		{
			UE_LOGF(LogTmvMedia, Verbose, "Failed to allocate a texture for plane: interleaved layouts are not implemented yet.");
			return false;
		}
		
		// Allocate cpu texture buffer - note that CPU buffer is the expected memory size for the decoder.
		const uint64 CpuBufferSize = static_cast<uint64>(PlaneInfo.Stride) * static_cast<uint64>(PlaneInfo.NumLines);

		TSharedPtr<FPlaneTextureBuffer> NewPlaneBuffer = MakeShared<FPlaneTextureBuffer>();
		NewPlaneBuffer->Buffer.SetNum(CpuBufferSize);
		NewPlaneBuffer->Pitch = PlaneInfo.Stride;
		PlaneBuffers.Add(NewPlaneBuffer);

		TSharedPtr<FPlaneTextureResource> NewPlaneTexture = MakeShared<FPlaneTextureResource>();
		NewPlaneTexture->Info = PlaneInfo;
		// While we may have pixel format, it may not be supported, we will only know on the render thread when trying to allocate the texture.
		NewPlaneTexture->PixelFormat = FrameUtils::GetPlanePixelFormat(PlaneInfo, /*Normalized*/ true);

		if (NewPlaneTexture->PixelFormat == PF_Unknown)
		{
			UE_LOGF(LogTmvMedia, Verbose,
				"Failed to allocate a texture for plane: no defined pixel format can accomodate the plane layout: %ls.",
				*GetFormatInfoForLogs(PlaneInfo));
			return false;
		}

		// Further validate if the platform supports the format.
		if (!GPixelFormats[NewPlaneTexture->PixelFormat].Supported)
		{
			UE_LOGF(LogTmvMedia, Verbose,
				"Failed to allocate a texture for plane: requested pixel format \"%ls\" is not supported on platform.",
				GPixelFormats[NewPlaneTexture->PixelFormat].Name);
			return false;
		}

		// This is not implemented yet. We might need to swizzle the components.
		if (ShouldShaderSwizzleComponents(NewPlaneTexture->PixelFormat))
		{
			// Not yet implemented, issue a diagnostic log about it.
			// @todo change this to a warning when better log/error handling per play session is in place.
			UE_LOGF(LogTmvMedia, Verbose, "Requested pixel format for texture plane would require a component swizzle but it is not implemented in shader.");
		}

		NewPlaneTexture->CpuBuffer = NewPlaneBuffer;
		NewPlaneTextureResources->PlaneTextures.Add(NewPlaneTexture);
	}

	PlaneTextureResources = NewPlaneTextureResources;
	TWeakPtr<FPlaneTextureResources> PlaneTextureResourcesWeak = NewPlaneTextureResources;
	
	// Allocate gpu texture
	ENQUEUE_RENDER_COMMAND(CreateTmvPooledTextureBuffer)([PlaneTextureResourcesWeak](FRHICommandListImmediate& RHICmdList)
		{
			TSharedPtr<FPlaneTextureResources> PlaneTextureResources = PlaneTextureResourcesWeak.Pin();
			if (!PlaneTextureResources)
			{
				return;
			}
			
			const static FLazyName ClassName(TEXT("FPlaneTextureResource"));
			for (const TSharedPtr<FPlaneTextureResource>& PlaneTexture : PlaneTextureResources->PlaneTextures)
			{
				const FRHITextureCreateDesc TextureDesc =
					FRHITextureCreateDesc::Create2D(TEXT("TmvPlaneTextureResource"))
					.SetExtent(GetPlaneTextureExtent(PlaneTexture->Info))
					.SetFormat(PlaneTexture->PixelFormat)
					.SetNumMips(1)
					.SetFlags(TexCreate_None | ETextureCreateFlags::ShaderResource)
					.SetInitialState(ERHIAccess::SRVMask)
					.SetClearValue(FClearValueBinding(FLinearColor::Black))
					.SetClassName(ClassName);

				PlaneTexture->Texture = RHICmdList.CreateTexture(TextureDesc);

				if (PlaneTexture->Texture)
				{
					PlaneTexture->Texture->SetName(TEXT("TmvPlaneTextureResource"));

					const FRHIViewDesc::FTextureSRV::FInitializer SrvDesc =
						FRHIViewDesc::CreateTextureSRV()
					    .SetDimensionFromTexture(PlaneTexture->Texture)
					    .SetFormat(PlaneTexture->PixelFormat);

					PlaneTexture->ShaderResourceView = RHICmdList.CreateShaderResourceView(PlaneTexture->Texture, SrvDesc);
				}
				else
				{
					UE_LOGF(LogTmvMedia, Verbose, "Failed to allocate a texture for plane. The format is probably not supported on the platform.");
				}
			}
	});
	return true;
}

void FTmvMediaFrameMipTextureBuffer::WaitAllocation()
{
	// No need to wait for the render thread since the plane buffers are on the cpu.
}

bool FTmvMediaFrameMipTextureBuffer::TryUpdateMipInfo(const FTmvMediaFrameMipInfo& InMipInfo)
{
	using namespace UE::TmvMedia;

	if (MipInfo.Layout != InMipInfo.Layout || MipInfo.Planes.Num() != InMipInfo.Planes.Num())
	{
		return false;
	}

	// Compare plane compatibility: memory and format.
	for (int32 PlaneIndex = 0; PlaneIndex < MipInfo.Planes.Num(); PlaneIndex++)
	{
		if (!WouldPlaneTexturesBeCompatible(MipInfo.Planes[PlaneIndex], InMipInfo.Planes[PlaneIndex]))
		{
			return false;
		}
	}

	// Memory layouts and formats should be compatible, so we can override.
	MipInfo = InMipInfo;
	return true;
}

void* FTmvMediaFrameMipTextureBuffer::GetMappedBuffer()
{
	// todo: if needed the plane buffers could be allocated in one big buffer and returned here.
	return nullptr;
}

void* FTmvMediaFrameMipTextureBuffer::GetPlaneBufferForComponent(int32 InComponentIndex)
{
	// Use the mip info to find the plane index for this component
	const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(InComponentIndex);
	if (PlaneBuffers.IsValidIndex(PlaneIndex) && PlaneBuffers[PlaneIndex])
	{
		return PlaneBuffers[PlaneIndex]->Buffer.GetData();
	}
	return nullptr;
}

FShaderResourceViewRHIRef FTmvMediaFrameMipTextureBuffer::GetShaderResourceView(int32 InComponentIndex) const
{
	if (PlaneTextureResources)
	{
		// Use the mip info to find the plane index for this component
		const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(InComponentIndex);
		if (PlaneTextureResources->PlaneTextures.IsValidIndex(PlaneIndex) && PlaneTextureResources->PlaneTextures[PlaneIndex])
		{
			return PlaneTextureResources->PlaneTextures[PlaneIndex]->ShaderResourceView;
		}
	}
	return nullptr;
}

void FTmvMediaFrameMipTextureBuffer::CopyTileRegions(int32 InFrameId, const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions, TConstArrayView<FTmvMediaShaderTileDesc> InTileInfos)
{
	using namespace UE::TmvMedia;

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvCopyTileRegions.Texture");

	TArray<FBufferRegion> TmpBufferRegionsToCopy = CalculateBufferRegionsToCopy(InTmvMipInfo, InTileRegions);

	const int32 NumRegionsIn  = InTileRegions.Num();
	const int32 NumRegionsOut = TmpBufferRegionsToCopy.Num();
	int64 TotalPixels = 0;
	for (const FBufferRegion& R : TmpBufferRegionsToCopy)
	{
		TotalPixels += static_cast<int64>(R.Size.X) * static_cast<int64>(R.Size.Y);
	}
	const uint32 PlayerId = static_cast<uint32>(reinterpret_cast<UPTRINT>(this) >> 4) & 0xFFFFu;
	const TCHAR* Path = TmpBufferRegionsToCopy.IsEmpty() ? TEXT("WHOLE") : TEXT("PARTIAL");
	UE_LOGF(LogTmvMedia, VeryVerbose,
		"TmvCopyTileRegions.Texture %ls player=%04x mip=%d frame=%d engFrame=%llu inRegs=%d outRegs=%d pixelsKpx=%lld",
		Path, PlayerId, InTmvMipInfo.MipLevel, InFrameId,
		static_cast<unsigned long long>(GFrameCounter),
		NumRegionsIn, NumRegionsOut, TotalPixels / 1024);

	TWeakPtr<FPlaneTextureResources> PlaneTextureResourcesWeak = PlaneTextureResources;
	TWeakPtr<FTmvMediaFrameMipBuffer> WeakSelf = AsWeak();

	// Render gate: claim a monotonic upload sequence (worker thread, before the
	// command); the command records it once the texture updates are laid down.
	const uint64 UploadSeq = BeginUpload();

	ENQUEUE_RENDER_COMMAND(CopyFromUploadBuffer)([PlaneTextureResourcesWeak, WeakSelf, UploadSeq, BufferRegionsToCopy = MoveTemp(TmpBufferRegionsToCopy), FrameId = InFrameId, MipLevel = InTmvMipInfo.MipLevel, PlayerId](FRHICommandListImmediate& InRHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT_F(InRHICmdList, TmvConverter_UploadTextureRegions, "TmvMediaConverter.StartCopy", "TmvMediaConverter.StartCopy %d", FrameId);
		SCOPE_CYCLE_COUNTER(STAT_TmvMedia_UploadTextureRegions);
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvConverter.RecordUpdateTexture2D");

		const TSharedPtr<FPlaneTextureResources> PlaneTextureResources = PlaneTextureResourcesWeak.Pin();
		if (!PlaneTextureResources)
		{
			// No update possible - still mark the seq recorded so the gate doesn't
			// pin this mip as permanently pending.
			if (const TSharedPtr<FTmvMediaFrameMipBuffer> Self = WeakSelf.Pin())
			{
				Self->RecordUploadExecuted(UploadSeq);
			}
			return;
		}

		int32 NumUpdates = 0;
		for (const TSharedPtr<FPlaneTextureResource>& PlaneTexture : PlaneTextureResources->PlaneTextures)
		{
			if (PlaneTexture && PlaneTexture->Texture && PlaneTexture->CpuBuffer)
			{
				const uint32 SourcePitch = PlaneTexture->CpuBuffer->Pitch;
				const uint8* Data = PlaneTexture->CpuBuffer->Buffer.GetData();

				if (!BufferRegionsToCopy.IsEmpty())
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvConverter.UpdateTexture2D.Partial");
					for (const FBufferRegion& Region : BufferRegionsToCopy)
					{
						// @todo: support interleaved layouts. Need a texture update for each interleaved component.
						InRHICmdList.UpdateTexture2D(PlaneTexture->Texture, 0, CalculateTextureRegion(Region.Offset, Region.Size, PlaneTexture->Info), SourcePitch, Data);
						NumUpdates++;
					}
				}
				else
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvConverter.UpdateTexture2D.Whole");
					const FUpdateTextureRegion2D Region(0, 0, 0, 0, PlaneTexture->Info.Width, PlaneTexture->Info.Height);
					// @todo: support interleaved layouts. Need a texture update for each interleaved component.
					InRHICmdList.UpdateTexture2D(PlaneTexture->Texture, 0, Region, SourcePitch, Data);
					NumUpdates++;
				}
			}
		}

		// Render gate: this upload's GPU work is now recorded on the render thread
		// (see FTmvMediaFrameMipBuffer::RecordUploadExecuted).
		if (const TSharedPtr<FTmvMediaFrameMipBuffer> Self = WeakSelf.Pin())
		{
			Self->RecordUploadExecuted(UploadSeq);
		}

		UE_LOGF(LogTmvMedia, VeryVerbose,
			"TmvCopyTileRegions.Texture.Recorded player=%04x mip=%d frame=%d engFrameRT=%llu updates=%d",
			PlayerId, MipLevel, FrameId,
			static_cast<unsigned long long>(GFrameCounterRenderThread), NumUpdates);
	});
}

bool FTmvMediaFrameMipTextureBuffer::IsValidForRendering() const
{
	if (!PlaneTextureResources || PlaneTextureResources->PlaneTextures.IsEmpty())
	{
		return false;
	}

	for (const TSharedPtr<FPlaneTextureResource>& PlaneTexture : PlaneTextureResources->PlaneTextures)
	{
		if (!PlaneTexture.IsValid() || !PlaneTexture->ShaderResourceView)
		{
			return false;
		}
	}

	return true;
}

