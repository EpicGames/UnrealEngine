// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "Templates/SharedPointer.h"

/**
 * Implementation of texture cpu buffering that can update regions.
 * This is best suited for storing the components on different planes.
 */
class FTmvMediaFrameMipTextureBuffer : public FTmvMediaFrameMipBuffer
{
public:
	/* Constructor */
	FTmvMediaFrameMipTextureBuffer();

	/** Destructor to clean up render resources */
	virtual ~FTmvMediaFrameMipTextureBuffer() override;

	//~ Begin FTmvMediaFrameMipBuffer
	virtual bool RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo) override;
	virtual void WaitAllocation() override;
	virtual bool TryUpdateMipInfo(const FTmvMediaFrameMipInfo& InMipInfo) override;
	virtual void* GetMappedBuffer() override;
	virtual void* GetPlaneBufferForComponent(int32 InComponentIndex) override;
	virtual FShaderResourceViewRHIRef GetShaderResourceView(int32 InComponentIndex) const override;
	virtual void CopyTileRegions(int32 InFrameId, const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions, TConstArrayView<FTmvMediaShaderTileDesc> InTileInfos) override;
	virtual bool IsValidForRendering() const override;
	//~ End FTmvMediaFrameMipBuffer

private:
	/** Plane Cpu buffer accessed by the decoder */ 
	struct FPlaneTextureBuffer
	{
		int32 Pitch = 0;
		TArray64<uint8> Buffer;
	};
	TArray<TSharedPtr<FPlaneTextureBuffer>> PlaneBuffers;
	
	/** Render thread resource: Texture for the memory plane. */
	struct FPlaneTextureResource
	{
		FTmvMediaFramePlaneInfo Info;
		EPixelFormat PixelFormat;
		TRefCountPtr<FRHITexture> Texture;
		FShaderResourceViewRHIRef ShaderResourceView;
		TSharedPtr<FPlaneTextureBuffer> CpuBuffer;
	};

	/** Container for all the render thread resources. */
	struct FPlaneTextureResources
	{
		TArray<TSharedPtr<FPlaneTextureResource>> PlaneTextures;
	};

	/** Reference to the render thread resources. */
	TSharedPtr<FPlaneTextureResources> PlaneTextureResources;
};