// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleConverter/TmvMediaTextureSampleConverter.h"

#include "ColorManagement/ColorSpace.h"
#include "ColorManagement/TransferFunctions.h"
#include "MediaShaders.h"
#include "Misc/Crc.h"
#include "PixelFormat.h"
#include "PostProcess/DrawRectangle.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RHIDefinitions.h"
#include "RHIResources.h"
#include "SampleConverter/SparseOutputMip/TmvSparseOutputBackend.h"
#include "SampleConverter/SparseOutputMip/TmvSparseOutputManager.h"
#include "SampleConverter/TmvMediaTileResidency.h"
#include "ScreenPass.h"
#include "TmvMediaFrameInfo.h"
#include "TmvMediaLog.h"
#include "TmvMediaTuningCVars.h"
#include "TmvStructuredBufferSwizzlingShader.h"
#include "TmvTextureBufferSwizzlingShader.h"
#include "Utils/TmvMediaFrameUtils.h"

DECLARE_GPU_STAT_NAMED(TmvMediaConverter, TEXT("TmvMediaConverter"));
DECLARE_GPU_STAT_NAMED(TmvMediaConverter_MipRender, TEXT("TmvMediaConverter.MipRender"));
DECLARE_GPU_STAT_NAMED(TmvMediaConverter_MipUpscale, TEXT("TmvMediaConverter.MipRender.MipUpscale"));

// PIMPL definition for the converter's tile-descriptor GPU cache. Keeps
// FBufferRHIRef / FShaderResourceViewRHIRef out of the converter's public-ish
// header (now Internal) and any future consumer of that header.
struct FTmvMediaTextureSampleConverter::FTileDescGpuCache
{
	struct FEntry
	{
		FBufferRHIRef Buffer;
		FShaderResourceViewRHIRef SRV;
		uint32 DescriptorsHash = 0;     // FCrc::MemCrc32 over the last-uploaded FTmvMediaShaderTileDesc array
		int32 Capacity = 0;             // number of FTmvMediaShaderTileDesc the buffer can hold
	};
	TMap<int32, FEntry> ByMip;
};

static bool bTmvConverterTintMipmaps = false;
static FAutoConsoleVariableRef CVarTmvConverterTintMipmaps(
	TEXT("TmvConverter.TintMipmaps"),
	bTmvConverterTintMipmaps,
	TEXT("The TMV sample converter will tint the mipmaps for visualizing (0-Red, 1-Green, 2-blue, 3-Yellow, 4+Pink).\n"),
	ECVF_RenderThreadSafe);

// Render gate: skip rendering a mip whose decode/upload for the frame being
// converted hasn't landed in the shader-access buffer yet, keeping the previous
// frame's output for that region instead of sampling not-yet-uploaded (green)
// memory. Closes the ordering race between the decode worker's CopyTileRegions
// and the converter's Convert. Default on; set 0 to render unconditionally
// (legacy behavior) for A/B comparison.
static int32 GTmvConverterSkipMipUntilUploaded = 1;
static FAutoConsoleVariableRef CVarTmvConverterSkipMipUntilUploaded(
	TEXT("r.TmvMedia.SkipMipUntilUploaded"),
	GTmvConverterSkipMipUntilUploaded,
	TEXT("If 1 (default), the TMV sample converter skips rendering a mip whose upload ")
	TEXT("for the current frame hasn't been recorded yet (keeping last frame's output ")
	TEXT("for the region) instead of sampling un-uploaded GPU memory and showing a green ")
	TEXT("frame. Set 0 to render unconditionally (pre-fix behavior)."),
	ECVF_RenderThreadSafe);

// Coarse-current fallback. The output RT is never cleared, so any region a mip
// doesn't redraw this frame keeps the PREVIOUS frame's pixels - under motion that
// shows as a stale ghost/seam (a gated mip frozen for several frames, or a mip
// only partially decoded this frame). When enabled, before drawing a mip's own
// (possibly partial) tiles the converter fills that output mip from the NEAREST
// coarser mip that is ready this frame, upscaled full-region, so the whole region
// is current; the mip's sharp tiles then overlay where decoded. Nearest (not
// coarsest) keeps it as sharp as possible.
static int32 GTmvConverterCoarseCurrentFallback = 2;
static FAutoConsoleVariableRef CVarTmvConverterCoarseCurrentFallback(
	TEXT("r.TmvMedia.CoarseCurrentFallback"),
	GTmvConverterCoarseCurrentFallback,
	TEXT("Fill a mip's region from the nearest COARSER mip that is ready this frame ")
	TEXT("(upscaled) so the never-cleared output RT never shows a previous frame where ")
	TEXT("the current frame didn't redraw (stale ghost/seam). 0=off; ")
	TEXT("1=only mips whose upload is still pending this frame (cheap; fixes gated-mip ")
	TEXT("stale); 2 (default)=also mips that are under-decoded this frame (decoded coverage ")
	TEXT("below r.TmvMedia.CoarseFallbackCoverageThreshold of their intended region) - fixes ")
	TEXT("partial-decode seams while leaving fully-decoded frames untouched."),
	ECVF_RenderThreadSafe);

// Coverage threshold (percent) below which a mip counts as under-decoded for the
// mode-2 coarse-current fallback. A mip whose decoded area is at least this % of
// its intended (full-visible) area is treated as complete and pays nothing; below
// it, the gap is filled from a coarser ready mip. Lower = fewer fallback draws
// (cheaper, but tolerates more stale); higher = more aggressive refresh.
static int32 GTmvConverterCoarseFallbackCoverageThreshold = 95;
static FAutoConsoleVariableRef CVarTmvConverterCoarseFallbackCoverageThreshold(
	TEXT("r.TmvMedia.CoarseFallbackCoverageThreshold"),
	GTmvConverterCoarseFallbackCoverageThreshold,
	TEXT("Percent of a mip's intended region that must be decoded this frame for it to ")
	TEXT("count as complete (no coarse-current fallback). Used only by ")
	TEXT("r.TmvMedia.CoarseCurrentFallback=2. Default 95."),
	ECVF_RenderThreadSafe);

// Adjacent-mip prefill. Hardware texture filtering (trilinear/aniso) can sample a
// mip just OUTSIDE the set the decoder was asked for this frame - when the
// per-pixel LOD lands between mip levels, or the camera moves so the sampled mip
// shifts to a neighbor before that mip's decode/upload has landed. Those neighbor
// mips are not in ConverterParams.Viewports, so r.TmvMedia.CoarseCurrentFallback
// (which only fills the requested set) never populates them and they read
// stale/garbage. When enabled, after rendering the visible set the converter
// dilates the populated range by one mip on each side: the finer neighbor
// (minVisible-1) upscaled from the finest ready visible mip, and the coarser
// neighbor (maxVisible+1) downscaled from the coarsest ready visible mip, over the
// visible footprint only - so filtering into an adjacent not-yet-ready mip during
// a transition reads coherent (blurry-but-current) data instead of glitching.
// Costs an extra bounded draw per side (the finer side is the larger); on by
// default. Complements CoarseCurrentFallback (which covers the in-set case).
static int32 GTmvConverterPrefillAdjacentMips = 1;
static FAutoConsoleVariableRef CVarTmvConverterPrefillAdjacentMips(
	TEXT("r.TmvMedia.PrefillAdjacentMips"),
	GTmvConverterPrefillAdjacentMips,
	TEXT("If 1 (default), after rendering the visible mips the TMV converter also fills ")
	TEXT("the mip immediately finer than the finest visible (upscaled) and the mip ")
	TEXT("immediately coarser than the coarsest visible (downscaled), over the visible ")
	TEXT("footprint, so hardware filtering into an adjacent not-yet-ready mip during a ")
	TEXT("transition shows coherent data instead of a glitch. 0=off. Complements ")
	TEXT("r.TmvMedia.CoarseCurrentFallback."),
	ECVF_RenderThreadSafe);

namespace UE::TmvMedia::Converter
{
	/** This function is similar to DrawScreenPass in OpenColorIODisplayExtension.cpp except it is catered for Viewless texture rendering. */
	template<typename TSetupFunction>
	void DrawScreenPass(
		FRHICommandListImmediate& RHICmdList,
		const FIntPoint& OutputResolution,
		const FIntRect& Viewport,
		const FScreenPassPipelineState& PipelineState,
		TSetupFunction SetupFunction)
	{
		RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.f, Viewport.Max.X, Viewport.Max.Y, 1.0f);

		SetScreenPassPipelineState(RHICmdList, PipelineState);

		// Setting up buffers.
		SetupFunction(RHICmdList);

		EDrawRectangleFlags DrawRectangleFlags = EDRF_UseTriangleOptimization;

		UE::Renderer::PostProcess::DrawPostProcessPass(
			RHICmdList,
			PipelineState.VertexShader,
			0, 0, OutputResolution.X, OutputResolution.Y,
			Viewport.Min.X, Viewport.Min.Y, Viewport.Width(), Viewport.Height(),
			OutputResolution,
			OutputResolution,
			INDEX_NONE,
			false,
			DrawRectangleFlags);
	}

	uint32 GetMaxValue(uint8 InBitDepth)
	{
		if (InBitDepth == 0)
		{
			return 0;
		}
		if (InBitDepth >= 32u)
		{
			return 0xFFFFFFFFu;
		}
		return (1u << static_cast<uint32>(InBitDepth)) - 1u;
	}

	/** Normalization factor for _UINT (not normalized) buffer lookup. */
	float GetDataNorm(ETmvMediaFrameComponentType InType, uint8 InBitDepth)
	{
		if (InType == ETmvMediaFrameComponentType::Int)
		{
			return InBitDepth > 0 ? 1.0f/static_cast<float>(GetMaxValue(InBitDepth)) : 1.0f;
		}
		return 1.0f;
	}

	/**
	 * Scale factor for normalized _UNORM texture lookup.
	 * This is scaling the data from a representation in InBitDepth bits to a number of bits aligned to bytes.
	 */
	float GetDataScale(ETmvMediaFrameComponentType InType, uint8 InBitDepth)
	{
		if (InType == ETmvMediaFrameComponentType::Int && InBitDepth > 0)
		{
			// 10 bits max value should scale to 16 bits max value e.g. 1023 should scale to 65535. The scale would be 65535/1023.
			return static_cast<float>(GetMaxValue((InBitDepth + 7) & ~7))/static_cast<float>(GetMaxValue(InBitDepth));
		}
		return 1.0f;
	}
	
	enum class EDataRangeType : uint8
    {
    	/** raw, non-negative whole number. These need to be normalized (or adjusted) to the bit depth. */
    	UInt,
    	/** sampled as floating point normalized between 0-1. These need to be scaled back (or denormalized) to map to bit depth. */
    	UNorm
    };
	
	float GetDataFactor(EDataRangeType InDataRangeType, ETmvMediaFrameComponentType InType, uint8 InBitDepth)
	{
		switch (InDataRangeType)
		{
		case EDataRangeType::UInt:
			return GetDataNorm(InType, InBitDepth);

		case EDataRangeType::UNorm:
		default:
			return GetDataScale(InType, InBitDepth);
		}
	}

	FMatrix CombineColorTransformAndOffset(const FMatrix& InMatrix, const FVector& InYUVOffset, float InDataNorm)
	{
			FMatrix Pre = FMatrix::Identity;
			Pre.M[0][0] = InDataNorm;
			Pre.M[1][1] = InDataNorm;
			Pre.M[2][2] = InDataNorm;
			Pre.M[0][3] = -InYUVOffset.X;
			Pre.M[1][3] = -InYUVOffset.Y;
			Pre.M[2][3] = -InYUVOffset.Z;
			return InMatrix * Pre;
	}

	// Calculating the tile buffer stride, with some extra validation.
	// @remark this value is only used by the shader in tiled mode.
	int32 GetTileBufferFullStride(const FTmvMediaFrameMipInfo& InMipInfo)
	{
		// Note: this should already be caught in the buffer allocation. This is just extra safety.
		
		// In tiled mode, the plane info is per tile.
		// The shader is limited to an int32 for this value.
		// It should be reasonable to expect the tile buffer stride to be small enough to fit in under 2 GB.
		const SIZE_T AllPlaneStride = InMipInfo.GetAllPlaneMemorySizeInBytes();
		if (ensureMsgf(AllPlaneStride <= MAX_int32, TEXT("The tile buffer stride exceed 2 GB. This is not supported by the conversion shader.")))
		{
			return static_cast<int32>(AllPlaneStride);
		}
		return 0;
	}

	static constexpr int NumMipColors = 5;
	static FLinearColor MipmapColorizing[5] =
		{
		{1.0f, 0.0f, 0.0f, 0.5f}, // red
		{0.0f, 1.0f, 0.0f, 0.5f}, // green
		{0.0f, 0.0f, 1.0f, 0.5f}, // blue
		{1.0f, 1.0f, 0.0f, 0.5f}, // yellow
		{1.0f, 0.0f, 1.0f, 0.5f}  // purple
		};
	
	void SetupTintColor(int32 InMipLevel, FTmvMediaShaderColorParameters& OutColorParams)
	{
		if (bTmvConverterTintMipmaps)
		{
			int TintIdx = FMath::Min(NumMipColors-1, InMipLevel);
			OutColorParams.MipTint = MipmapColorizing[TintIdx];
		}
		else
		{
			OutColorParams.MipTint = FLinearColor(1.0f, 1.0f, 1.0f, 0.0f);
		}
	}
	

	void SetupYuvColorParams( 
		const FTmvMediaFrameMipInfo& InMipInfo,
		EDataRangeType InDataRange,
		FTmvMediaShaderColorParameters& OutColorParams)
	{
		OutColorParams.bConvertYUV = (InMipInfo.ColorModel == ETmvMediaFrameColorModel::YUV && InMipInfo.Planes.IsValidIndex(0)) ? 1u : 0u;
		if (OutColorParams.bConvertYUV)
		{
			const ETmvMediaFrameComponentType ComponentType = InMipInfo.Planes[0].Type;
			const uint8 BitDepth = InMipInfo.Planes[0].BitDepth;
			const FVector YUVOffset = FrameUtils::GetYuvOffset(ComponentType, InMipInfo.ColorInfo.YuvMatrixRange, BitDepth);
			const float DataFactor = GetDataFactor(InDataRange, ComponentType, BitDepth);
			const FMatrix YuvToRgbMatrix = FrameUtils::GetYuvToRgbMatrix(InMipInfo.ColorInfo.YuvMatrix, InMipInfo.ColorInfo.YuvMatrixRange);
			OutColorParams.YUVMatrix = static_cast<FMatrix44f>(CombineColorTransformAndOffset(YuvToRgbMatrix, YUVOffset, DataFactor));
		}
		else
		{
			OutColorParams.YUVMatrix = FMatrix44f::Identity;
		}
	}
	
	/** Configure the color related shader parameters. */
	void SetupColorParams( 
		const FTmvMediaTextureSampleConverterParameters& InConverterParams, 
		const FTmvMediaFrameMipInfo& InMipInfo,
		FTmvMediaShaderColorParameters& OutColorParams)
	{
		// Color settings from the decoded media (if available).
		Color::EEncoding SourceEncoding = InMipInfo.ColorInfo.Encoding;
		Color::EReferenceWhite ReferenceWhite = InMipInfo.ColorInfo.ReferenceWhiteOverride;
		Color::FColorSpace SourceColorSpace = Color::FColorSpace(InMipInfo.ColorInfo.ColorSpace);
		Color::EChromaticAdaptationMethod Method = Color::DEFAULT_CHROMATIC_ADAPTATION_METHOD;

		// Apply potential media source color settings overrides.
		if (InConverterParams.SourceColorSettings.IsValid())
		{
			if (InConverterParams.SourceColorSettings->HasEncodingOverride())
			{
				SourceEncoding = InConverterParams.SourceColorSettings->GetEncodingOverride();
			}
			const Color::EReferenceWhite UserReferenceWhite = InConverterParams.SourceColorSettings->GetReferenceWhiteOverride();
			if (UserReferenceWhite != Color::EReferenceWhite::None)
			{
				ReferenceWhite = UserReferenceWhite;
			}
			SourceColorSpace = InConverterParams.SourceColorSettings->GetColorSpaceOverride(SourceColorSpace);
			Method = InConverterParams.SourceColorSettings->GetChromaticAdaptationMethod();
		}

		OutColorParams.bApplyColorTransform = 0u;

		// Note: we always decode to linear, if the destination RT is sRGB, it will encode to that automatically.
		if (SourceEncoding != Color::EEncoding::Linear && SourceEncoding != Color::EEncoding::None)
		{
			OutColorParams.bApplyColorTransform = 1u;
			OutColorParams.EOTF = static_cast<uint32>(SourceEncoding);
		}
		else
		{
			OutColorParams.EOTF = static_cast<uint32>(Color::EEncoding::None);
		}

		// Destination color space is the "working" color space (should be sRGB primaries).
		const UE::Color::FColorSpace& DestinationCS = UE::Color::FColorSpace::GetWorking();

		if (SourceColorSpace.Equals(DestinationCS))
		{
			OutColorParams.ColorSpaceMatrix = FMatrix44f::Identity;
		}
		else
		{
			OutColorParams.bApplyColorTransform = 1u;
			OutColorParams.ColorSpaceMatrix = UE::Color::Transpose<float>(UE::Color::FColorSpaceTransform(SourceColorSpace, DestinationCS, Method));
		}

		// Apply reference-white normalization as a pre-scale on the color-space transform,
		// so the shader's spec-literal transfer-function output (nits for PQ, BT.2100 linear
		// for HLG) is normalized to UE scene-linear. Non-unity factor also forces the color
		// conversion pass to run.
		const float NormalizationFactor = Color::GetReferenceWhiteLinearScale(SourceEncoding, ReferenceWhite);
		if (!FMath::IsNearlyEqual(NormalizationFactor, 1.0f))
		{
			OutColorParams.bApplyColorTransform = 1u;
			OutColorParams.ColorSpaceMatrix = OutColorParams.ColorSpaceMatrix.ApplyScale(NormalizationFactor);
		}
	}

	void RenderMip_PlaneTextures
		( FRHICommandListImmediate& RHICmdList
		, const FTextureRHIRef& RenderTargetTextureRHI
		, const FTmvMediaTextureSampleConverterParameters& ConverterParams
		, int32 TextureMipLevel
		, const FTmvMediaFrameMipBufferHandle& BufferData
		, const FIntPoint& TextureSize
		, const TArray<FIntRect>& MipViewports)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TmvMediaConverter_MipRender, "TmvMediaConverter.MipRender");

		FRHIRenderPassInfo RPInfo(RenderTargetTextureRHI, ERenderTargetActions::DontLoad_Store, nullptr, TextureMipLevel);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("TmvTextureSwizzle"));

		const FTmvMediaFrameMipInfo& MipInfo = BufferData->GetMipInfoRef();
		
		FTmvTextureBufferSwizzlePS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FTmvTextureBufferSwizzlePS::FNumComponents>(MipInfo.NumComponents - 1);

		FTmvTextureBufferSwizzlePS::FParameters Parameters = FTmvTextureBufferSwizzlePS::FParameters();

		Parameters.ColorParams.AlphaScale = MipInfo.Planes.Num() == 4 ? GetDataScale(MipInfo.Planes[3].Type, MipInfo.Planes[3].BitDepth) : 1.0f;
		SetupYuvColorParams(MipInfo, EDataRangeType::UNorm, Parameters.ColorParams);
		SetupColorParams(ConverterParams, MipInfo, Parameters.ColorParams);
		SetupTintColor(TextureMipLevel, Parameters.ColorParams);
		
		Parameters.InPlaneTexture0 = BufferData->GetShaderResourceView(0);
		Parameters.InPlaneTexture1 = BufferData->GetShaderResourceView(1);
		Parameters.InPlaneTexture2 = BufferData->GetShaderResourceView(2);
		Parameters.InPlaneTexture3 = BufferData->GetShaderResourceView(3);

		Parameters.InPlaneTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		TShaderMapRef<FTmvSwizzleVS> SwizzleShaderVS(ShaderMap);
		TShaderMapRef<FTmvTextureBufferSwizzlePS> SwizzleShaderPS(ShaderMap, PermutationVector);

		FScreenPassPipelineState PipelineState(SwizzleShaderVS, SwizzleShaderPS, TStaticBlendState<>::GetRHI(), TStaticDepthStencilState<false, CF_Always>::GetRHI());

		// If there are tiles determines if we should deliver tiles one by one or in a bulk.
		for (const FIntRect& Viewport : MipViewports)
		{
			DrawScreenPass(RHICmdList, TextureSize, Viewport, PipelineState, [&](FRHICommandListImmediate& InRHICmdList)
				{
					SetShaderParameters(InRHICmdList, SwizzleShaderPS, SwizzleShaderPS.GetPixelShader(), Parameters);
				});
		}

		// Resolve render target.
		RHICmdList.EndRenderPass();
	}
	
	void RenderMip_StructuredBuffer
		( FRHICommandListImmediate& RHICmdList
		, const FTextureRHIRef& RenderTargetTextureRHI
		, const FTmvMediaTextureSampleConverterParameters& ConverterParams
		, int32 SampleMipLevel
		, int32 TextureMipLevel
		, const FTmvMediaFrameMipBufferHandle& BufferData
		, const FIntPoint& SampleSize
		, const FIntPoint& TextureSize
		, const TArray<FIntRect>& MipViewports
		, FTmvMediaTextureSampleConverter::FTileDescGpuCache& TileDescGpuCache)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TmvMediaConverter_MipRender, "TmvMediaConverter.MipRender");

		FRHIRenderPassInfo RPInfo(RenderTargetTextureRHI, ERenderTargetActions::DontLoad_Store, nullptr, TextureMipLevel);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("TmvTextureSwizzle"));

		const FTmvMediaFrameMipInfo& MipInfo = BufferData->GetMipInfoRef();
		ETmvMediaFrameComponentType ComponentType = MipInfo.Planes.IsValidIndex(0) ? MipInfo.Planes[0].Type : ETmvMediaFrameComponentType::Int;
		
		FTmvStructuredBufferSwizzlePS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FTmvStructuredBufferSwizzlePS::FNumComponents>(MipInfo.NumComponents - 1);
		PermutationVector.Set<FTmvStructuredBufferSwizzlePS::FElementFormat>(ComponentType == ETmvMediaFrameComponentType::Float ? 0 : 1);

		// Can have tiles but be planar. Need another parameter for planar vs tiled layout.
		ETmvSwizzleBufferLayouts BufferLayout = ETmvSwizzleBufferLayouts::PlanarFull;
		switch (MipInfo.Layout)
		{
		case ETmvMediaFrameBufferLayout::Tiled:
			// Check if we have tile info for this mip level.
			if (ConverterParams.TileInfoPerMipLevel.Num() > SampleMipLevel && ConverterParams.TileInfoPerMipLevel[SampleMipLevel].Num() > 0)
			{
				// Tiles are resolved through lookup of tile info.
				BufferLayout = ETmvSwizzleBufferLayouts::TiledPartial;
			}
			else
			{
				// If no tile info, we assume it is a standard full tiled layout.
				BufferLayout = ETmvSwizzleBufferLayouts::TiledFull;
			}
			 break;
		case ETmvMediaFrameBufferLayout::ScanLine:
			BufferLayout = ETmvSwizzleBufferLayouts::PlanarFull;
			break;
		}
		PermutationVector.Set<FTmvStructuredBufferSwizzlePS::FBufferLayout>(static_cast<int32>(BufferLayout)); 

		FTmvStructuredBufferSwizzlePS::FParameters Parameters = FTmvStructuredBufferSwizzlePS::FParameters();
		Parameters.DestTextureSize = SampleSize;
		Parameters.TileSize = ConverterParams.TileDimWithBorders;
		Parameters.TileBufferFullStride =  MipInfo.Layout == ETmvMediaFrameBufferLayout::Tiled ? GetTileBufferFullStride(MipInfo) : 0;

		// Setup memory layout of the planes
		for (int32 ComponentIndex = 0; ComponentIndex < MipInfo.NumComponents; ComponentIndex++)
		{
			int32 ComponentIndexInPlane = 0;
			int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(ComponentIndex, &ComponentIndexInPlane);
			if (MipInfo.Planes.IsValidIndex(PlaneIndex))
			{
				const FTmvMediaFramePlaneInfo& PlaneInfo = MipInfo.Planes[PlaneIndex];

				// Calculating the offset of the component from the start of the mip buffer.
				SIZE_T ComponentOffsetInBytes = 0;
				MipInfo.GetPlaneBufferOffset(PlaneIndex, ComponentOffsetInBytes);
				ComponentOffsetInBytes += PlaneInfo.GetStartComponentOffsetInBytes(ComponentIndexInPlane);
				
				// Note: Shader expects strides and offsets in shorts.
				Parameters.TileBufferOffsets[ComponentIndex] = ComponentOffsetInBytes / 2;	// tiled mode, plane info is per tile.
				Parameters.TileBufferStrides[ComponentIndex] = PlaneInfo.Stride / 2;	// tiled mode, plane info is per tile.
				Parameters.PlaneBufferOffsets[ComponentIndex] = ComponentOffsetInBytes / 2;
				Parameters.PlaneBufferStrides[ComponentIndex] = PlaneInfo.Stride / 2;
				Parameters.PlaneWidthRatios[ComponentIndex] = PlaneInfo.WidthRatio;
				Parameters.PlaneHeightRatios[ComponentIndex] = PlaneInfo.HeightRatio;
			}
		}

		Parameters.ColorParams.AlphaScale = MipInfo.Planes.Num() == 4 ? GetDataNorm(MipInfo.Planes[3].Type, MipInfo.Planes[3].BitDepth) : 1.0f;
		
		SetupYuvColorParams(MipInfo, EDataRangeType::UInt, Parameters.ColorParams);
		SetupColorParams(ConverterParams, MipInfo, Parameters.ColorParams);
		SetupTintColor(TextureMipLevel, Parameters.ColorParams);
	
		if (ConverterParams.bHasTiles)
		{
			Parameters.NumTiles = FIntPoint(FMath::CeilToInt(float(SampleSize.X) / ConverterParams.TileDimWithBorders.X), FMath::CeilToInt(float(SampleSize.Y) / ConverterParams.TileDimWithBorders.Y));
		}
		
		if (ConverterParams.bHasTiles &&
			(ConverterParams.TileInfoPerMipLevel.Num() > SampleMipLevel && ConverterParams.TileInfoPerMipLevel[SampleMipLevel].Num() > 0))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("TmvMediaConverter.TileDesc");

			// Cache the structured buffer + SRV per mip level. The tile-descriptor set on a given
			// mip rarely changes frame to frame (only when the visible tile set shifts), so without
			// caching we re-create N tiny FRHIBuffer+SRV pairs every frame, fill the RHI deferred-
			// delete queue during mip transitions, and trigger FlushPendingDeleteRHIResources_
			// NonRealtime stalls on the render thread.
			const TArray<FTmvMediaShaderTileDesc>& Descs = ConverterParams.TileInfoPerMipLevel[SampleMipLevel];
			const int32 DescsNum = Descs.Num();
			const SIZE_T DescsBytes = static_cast<SIZE_T>(DescsNum) * sizeof(FTmvMediaShaderTileDesc);
			const uint32 DescsHash = FCrc::MemCrc32(Descs.GetData(), DescsBytes);

			FTmvMediaTextureSampleConverter::FTileDescGpuCache::FEntry& Cache = TileDescGpuCache.ByMip.FindOrAdd(SampleMipLevel);

			if (!Cache.Buffer.IsValid() || Cache.Capacity < DescsNum)
			{
				// First touch of this mip level, or the tile count grew past capacity: allocate.
				// This is the only path that hits the RHI buffer allocator; it should only fire
				// on initial decode and on rare growth events, not every frame.
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateStructured<FTmvMediaShaderTileDesc>(TEXT("TmvMediaConverter_TileDesc"), DescsNum)
					.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic | EBufferUsageFlags::FastVRAM)
					.SetInitActionInitializer()
					.DetermineInitialState();

				TRHIBufferInitializer<FTmvMediaShaderTileDesc> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
				Initializer.WriteArray(MakeConstArrayView(Descs));

				Cache.Buffer = Initializer.Finalize();
				Cache.SRV = RHICmdList.CreateShaderResourceView(Cache.Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(Cache.Buffer));
				Cache.Capacity = DescsNum;
				Cache.DescriptorsHash = DescsHash;
			}
			else if (Cache.DescriptorsHash != DescsHash)
			{
				// Buffer fits but contents changed: re-upload in place. No new RHI allocation,
				// no SRV recreation, nothing enters the deferred-delete queue.
				void* Mapped = RHICmdList.LockBuffer(Cache.Buffer, 0, DescsBytes, RLM_WriteOnly);
				FMemory::Memcpy(Mapped, Descs.GetData(), DescsBytes);
				RHICmdList.UnlockBuffer(Cache.Buffer);
				Cache.DescriptorsHash = DescsHash;
			}

			Parameters.TileMappingBuffer = Cache.SRV;
		}

		Parameters.UnswizzledBuffer = BufferData->GetShaderResourceView(0);

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		TShaderMapRef<FTmvSwizzleVS> SwizzleShaderVS(ShaderMap);
		TShaderMapRef<FTmvStructuredBufferSwizzlePS> SwizzleShaderPS(ShaderMap, PermutationVector);

		FScreenPassPipelineState PipelineState(SwizzleShaderVS, SwizzleShaderPS, TStaticBlendState<>::GetRHI(), TStaticDepthStencilState<false, CF_Always>::GetRHI());

		// If there are tiles determines if we should deliver tiles one by one or in a bulk.
		for (const FIntRect& Viewport : MipViewports)
		{
			DrawScreenPass(RHICmdList, TextureSize, Viewport, PipelineState, [&](FRHICommandListImmediate& InRHICmdList)
				{
					SetShaderParameters(InRHICmdList, SwizzleShaderPS, SwizzleShaderPS.GetPixelShader(), Parameters);
				});
		}

		// Resolve render target.
		RHICmdList.EndRenderPass();
	}

	void RenderMip
		( FRHICommandListImmediate& RHICmdList
		, const FTextureRHIRef& RenderTargetTextureRHI
		, const FTmvMediaTextureSampleConverterParameters& ConverterParams
		, int32 SampleMipLevel
		, int32 TextureMipLevel
		, const FTmvMediaFrameMipBufferHandle& BufferData
		, const FIntPoint& SampleSize
		, const FIntPoint& TextureSize
		, const TArray<FIntRect>& MipViewports
		, FTmvMediaTextureSampleConverter::FTileDescGpuCache& TileDescGpuCache)
	{
		if (!RenderTargetTextureRHI.IsValid() || !BufferData.IsValid())
		{
			return;
		}

		// Using the same transition pattern as ConvertTextureToOutput.
		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));

		if (BufferData->IsStructuredBuffer())
		{
			RenderMip_StructuredBuffer(RHICmdList, RenderTargetTextureRHI, ConverterParams, SampleMipLevel, TextureMipLevel, BufferData, SampleSize, TextureSize, MipViewports, TileDescGpuCache);
		}
		else
		{
			RenderMip_PlaneTextures(RHICmdList, RenderTargetTextureRHI, ConverterParams, TextureMipLevel, BufferData, TextureSize, MipViewports);
		}

		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::RTV, ERHIAccess::SRVGraphics));
	}


	bool RenderThreadSwizzler(FRHICommandListImmediate& RHICmdList, FTextureRHIRef RenderTargetTextureRHI, TMap<int32, FTmvMediaFrameMipBufferHandle>& MipBuffers, const FTmvMediaTextureSampleConverterParameters& ConverterParams, FTmvMediaTextureSampleConverter::FTileDescGpuCache& TileDescGpuCache, int32 ResidentFloor)
	{
		RHI_BREADCRUMB_EVENT_STAT_F(RHICmdList, TmvMediaConverter, "TmvMediaConverter.Convert", "TmvMediaConverter.Convert %d", ConverterParams.FrameId);
		int32 MipToUpscale = ConverterParams.UpscaleMip;

		// Upscale to all mips below the mip to upscale.
		if (MipBuffers.Contains(MipToUpscale))
		{
			for (int32 MipLevel = 0; MipLevel <= MipToUpscale; MipLevel++)
			{
				// Defense-in-depth: never write to a mip the sparse-output manager
				// hasn't backed with physical memory. Under normal operation the
				// manager commits everything in [TargetFloor, MaxSparse] before
				// this swizzler runs, but if a commit failed (budget exhaustion)
				// we'd otherwise write to unmapped tiles and trigger undefined
				// behavior. The cost of skipping is a temporarily missing fine
				// mip; the UpscaleMip path already covers reading it back via a
				// coarser source.
				if (MipLevel < ResidentFloor)
				{
					continue;
				}

				FTmvMediaFrameMipBufferHandle BufferDataToUpscale = MipBuffers[MipToUpscale];
				// Render gate: don't upscale from a source mip whose upload for this
				// frame hasn't landed yet - keep last frame's output instead of
				// blowing up not-yet-uploaded (green) memory across the lower mips.
				if (GTmvConverterSkipMipUntilUploaded != 0
					&& BufferDataToUpscale.IsValid()
					&& BufferDataToUpscale->IsUploadPending())
				{
					continue;
				}

				if (BufferDataToUpscale.IsValid())
				{
					RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TmvMediaConverter_MipUpscale, "TmvMediaConverter.MipRender.MipUpscale");

					TArray<FIntRect> FakeViewport;
					const int MipLevelDiv = 1 << MipLevel;
					const FIntPoint Dim = ConverterParams.FullResolution / MipLevelDiv;
					FakeViewport.Add(FIntRect(FIntPoint(0, 0), Dim));
					RenderMip(RHICmdList, RenderTargetTextureRHI, ConverterParams, MipToUpscale, MipLevel, BufferDataToUpscale, (ConverterParams.FullResolution / (1 << MipToUpscale)), Dim, FakeViewport, TileDescGpuCache);
				}
			}
		}
		else if (MipToUpscale != INDEX_NONE)
		{
			UE_LOGF(LogTmvMedia, Warning, "Requested mip could not be found %d", MipToUpscale);
		}

		for (const TPair<int32, TArray<FIntRect>>& MipLevelViewports : ConverterParams.Viewports)
		{
			int32 MipLevel = MipLevelViewports.Key;

			// Sanity check.
			if (!MipBuffers.Contains(MipLevel))
			{
				continue;
			}

			// Same defense-in-depth clamp as the upscale loop above.
			if (MipLevel < ResidentFloor)
			{
				continue;
			}

			FTmvMediaFrameMipBufferHandle BufferData = MipBuffers[MipLevel];
			int MipLevelDiv = 1 << MipLevel;
			FIntPoint Dim = ConverterParams.FullResolution / MipLevelDiv;

			if (BufferData.IsValid() && BufferData->IsValidForRendering())
			{
				// Skip the mip to upscale because it is read and rendered already.
				if (MipLevelViewports.Key == MipToUpscale)
				{
					continue;
				}

				const bool bMipPending = GTmvConverterSkipMipUntilUploaded != 0
					&& BufferData->IsUploadPending();

				// Coarse-current fallback (r.TmvMedia.CoarseCurrentFallback).
				//   mode 1 -> fill gated mips only.
				//   mode 2 -> also fill UNDER-DECODED mips: those whose decoded area
				//             (Viewports) is below CoverageThreshold% of their intended
				//             area (FullVisibleViewports). Fully-decoded mips skip the
				//             fallback entirely, so good frames pay nothing.
				bool bUnderDecoded = false;
				if (GTmvConverterCoarseCurrentFallback >= 2 && !bMipPending)
				{
					const TArray<FIntRect>* IntendedVp = ConverterParams.FullVisibleViewports.Find(MipLevel);
					if (IntendedVp)
					{
						int64 IntendedArea = 0;
						for (const FIntRect& R : *IntendedVp)
						{
							IntendedArea += static_cast<int64>(FMath::Max(0, R.Width())) * FMath::Max(0, R.Height());
						}
						int64 DecodedArea = 0;
						for (const FIntRect& R : MipLevelViewports.Value)
						{
							DecodedArea += static_cast<int64>(FMath::Max(0, R.Width())) * FMath::Max(0, R.Height());
						}
						bUnderDecoded = IntendedArea > 0
							&& DecodedArea * 100 < IntendedArea * static_cast<int64>(GTmvConverterCoarseFallbackCoverageThreshold);
					}
				}

				const bool bWantCoarseBase =
					(GTmvConverterCoarseCurrentFallback == 1 && bMipPending)
					|| (GTmvConverterCoarseCurrentFallback >= 2 && (bMipPending || bUnderDecoded));

				int32 NearestReadyCoarser = INDEX_NONE;
				if (bWantCoarseBase || bMipPending)
				{
					for (const TPair<int32, FTmvMediaFrameMipBufferHandle>& Pair : MipBuffers)
					{
						if (Pair.Key > MipLevel
							&& Pair.Value.IsValid() && Pair.Value->IsValidForRendering()
							&& !(GTmvConverterSkipMipUntilUploaded != 0 && Pair.Value->IsUploadPending())
							&& (NearestReadyCoarser == INDEX_NONE || Pair.Key < NearestReadyCoarser))
						{
							NearestReadyCoarser = Pair.Key;
						}
					}
				}

				// Fill mip M's intended region with the nearest coarser ready mip,
				// upscaled, BEFORE mip M's own (possibly partial) tiles overlay it. We
				// draw over the intended region (not the whole output mip) so the cost
				// scales with the mip's actual footprint, and only when the mip is
				// gated or under-decoded - so a fully decoded frame draws no base.
				if (bWantCoarseBase && NearestReadyCoarser != INDEX_NONE)
				{
					const TArray<FIntRect>* IntendedVp = ConverterParams.FullVisibleViewports.Find(MipLevel);
					TArray<FIntRect> FallbackViewport;
					if (IntendedVp && IntendedVp->Num() > 0)
					{
						FallbackViewport = *IntendedVp;
					}
					else
					{
						FallbackViewport.Add(FIntRect(FIntPoint(0, 0), Dim));
					}
					RenderMip(RHICmdList, RenderTargetTextureRHI, ConverterParams,
						NearestReadyCoarser, MipLevel, MipBuffers[NearestReadyCoarser],
						(ConverterParams.FullResolution / (1 << NearestReadyCoarser)).ComponentMax(FIntPoint(1, 1)),
						Dim, FallbackViewport, TileDescGpuCache);
				}

				// Render gate: if this frame's upload for this mip hasn't been
				// recorded on the render thread yet, sampling the buffer now would
				// read not-yet-uploaded (green) memory. Skip the sharp draw; the
				// coarse-current base above (when enabled) keeps the region current,
				// otherwise the never-cleared RT keeps last frame's output.
				if (bMipPending)
				{
					continue;
				}

				RenderMip(RHICmdList, RenderTargetTextureRHI, ConverterParams, MipLevel, MipLevel, BufferData, Dim, Dim, MipLevelViewports.Value, TileDescGpuCache);
			}
		}

		// Adjacent-mip prefill (r.TmvMedia.PrefillAdjacentMips): dilate the populated
		// mip range by one on each side so hardware filtering into a neighbor mip
		// during a visibility/LOD transition reads coherent data instead of a stale or
		// not-yet-decoded mip. The finer neighbor is upscaled from the finest ready
		// visible mip; the coarser neighbor downscaled from the coarsest ready visible
		// mip; both over the visible footprint scaled to the target resolution.
		if (GTmvConverterPrefillAdjacentMips != 0 && ConverterParams.Viewports.Num() > 0)
		{
			// A visible mip is usable as a prefill source only once its own upload has
			// landed (same gate the sharp draw uses), else we'd propagate green memory.
			auto IsReadySource = [&MipBuffers](int32 InMipLevel) -> bool
			{
				const FTmvMediaFrameMipBufferHandle* Buf = MipBuffers.Find(InMipLevel);
				return Buf && Buf->IsValid() && (*Buf)->IsValidForRendering()
					&& !(GTmvConverterSkipMipUntilUploaded != 0 && (*Buf)->IsUploadPending());
			};

			// Dest footprint for a prefill: the source mip's intended visible region
			// scaled to the target resolution (bounded to the on-screen plate area).
			// Falls back to the full target mip if the source footprint is unknown.
			auto BuildFootprint = [&ConverterParams](int32 InSourceMip, int32 InTargetMip, const FIntPoint& InTargetDim) -> TArray<FIntRect>
			{
				TArray<FIntRect> Out;
				if (const TArray<FIntRect>* SrcVp = ConverterParams.FullVisibleViewports.Find(InSourceMip))
				{
					const int32 Shift = InSourceMip - InTargetMip; // >0 finer target (upscale), <0 coarser (downscale)
					Out.Reserve(SrcVp->Num());
					for (const FIntRect& R : *SrcVp)
					{
						FIntRect S = R;
						if (Shift > 0)
						{
							S.Min.X <<= Shift; S.Min.Y <<= Shift;
							S.Max.X <<= Shift; S.Max.Y <<= Shift;
						}
						else if (Shift < 0)
						{
							const int32 Down = -Shift;
							S.Min.X >>= Down; S.Min.Y >>= Down;
							S.Max.X >>= Down; S.Max.Y >>= Down;
						}
						S.Min = S.Min.ComponentMax(FIntPoint(0, 0));
						S.Max = S.Max.ComponentMin(InTargetDim);
						if (S.Max.X > S.Min.X && S.Max.Y > S.Min.Y)
						{
							Out.Add(S);
						}
					}
				}
				if (Out.Num() == 0)
				{
					Out.Add(FIntRect(FIntPoint(0, 0), InTargetDim));
				}
				return Out;
			};

			// Viewports is a TSortedMap (ascending): first key is the finest visible
			// mip, last key the coarsest.
			const int32 MinVisible = ConverterParams.Viewports.CreateConstIterator().Key();
			int32 MaxVisible = MinVisible;
			for (const TPair<int32, TArray<FIntRect>>& Vp : ConverterParams.Viewports)
			{
				MaxVisible = Vp.Key;
			}

			// The output texture's mip chain can be shorter than the full resolution
			// implies, so a neighbor target must be clamped to the texture's actual
			// mip count - otherwise RenderMip binds a non-existent mip subresource
			// (null RTV) and FD3D12CommandContext::SetRenderTargets ensures.
			const int32 OutputNumMips = RenderTargetTextureRHI.IsValid() ? RenderTargetTextureRHI->GetDesc().NumMips : 0;

			// Finer neighbor: upscale from the finest visible mip into MinVisible-1.
			// Skip if below the sparse resident floor (its tiles aren't backed) or if
			// the existing UpscaleMip path already filled it.
			const int32 FinerTarget = MinVisible - 1;
			if (FinerTarget >= 0 && FinerTarget >= ResidentFloor && FinerTarget > MipToUpscale && IsReadySource(MinVisible))
			{
				const FIntPoint SourceDim = (ConverterParams.FullResolution / (1 << MinVisible)).ComponentMax(FIntPoint(1, 1));
				const FIntPoint TargetDim = (ConverterParams.FullResolution / (1 << FinerTarget)).ComponentMax(FIntPoint(1, 1));
				RenderMip(RHICmdList, RenderTargetTextureRHI, ConverterParams, MinVisible, FinerTarget,
					MipBuffers[MinVisible], SourceDim, TargetDim, BuildFootprint(MinVisible, FinerTarget, TargetDim), TileDescGpuCache);
			}

			// Coarser neighbor: downscale from the coarsest visible mip into MaxVisible+1
			// (only while the target mip still has a non-degenerate resolution).
			const int32 CoarserTarget = MaxVisible + 1;
			const FIntPoint CoarserTargetDim = ConverterParams.FullResolution / (1 << CoarserTarget);
			if (CoarserTarget < OutputNumMips && CoarserTargetDim.X >= 1 && CoarserTargetDim.Y >= 1 && IsReadySource(MaxVisible))
			{
				const FIntPoint SourceDim = (ConverterParams.FullResolution / (1 << MaxVisible)).ComponentMax(FIntPoint(1, 1));
				RenderMip(RHICmdList, RenderTargetTextureRHI, ConverterParams, MaxVisible, CoarserTarget,
					MipBuffers[MaxVisible], SourceDim, CoarserTargetDim, BuildFootprint(MaxVisible, CoarserTarget, CoarserTargetDim), TileDescGpuCache);
			}
		}

		// Doesn't need further conversion so returning false.
		return false;
	}
}

FTmvMediaTextureSampleConverter::FTmvMediaTextureSampleConverter()
	: TileDescGpuCache(MakeUnique<FTileDescGpuCache>())
{
}

FTmvMediaTextureSampleConverter::~FTmvMediaTextureSampleConverter() = default;

bool FTmvMediaTextureSampleConverter::Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
{
	const FTmvMediaTextureSampleConverterParameters ConverterParamsCopy = GetParams_ThreadSafe();

	// Sparse-output residency. When the intermediate target was created with
	// ReservedResource (decided per-sample by GetIntermediateTargetCreateFlagsHint
	// against UE::TmvMedia::IsSparseOutputMipEnabled + plate size threshold),
	// the destination texture has virtual address space but no physical
	// backing. Register it with the singleton manager (idempotent) and
	// request the floor mip the converter is about to write this frame -
	// the minimum key in Viewports is the finest mip the decoder fed us.
	// UpdateTileMappings runs on the graphics queue and is ordered before
	// ExecuteCommandLists for the converter's render passes below, so the
	// commit is live by the time RenderMip executes on the GPU.
	//
	// ResidentFloor is published by the manager after the commit pass; we
	// hand it to the swizzler so any below-floor render passes are skipped
	// (defense-in-depth backstop for budget-exhausted commit failures).
	int32 ResidentFloor = 0;
	if (InDstTexture.IsValid()
		&& UE::TmvMedia::IsSparseOutputMipEnabled()
		&& EnumHasAnyFlags(InDstTexture->GetDesc().Flags, ETextureCreateFlags::ReservedResource))
	{
		FTmvSparseOutputManager& Manager = FTmvSparseOutputManager::Get();
		if (Manager.IsEnabled())
		{
			const FRHITextureDesc& Desc = InDstTexture->GetDesc();
			FTmvSparseOutputLayout Layout;
			Layout.Dim = Desc.Extent;
			Layout.Format = Desc.Format;
			Layout.NumMips = Desc.NumMips;
			Layout.BytesPerPixel = static_cast<int32>(GPixelFormats[Desc.Format].BlockBytes);
			Layout.MinSparseBytes = static_cast<int64>(UE::TmvMedia::GetSparseOutputMipMinBytesMB()) * 1024ll * 1024ll;

			if (Manager.RegisterOutput(InDstTexture.GetReference(), Layout))
			{
				int32 TargetFloor = Layout.NumMips - 1;
				if (ConverterParamsCopy.Viewports.Num() > 0)
				{
					// TSortedMap iterates ascending; first key is the smallest (finest) mip.
					TargetFloor = ConverterParamsCopy.Viewports.CreateConstIterator().Key();
				}

				FTmvSparseOutputFrameRequest Request;
				Request.Texture = InDstTexture.GetReference();
				Request.TargetFloor = TargetFloor;
				Manager.RequestFloor(Request);

				ResidentFloor = Manager.GetResidentFloor(InDstTexture.GetReference());
			}
		}
	}

	//smell: lock MipBufferCriticalSection to protect MipBuffers or make a copy?
	return UE::TmvMedia::Converter::RenderThreadSwizzler(RHICmdList, InDstTexture, MipBuffers, ConverterParamsCopy, *TileDescGpuCache, ResidentFloor);
}

bool FTmvMediaTextureSampleConverter::HasMipLevelBuffer(int32 RequestedMipLevel) const
{
	FScopeLock Lock(&MipBufferCriticalSection);
	return MipBuffers.Contains(RequestedMipLevel);
}

void FTmvMediaTextureSampleConverter::ClearInactiveOwnerVisibility(const TSet<int32>& InActiveMipLevels)
{
	// Snapshot the (MipLevel -> BufferHandle) entries under the map lock, then
	// release the lock before touching the pool (SetOwnerVisibility takes the
	// pool's own critical section, and nesting both could create a lock-order
	// hazard with concurrent allocator threads).
	TArray<TPair<int32, FTmvMediaFrameMipBufferHandle>, TInlineAllocator<8>> Snapshot;
	{
		FScopeLock Lock(&MipBufferCriticalSection);
		Snapshot.Reserve(MipBuffers.Num());
		for (const TPair<int32, FTmvMediaFrameMipBufferHandle>& Pair : MipBuffers)
		{
			if (!InActiveMipLevels.Contains(Pair.Key))
			{
				Snapshot.Emplace(Pair.Key, Pair.Value);
			}
		}
	}
	const TSet<FIntPoint> EmptyVisible;
	for (const TPair<int32, FTmvMediaFrameMipBufferHandle>& Pair : Snapshot)
	{
		const FTmvMediaFrameMipBufferHandle& Buffer = Pair.Value;
		FTmvMediaTileResidency* Residency = Buffer.IsValid() ? Buffer->GetResidency() : nullptr;
		if (Residency && Residency->IsShared() && Residency->SharedPool.IsValid())
		{
			Residency->SharedPool->SetOwnerVisibility(Residency->SharedOwner, EmptyVisible);
		}
	}
}

FTmvMediaFrameMipBufferHandle FTmvMediaTextureSampleConverter::GetOrCreateMipLevelBuffer(int32 RequestedMipLevel, TFunction<FTmvMediaFrameMipBufferHandle()> AllocatorFunc)
{
	FTmvMediaFrameMipBufferHandle Result;

	{
		FScopeLock Lock(&MipBufferCriticalSection);

		if (FTmvMediaFrameMipBufferHandle* BufferDataPtr = MipBuffers.Find(RequestedMipLevel))
		{
			Result = *BufferDataPtr;
		}
		else
		{
			Result = MipBuffers.Add(RequestedMipLevel, AllocatorFunc());
		}
	}
		
	// Wait for render thread buffer allocations before using resources
	Result->WaitAllocation();

	return Result;
}