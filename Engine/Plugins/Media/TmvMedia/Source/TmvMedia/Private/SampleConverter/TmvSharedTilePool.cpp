// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleConverter/TmvSharedTilePool.h"

#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RHICommandList.h"
#include "RHIGlobals.h"
#include "RenderingThread.h"
#include "TmvMediaLog.h"

DECLARE_GPU_STAT_NAMED(TmvSharedTilePool_AllocateBuffer, TEXT("TmvSharedTilePool.AllocateBuffer"));

FTmvSharedTilePool::FKey FTmvSharedTilePool::MakeKey(const FTmvMediaFrameMipInfo& InMipInfo)
{
	// Pool keying is (per-tile bytes, plane count) only - no per-mip layout
	// dimensions. Different mips of the same APV format share the pool. See
	// FKey's doc comment.
	FKey K;
	K.PerTileBytes = static_cast<int32>(InMipInfo.GetAllPlaneMemorySizeInBytes());
	K.NumPlanes    = InMipInfo.Planes.Num();
	return K;
}

bool FTmvSharedTilePool::MatchesMipInfo(const FTmvMediaFrameMipInfo& InMipInfo) const
{
	return MakeKey(InMipInfo) == Key;
}

FTmvSharedTilePool::FTmvSharedTilePool(const FKey& InKey, int32 InInitialCapacity)
	: Key(InKey)
{
	Capacity = FMath::Max(0, InInitialCapacity);
	Slots.SetNum(Capacity);
	AllocatedBufferSize = static_cast<SIZE_T>(Capacity) * static_cast<SIZE_T>(Key.PerTileBytes);

	// All slots start in the Free list (no owner, no tile, no visibility).
	// Insert at head - order doesn't matter for free list, just need the chain.
	for (int32 i = 0; i < Capacity; ++i)
	{
		ListInsertAtHead_Free(i);
	}

	// Manual-reset event: stays signaled forever after the first trigger so
	// every caller of WaitForReady (potentially many decoder threads across
	// many owners) sees the ready state without per-caller re-arming.
	GpuReadyEvent = FPlatformProcess::GetSynchEventFromPool(/*bIsManualReset=*/true);

	// Auto-reset for grow completion. Each grow fires Trigger() once per
	// blocked waiter (tracked via GrowWaiters). See WaitForGrowComplete.
	GrowDoneEvent = FPlatformProcess::GetSynchEventFromPool(/*bIsManualReset=*/false);

	UE_LOGF(LogTmvMedia, Verbose,
		"FTmvSharedTilePool: created key{bytes=%d planes=%d} capacity=%d (%.2f MB) [async alloc pending]",
		Key.PerTileBytes, Key.NumPlanes, Capacity,
		double(AllocatedBufferSize) / (1024.0 * 1024.0));

	// NOTE: GPU allocation can't start here - AsWeak() (used inside
	// EnqueueInitialAllocation_GameOrDecodeThread to capture a TWeakPtr in
	// the render command) asserts that the shared instance exists, which it
	// doesn't yet (we're mid-construction). The manager calls
	// BeginInitialAllocation() immediately after MakeShared returns.
}

FTmvSharedTilePool::~FTmvSharedTilePool()
{
	if (UploadBufferRef && UploadBufferMapped)
	{
		// At engine shutdown the RHI is torn down before module statics
		// (and the manager's Pools map is one such static), so a render-
		// command enqueue here would dereference a null GDynamicRHI. The
		// process is exiting anyway; the buffer's GPU memory is reclaimed
		// by the OS, so just drop the references without unlocking.
		if (GIsRHIInitialized)
		{
			FBufferRHIRef ToUnlock = UploadBufferRef;
			ENQUEUE_RENDER_COMMAND(UnlockOnDestroy)([ToUnlock](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.UnlockBuffer(ToUnlock);
			});
		}
		UploadBufferMapped = nullptr;
	}

	if (GpuReadyEvent)
	{
		// Trigger before return so any thread blocked in WaitForReady wakes
		// up rather than hanging forever on a pool we're destroying.
		GpuReadyEvent->Trigger();
		FPlatformProcess::ReturnSynchEventToPool(GpuReadyEvent);
		GpuReadyEvent = nullptr;
	}

	if (GrowDoneEvent)
	{
		// Same shutdown courtesy as GpuReadyEvent: wake any parked waiters
		// before we drop the event. Trigger once per known waiter (we don't
		// know the exact count post-destruction race window, but spamming
		// triggers on an auto-reset event is harmless once no waiters remain).
		const int32 WaiterCount = GrowWaiters.load(std::memory_order_relaxed);
		for (int32 i = 0; i < WaiterCount + 1; ++i)
		{
			GrowDoneEvent->Trigger();
		}
		FPlatformProcess::ReturnSynchEventToPool(GrowDoneEvent);
		GrowDoneEvent = nullptr;
	}
}

// =====================================================================
// Two-LRU list helpers
// =====================================================================
// All callers must hold Cs. Lists are intrusive into Slots[] via the
// PrevInList / NextInList fields. Head = MRU end (recently touched);
// Tail = LRU end (evicted first). The Free list doesn't need a tail.

void FTmvSharedTilePool::ListRemove(int32 SlotIdx)
{
	FSlot& S = Slots[SlotIdx];
	const int32 Prev = S.PrevInList;
	const int32 Next = S.NextInList;
	if (Prev != INDEX_NONE) { Slots[Prev].NextInList = Next; }
	if (Next != INDEX_NONE) { Slots[Next].PrevInList = Prev; }

	// Patch the relevant head/tail pointers.
	switch (S.List)
	{
	case ESlotList::Free:
		if (FreeHead == SlotIdx) { FreeHead = Next; }
		break;
	case ESlotList::VisibleLRU:
		if (VisibleLruHead == SlotIdx) { VisibleLruHead = Next; }
		if (VisibleLruTail == SlotIdx) { VisibleLruTail = Prev; }
		break;
	case ESlotList::NonVisibleLRU:
		if (NonVisibleLruHead == SlotIdx) { NonVisibleLruHead = Next; }
		if (NonVisibleLruTail == SlotIdx) { NonVisibleLruTail = Prev; }
		break;
	}

	S.PrevInList = INDEX_NONE;
	S.NextInList = INDEX_NONE;
}

void FTmvSharedTilePool::ListInsertAtHead_Free(int32 SlotIdx)
{
	FSlot& S = Slots[SlotIdx];
	S.PrevInList = INDEX_NONE;
	S.NextInList = FreeHead;
	if (FreeHead != INDEX_NONE) { Slots[FreeHead].PrevInList = SlotIdx; }
	FreeHead = SlotIdx;
	S.List = ESlotList::Free;
}

void FTmvSharedTilePool::ListInsertAtHead_VisibleLRU(int32 SlotIdx)
{
	FSlot& S = Slots[SlotIdx];
	S.PrevInList = INDEX_NONE;
	S.NextInList = VisibleLruHead;
	if (VisibleLruHead != INDEX_NONE) { Slots[VisibleLruHead].PrevInList = SlotIdx; }
	VisibleLruHead = SlotIdx;
	if (VisibleLruTail == INDEX_NONE) { VisibleLruTail = SlotIdx; }
	S.List = ESlotList::VisibleLRU;
}

void FTmvSharedTilePool::ListInsertAtHead_NonVisibleLRU(int32 SlotIdx)
{
	FSlot& S = Slots[SlotIdx];
	S.PrevInList = INDEX_NONE;
	S.NextInList = NonVisibleLruHead;
	if (NonVisibleLruHead != INDEX_NONE) { Slots[NonVisibleLruHead].PrevInList = SlotIdx; }
	NonVisibleLruHead = SlotIdx;
	if (NonVisibleLruTail == INDEX_NONE) { NonVisibleLruTail = SlotIdx; }
	S.List = ESlotList::NonVisibleLRU;
}

void FTmvSharedTilePool::MoveToList(int32 SlotIdx, ESlotList NewList)
{
	if (Slots[SlotIdx].List == NewList)
	{
		return; // Already there.
	}
	ListRemove(SlotIdx);
	switch (NewList)
	{
	case ESlotList::Free:           ListInsertAtHead_Free(SlotIdx);          break;
	case ESlotList::VisibleLRU:     ListInsertAtHead_VisibleLRU(SlotIdx);    break;
	case ESlotList::NonVisibleLRU:  ListInsertAtHead_NonVisibleLRU(SlotIdx); break;
	}
}

int32 FTmvSharedTilePool::PickFreeOrEvictableSlot()
{
	// Free list first - no eviction work needed.
	if (FreeHead != INDEX_NONE)
	{
		return FreeHead;
	}
	// Otherwise the LRU end of the non-visible list. If empty, every slot is
	// pinned to some owner's visible set -> caller must grow the pool.
	return NonVisibleLruTail;
}

void FTmvSharedTilePool::WaitForGrowComplete()
{
	// Snapshot of bGrowing: if no grow is in flight, return immediately.
	if (!bGrowing.load(std::memory_order_acquire))
	{
		return;
	}
	if (!GrowDoneEvent)
	{
		return; // pool teardown
	}

	// Register as a waiter, then re-check bGrowing. The increment and this load
	// MUST be seq_cst, and the grow-completion paths' matching
	// bGrowing.store(false) / GrowWaiters.load() are seq_cst too. That single
	// total order rules out a lost-wakeup deadlock: if this load still observes
	// bGrowing==true, then in the total order our increment precedes the
	// completion's GrowWaiters read, so the completion is guaranteed to Trigger
	// for us. Under weaker acquire/release a StoreLoad reorder could let the
	// completion read GrowWaiters (missing us) before our increment is visible
	// while we read a stale bGrowing==true - parking us with no Trigger coming.
	GrowWaiters.fetch_add(1, std::memory_order_seq_cst);
	const double WaitStartT = FPlatformTime::Seconds();

	// Loop: a stale auto-reset signal from a prior grow can wake us early, and
	// the next grow's waiters must not be released by the previous grow.
	while (bGrowing.load(std::memory_order_seq_cst))
	{
		GrowDoneEvent->Wait();
	}

	GrowWaiters.fetch_sub(1, std::memory_order_acq_rel);
	const double WaitMs = (FPlatformTime::Seconds() - WaitStartT) * 1000.0;
	if (WaitMs > 50.0)
	{
		UE_LOGF(LogTmvMedia, Display,
			"FTmvSharedTilePool.WaitForGrowComplete: %.0f ms", WaitMs);
	}
}

void FTmvSharedTilePool::WaitForReady()
{
	if (bGpuReady.load(std::memory_order_relaxed))
	{
		return;
	}
	if (GpuReadyEvent)
	{
		const double WaitStartT = FPlatformTime::Seconds();
		UE_LOGF(LogTmvMedia, Verbose,
			"FTmvSharedTilePool.WaitForReady: blocking (key{bytes=%d planes=%d} cap=%d)",
			Key.PerTileBytes, Key.NumPlanes, Capacity);
		GpuReadyEvent->Wait();
		const double WaitMs = (FPlatformTime::Seconds() - WaitStartT) * 1000.0;
		UE_LOGF(LogTmvMedia, Verbose,
			"FTmvSharedTilePool.WaitForReady: unblocked after %.0f ms", WaitMs);
	}
}

void FTmvSharedTilePool::EnqueueInitialAllocation_GameOrDecodeThread(int32 InCapacity, SIZE_T InAllocSize)
{
	if (InAllocSize == 0)
	{
		bGpuReady.store(false, std::memory_order_relaxed);
		return;
	}
	if (InAllocSize > MAX_uint32)
	{
		UE_LOGF(LogTmvMedia, Error,
			"FTmvSharedTilePool: requested buffer size %llu exceeds 4 GB RHI limit; pool will stay unready.",
			static_cast<uint64>(InAllocSize));
		bGpuReady.store(false, std::memory_order_relaxed);
		return;
	}

	const double EnqueueT = FPlatformTime::Seconds();
	TWeakPtr<FTmvSharedTilePool, ESPMode::ThreadSafe> WeakSelf = AsWeak();
	ENQUEUE_RENDER_COMMAND(AllocSharedPoolAsync)(
		[WeakSelf, InAllocSize, EnqueueT](FRHICommandListImmediate& RHICmdList)
		{
			TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe> Self = WeakSelf.Pin();
			if (!Self.IsValid())
			{
				return; // Pool destructed before allocation ran.
			}

			const double RtStartT = FPlatformTime::Seconds();
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TmvSharedTilePool_AllocateBuffer, "TmvSharedTilePool.AllocateAsync");
			constexpr uint32 Stride = sizeof(uint16) * 2;

			const double CreateUploadT = FPlatformTime::Seconds();
			const FRHIBufferCreateDesc UploadDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("TmvSharedTilePool.UploadBuffer"), InAllocSize, Stride)
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic | EBufferUsageFlags::FastVRAM)
				.DetermineInitialState();
			FBufferRHIRef LocalUpload = RHICmdList.CreateBuffer(UploadDesc);
			void* LocalMapped = RHICmdList.LockBuffer(LocalUpload, 0, InAllocSize, RLM_WriteOnly);
			RHICmdList.Transition(FRHITransitionInfo(LocalUpload, UploadDesc.InitialState, ERHIAccess::CopySrc));

			const double CreateDestT = FPlatformTime::Seconds();
			const FRHIBufferCreateDesc DestDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("TmvSharedTilePool.DestBuffer"), InAllocSize, Stride)
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::FastVRAM)
				.DetermineInitialState();
			FBufferRHIRef LocalShaderAccess = RHICmdList.CreateBuffer(DestDesc);
			FShaderResourceViewRHIRef LocalSRV = RHICmdList.CreateShaderResourceView(LocalShaderAccess, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(LocalShaderAccess));
			const double PublishT = FPlatformTime::Seconds();

			// Publish RHI refs under the write lock so any decoder that races
			// in with a read lock either sees null (pre-publish) or a fully
			// consistent set (post-publish).
			{
				FRWScopeLock Scope(Self->GrowLock, SLT_Write);
				Self->UploadBufferRef       = LocalUpload;
				Self->UploadBufferMapped    = LocalMapped;
				Self->ShaderAccessBufferRef = LocalShaderAccess;
				Self->ShaderResourceView    = LocalSRV;
			}
			Self->bGpuReady.store(true, std::memory_order_seq_cst);
			// Wake every thread blocked in WaitForReady. Must come AFTER the
			// atomic flip so a waker that loses the race to check IsReady()
			// and falls through to Wait() still sees the trigger.
			if (Self->GpuReadyEvent)
			{
				Self->GpuReadyEvent->Trigger();
			}

			const double DoneT = FPlatformTime::Seconds();
			UE_LOGF(LogTmvMedia, Verbose,
				"FTmvSharedTilePool.AllocAsync: complete (%.2f MB). breakdown ms: enq->rt=%.0f, createUpload+lock=%.0f, createDest+SRV=%.0f, publish=%.0f, TOTAL=%.0f",
				double(InAllocSize) / (1024.0 * 1024.0),
				(RtStartT - EnqueueT) * 1000.0,
				(CreateDestT - CreateUploadT) * 1000.0,
				(PublishT - CreateDestT) * 1000.0,
				(DoneT - PublishT) * 1000.0,
				(DoneT - EnqueueT) * 1000.0);
		});
}

FTmvSharedTilePool::FOwnerHandle FTmvSharedTilePool::RegisterOwner()
{
	FScopeLock Lock(&Cs);
	const int32 Id = NextOwnerId++;
	Owners.Add(Id, FOwnerInfo{});
	return FOwnerHandle{ Id };
}

void FTmvSharedTilePool::UnregisterOwner(FOwnerHandle InHandle)
{
	if (!InHandle.IsValid())
	{
		return;
	}
	FScopeLock Lock(&Cs);
	const int32 OwnerId = InHandle.OwnerId;

	// Iterate this owner's slots from its SlotIndices list (O(N_owned)) and
	// release them back to the Free list. Avoid scanning the entire pool.
	if (FOwnerInfo* OI = Owners.Find(OwnerId))
	{
		for (int32 SlotIdx : OI->SlotIndices)
		{
			FSlot& S = Slots[SlotIdx];
			OwnerKeyToSlot.Remove(TPair<int32, FIntPoint>(OwnerId, S.Tile));
			S.OwnerId = INDEX_NONE;
			S.Tile = FIntPoint(INT32_MIN, INT32_MIN);
			MoveToList(SlotIdx, ESlotList::Free);
		}
		OI->SlotIndices.Reset();
	}
	Owners.Remove(OwnerId);
}

int32 FTmvSharedTilePool::GetOwnerCount() const
{
	FScopeLock Lock(&Cs);
	return Owners.Num();
}

int32 FTmvSharedTilePool::GetSlot(FOwnerHandle InOwner, FIntPoint InTile) const
{
	if (!InOwner.IsValid())
	{
		return INDEX_NONE;
	}
	FScopeLock Lock(&Cs);
	if (const int32* Found = OwnerKeyToSlot.Find(TPair<int32, FIntPoint>(InOwner.OwnerId, InTile)))
	{
		return *Found;
	}
	return INDEX_NONE;
}

int32 FTmvSharedTilePool::AllocateSlot(FOwnerHandle InOwner, FIntPoint InTile)
{
	if (!InOwner.IsValid())
	{
		return INDEX_NONE;
	}
	// Pool not ready yet (initial async alloc in flight) -> caller falls back.
	if (!bGpuReady.load(std::memory_order_relaxed))
	{
		return INDEX_NONE;
	}
	FScopeLock Lock(&Cs);

	// Already resident: touch (move to MRU of VisibleLRU since the tile is
	// being requested again - by contract callers pass tiles in their
	// VisibleTiles set, so this slot is visible) and return.
	if (int32* Existing = OwnerKeyToSlot.Find(TPair<int32, FIntPoint>(InOwner.OwnerId, InTile)))
	{
		MoveToList(*Existing, ESlotList::VisibleLRU);
		return *Existing;
	}

	// Pick a slot: free first, else LRU end of non-visible. If neither
	// exists, every slot is pinned to some owner's visible set -> caller
	// must grow.
	const int32 PickedSlot = PickFreeOrEvictableSlot();
	if (PickedSlot == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	// Evict previous occupant if any.
	FSlot& Picked = Slots[PickedSlot];
	if (Picked.OwnerId != INDEX_NONE)
	{
		OwnerKeyToSlot.Remove(TPair<int32, FIntPoint>(Picked.OwnerId, Picked.Tile));
		if (FOwnerInfo* PrevOwner = Owners.Find(Picked.OwnerId))
		{
			PrevOwner->SlotIndices.RemoveSingleSwap(PickedSlot);
		}
	}

	// Assign to (InOwner, InTile) and place at MRU of VisibleLRU.
	Picked.OwnerId = InOwner.OwnerId;
	Picked.Tile    = InTile;
	MoveToList(PickedSlot, ESlotList::VisibleLRU);
	OwnerKeyToSlot.Add(TPair<int32, FIntPoint>(InOwner.OwnerId, InTile), PickedSlot);
	if (FOwnerInfo* NewOwner = Owners.Find(InOwner.OwnerId))
	{
		NewOwner->SlotIndices.Add(PickedSlot);
	}
	return PickedSlot;
}

void FTmvSharedTilePool::SetOwnerVisibility(FOwnerHandle InOwner, const TSet<FIntPoint>& InVisible)
{
	if (!InOwner.IsValid())
	{
		return;
	}
	FScopeLock Lock(&Cs);
	FOwnerInfo* OI = Owners.Find(InOwner.OwnerId);
	if (!OI)
	{
		return;
	}
	OI->VisibleTiles = InVisible;

	// Re-classify this owner's slots between VisibleLRU and NonVisibleLRU
	// based on the new visible set. Walk only owner's slots (O(N_owned)),
	// not the whole pool. Visible tiles -> MoveToList(VisibleLRU); the rest
	// -> MoveToList(NonVisibleLRU). The eviction path now finds tail of
	// NonVisibleLRU in O(1) instead of linear-scanning the pool.
	for (const int32 SlotIdx : OI->SlotIndices)
	{
		const FSlot& S = Slots[SlotIdx];
		const bool bShouldBeVisible = InVisible.Contains(S.Tile);
		MoveToList(SlotIdx, bShouldBeVisible ? ESlotList::VisibleLRU : ESlotList::NonVisibleLRU);
	}
}

int32 FTmvSharedTilePool::GetTotalVisibleTileCount() const
{
	FScopeLock Lock(&Cs);
	int32 Total = 0;
	for (const TPair<int32, FOwnerInfo>& Pair : Owners)
	{
		Total += Pair.Value.VisibleTiles.Num();
	}
	return Total;
}

bool FTmvSharedTilePool::EnsureCapacityFor(int32 InMinCapacity)
{
	// Quick capacity check. Capacity is also written by the async grow on the render
	// thread (under Cs - see GrowSharedPoolAsync), so this is a deliberately lock-free
	// read of an aligned int32: a stale value at worst triggers a redundant grow
	// attempt, which the bGrowing CAS below collapses.
	if (Capacity >= InMinCapacity)
	{
		return true;
	}

	// Suppress duplicate triggers: if grow is already in flight, the next
	// AllocateSlot after it completes will see the larger pool. Use a CAS
	// to claim the in-flight slot; if someone else already claimed it, bail.
	bool bExpected = false;
	if (!bGrowing.compare_exchange_strong(bExpected, true, std::memory_order_acq_rel))
	{
		return true; // Another thread is growing.
	}

	// We own the grow slot. Compute target capacity once.
	//
	// Hybrid: max(InMinCapacity, OldCapacity * 2).
	//
	//   - InMinCapacity is what the caller measured this instant
	//     (in the INDEX_NONE retry path: sum_visible_across_owners). Guarantees
	//     we cover the present multi-owner demand.
	//   - OldCapacity * 2 absorbs the implicit lookahead factor - the media
	//     loader pre-decodes future frames while the current frame is on
	//     screen, so the pool typically holds the union of "current visible"
	//     and "next N visible" tile sets. Doubling sizes for that headroom
	//     and amortizes future grow events: one big grow now beats many
	//     small ones over the next few seconds.
	//
	// The caller's loop in the INDEX_NONE retry path iterates if this target
	// still wasn't enough (concurrent owner published visibility between this
	// grow and the retry).
	const int32 OldCapacity = Capacity;
	const int32 NewCapacity = FMath::Max(InMinCapacity, FMath::Max(1, OldCapacity) * 2);
	const SIZE_T NewAllocSize = static_cast<SIZE_T>(NewCapacity) * static_cast<SIZE_T>(Key.PerTileBytes);

	if (NewAllocSize > MAX_uint32)
	{
		UE_LOGF(LogTmvMedia, Error,
			"FTmvSharedTilePool.Grow: target size %llu exceeds 4 GB RHI limit; staying at capacity %d.",
			static_cast<uint64>(NewAllocSize), OldCapacity);
		// seq_cst (with the seq_cst GrowWaiters.load below) so a concurrent
		// WaitForGrowComplete can't be lost to a StoreLoad reorder - see that
		// function's comment.
		bGrowing.store(false, std::memory_order_seq_cst);
		// Wake any waiters parked in WaitForGrowComplete - they'd otherwise
		// sleep until the next (potentially-never) grow.
		if (GrowDoneEvent)
		{
			const int32 N = GrowWaiters.load(std::memory_order_seq_cst);
			for (int32 i = 0; i < N; ++i)
			{
				GrowDoneEvent->Trigger();
			}
		}
		return false;
	}

	UE_LOGF(LogTmvMedia, Verbose,
		"FTmvSharedTilePool.Grow scheduled %d -> %d (%.2f MB -> %.2f MB)",
		OldCapacity, NewCapacity,
		double(AllocatedBufferSize) / (1024.0 * 1024.0),
		double(NewAllocSize) / (1024.0 * 1024.0));

	// All work - alloc, copy, swap - happens on the render thread. Caller
	// returns immediately. While grow is in flight, the pool keeps serving
	// the old buffer; AllocateSlot may return INDEX_NONE for tiles that
	// would only fit in the new capacity (caller falls back via shader
	// discard / lower-mip upscale).
	const double GrowEnqueueT = FPlatformTime::Seconds();
	TWeakPtr<FTmvSharedTilePool, ESPMode::ThreadSafe> WeakSelf = AsWeak();
	ENQUEUE_RENDER_COMMAND(GrowSharedPoolAsync)(
		[WeakSelf, NewCapacity, NewAllocSize, GrowEnqueueT](FRHICommandListImmediate& RHICmdList)
		{
			TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe> Self = WeakSelf.Pin();
			if (!Self.IsValid())
			{
				return;
			}

			const double GrowRtStartT = FPlatformTime::Seconds();
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TmvSharedTilePool_AllocateBuffer, "TmvSharedTilePool.GrowAsync");
			constexpr uint32 Stride = sizeof(uint16) * 2;

			// 1) Allocate new buffers (no lock held; not yet observable).
			const FRHIBufferCreateDesc UploadDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("TmvSharedTilePool.UploadBuffer"), NewAllocSize, Stride)
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic | EBufferUsageFlags::FastVRAM)
				.DetermineInitialState();
			FBufferRHIRef NewUpload = RHICmdList.CreateBuffer(UploadDesc);
			void* NewMapped = RHICmdList.LockBuffer(NewUpload, 0, NewAllocSize, RLM_WriteOnly);
			RHICmdList.Transition(FRHITransitionInfo(NewUpload, UploadDesc.InitialState, ERHIAccess::CopySrc));

			const FRHIBufferCreateDesc DestDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("TmvSharedTilePool.DestBuffer"), NewAllocSize, Stride)
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::FastVRAM)
				.DetermineInitialState();
			FBufferRHIRef NewShaderAccess = RHICmdList.CreateBuffer(DestDesc);
			FShaderResourceViewRHIRef NewSRV = RHICmdList.CreateShaderResourceView(NewShaderAccess, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(NewShaderAccess));

			const double PostAllocT = FPlatformTime::Seconds();

			// 2) Take the write lock for the swap. This waits until every
			// active decoder releases its read lock, so the old upload
			// buffer is no longer being written to.
			//
			// All buffer-content preservation (GPU SA copy + CPU upload
			// memcpy) happens INSIDE the write lock so the swap is atomic
			// w.r.t. decoders. Decoders briefly block during the memcpy;
			// this is acceptable as a one-time hitch because pre-sizing via
			// SharedPoolExpectedOwners makes grows rare in steady state.
			//
			// Mirrors the per-clip grow in FTmvMediaFrameMipStructuredBuffer
			// (line ~712) - same correctness contract: NewSA holds OldSA's
			// decoded bytes, NewUpload holds OldUpload's staged bytes, and
			// the residency map is preserved so existing (owner,tile) -> slot
			// mappings continue to be valid.
			FBufferRHIRef OldUploadToUnlock;
			{
				FRWScopeLock Scope(Self->GrowLock, SLT_Write);

				const FBufferRHIRef OldUpload       = Self->UploadBufferRef;
				void* const         OldMapped       = Self->UploadBufferMapped;
				const SIZE_T        OldSize         = Self->AllocatedBufferSize;

				// 3) GPU-side: copy OldUpload -> NewSA. This single async copy
				// preserves all shader-visible state in one pass:
				//
				//  - The shader binds NewSA's SRV. As long as NewSA has the
				//    correct bytes per slot, the shader is satisfied.
				//  - OldUpload and OldSA are byte-equivalent for every resident
				//    slot: every decode writes to upload, then CopyTileRegions
				//    immediately copies upload->SA. So reading OldUpload gives
				//    us the same content OldSA holds, without an OldSA->NewSA
				//    copy (which collides in the D3D12 sub-allocator because
				//    same-flag buffers can share an ID3D12Resource).
				//  - OldUpload lives in the upload heap (Dynamic, write-
				//    combined). NewSA lives in the default heap (SR|FastVRAM).
				//    Different heap categories -> guaranteed different
				//    ID3D12Resource -> no sub-allocator collision.
				//  - NewUpload starts empty. That's fine: future decodes write
				//    to NewUpload then CopyTileRegions onto the SLOT in NewSA,
				//    overwriting whatever the grow copy put there. Slots that
				//    were already resident but receive no further decode keep
				//    their preserved bytes in NewSA, which is what the shader
				//    reads.
				//
				// The CPU work here is just an RHI command enqueue (~us). The
				// GPU does the actual byte-pumping asynchronously, in parallel
				// with subsequent rendering and decode work. Decoders that
				// were waiting on the write lock can proceed immediately after
				// step 4's swap.
				if (OldUpload.IsValid() && OldSize > 0)
				{
					RHICmdList.Transition(FRHITransitionInfo(NewShaderAccess, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
					RHICmdList.CopyBufferRegion(NewShaderAccess, 0, OldUpload, 0, OldSize);
					RHICmdList.Transition(FRHITransitionInfo(NewShaderAccess, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
				}

				// 4) Swap GPU resource pointers.
				if (OldUpload && OldMapped)
				{
					OldUploadToUnlock = OldUpload;
				}
				Self->UploadBufferRef       = NewUpload;
				Self->UploadBufferMapped    = NewMapped;
				Self->ShaderAccessBufferRef = NewShaderAccess;
				Self->ShaderResourceView    = NewSRV;
				Self->AllocatedBufferSize   = NewAllocSize;

				// 5) Grow the slot table. Existing entries (0..OldCapacity)
				// keep their OwnerId/Tile/list-membership so already-resident
				// tiles remain valid - they map to the same slot index in
				// NewSA which now holds their preserved bytes (step 3a).
				// New slots default-construct to (INDEX_NONE, ESlotList::Free,
				// no prev/next) per FSlot's defaults - they need to be
				// linked into the Free list so AllocateSlot can pick them.
				{
					FScopeLock SlotLock(&Self->Cs);
					const int32 OldNumSlots = Self->Slots.Num();
					Self->Slots.SetNum(NewCapacity);
					Self->Capacity = NewCapacity;
					// Insert all newly-added slots at head of Free list.
					for (int32 i = OldNumSlots; i < NewCapacity; ++i)
					{
						Self->ListInsertAtHead_Free(i);
					}
				}
			}

			// Out-of-critical-section unlock. The driver flush for a Dynamic
			// upload buffer's unlock can be slow; decoders are now free to
			// proceed against the new buffer while this completes.
			if (OldUploadToUnlock.IsValid())
			{
				RHICmdList.UnlockBuffer(OldUploadToUnlock);
			}

			// 5) Release in-flight flag; decoders are free to re-trigger
			// future grows now. seq_cst (>= release) so any worker that
			// observes bGrowing==false also sees the swapped pool pointers
			// (UploadBufferRef / etc.) written above, AND so the matching
			// seq_cst GrowWaiters.load below can't miss a concurrent waiter to
			// a StoreLoad reorder (see WaitForGrowComplete's comment).
			Self->bGrowing.store(false, std::memory_order_seq_cst);

			// Wake threads parked in WaitForGrowComplete. Auto-reset event:
			// fire one Trigger() per known waiter so each gets unblocked.
			if (Self->GrowDoneEvent)
			{
				const int32 N = Self->GrowWaiters.load(std::memory_order_seq_cst);
				for (int32 i = 0; i < N; ++i)
				{
					Self->GrowDoneEvent->Trigger();
				}
			}

			const double GrowDoneT = FPlatformTime::Seconds();
			UE_LOGF(LogTmvMedia, Verbose,
				"FTmvSharedTilePool.Grow complete: capacity now %d (%.2f MB) [total %.0f ms: enq->rt=%.0f, alloc=%.0f, swap=%.0f]",
				NewCapacity, double(NewAllocSize) / (1024.0 * 1024.0),
				(GrowDoneT - GrowEnqueueT) * 1000.0,
				(GrowRtStartT - GrowEnqueueT) * 1000.0,
				(PostAllocT - GrowRtStartT) * 1000.0,
				(GrowDoneT - PostAllocT) * 1000.0);
		});

	return true;
}

void* FTmvSharedTilePool::GetMappedBuffer() const
{
	// Lock-free; if the caller hasn't taken a read lock on GrowLock, the
	// pointer can shift under them mid-grow. Decoders MUST acquire an
	// FReadLockScope across mapped-buffer use.
	if (!bGpuReady.load(std::memory_order_seq_cst))
	{
		return nullptr;
	}
	return UploadBufferMapped;
}

FShaderResourceViewRHIRef FTmvSharedTilePool::GetShaderResourceView() const
{
	if (!bGpuReady.load(std::memory_order_seq_cst))
	{
		return FShaderResourceViewRHIRef();
	}
	FScopeLock Lock(&Cs);
	return ShaderResourceView;
}

FBufferRHIRef FTmvSharedTilePool::GetUploadBuffer() const
{
	if (!bGpuReady.load(std::memory_order_seq_cst))
	{
		return FBufferRHIRef();
	}
	FScopeLock Lock(&Cs);
	return UploadBufferRef;
}

FBufferRHIRef FTmvSharedTilePool::GetShaderAccessBuffer() const
{
	if (!bGpuReady.load(std::memory_order_seq_cst))
	{
		return FBufferRHIRef();
	}
	FScopeLock Lock(&Cs);
	return ShaderAccessBufferRef;
}

// =====================================================================
// FTmvSharedTilePoolManager
// =====================================================================

FTmvSharedTilePoolManager& FTmvSharedTilePoolManager::Get()
{
	static FTmvSharedTilePoolManager Singleton;
	return Singleton;
}

TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe> FTmvSharedTilePoolManager::GetOrCreatePool(const FTmvMediaFrameMipInfo& InMipInfo, int32 InitialCapacity)
{
	const FTmvSharedTilePool::FKey Key = FTmvSharedTilePool::MakeKey(InMipInfo);
	FScopeLock Lock(&PoolsLock);
	if (TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe>* Found = Pools.Find(Key))
	{
		return *Found;
	}
	TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe> NewPool = MakeShared<FTmvSharedTilePool, ESPMode::ThreadSafe>(Key, InitialCapacity);
	// Now that the pool is owned by a TSharedPtr, it is safe to call AsWeak()
	// (which BeginInitialAllocation does indirectly inside the lambda capture).
	NewPool->BeginInitialAllocation();
	Pools.Add(Key, NewPool);
	return NewPool;
}

void FTmvSharedTilePoolManager::ReleaseOwner(const TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe>& InPool, FTmvSharedTilePool::FOwnerHandle InOwner)
{
	if (!InPool.IsValid())
	{
		return;
	}

	// Hold PoolsLock across the unregister + owner-count check so a concurrent
	// GetOrCreatePool can't hand this pool to a new clip in the window between
	// "last owner left" and "removed from map". GetOrCreatePool / ReleaseOwner
	// both take PoolsLock first, then the pool's Cs, so the ordering is consistent.
	FScopeLock Lock(&PoolsLock);
	InPool->UnregisterOwner(InOwner);
	if (InPool->GetOwnerCount() == 0)
	{
		// Only evict the entry if it is still THIS pool. A rare interleave (last
		// owner closing while a same-format clip opens) can leave a different
		// pool under the key; removing by key alone would evict the wrong one.
		// The closing buffer's own TSharedPtr then frees this pool on destruction.
		if (const TSharedPtr<FTmvSharedTilePool, ESPMode::ThreadSafe>* Found = Pools.Find(InPool->GetKey());
			Found && *Found == InPool)
		{
			Pools.Remove(InPool->GetKey());
		}
	}
}

void FTmvSharedTilePoolManager::ReleaseAll()
{
	FScopeLock Lock(&PoolsLock);
	Pools.Empty();
}
