// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleConverter/TmvSharedTileSession.h"

#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "SampleConverter/TmvMediaTileResidency.h"
#include "SampleConverter/TmvSharedTilePool.h"

namespace
{
	// Validate-then-deref so the null check runs BEFORE the Residency reference
	// member is bound. A member initializer list always runs before the constructor
	// body, so a check() in the body cannot guard the bind. Only virtualized
	// (structured) buffers create a session, so GetResidency() is always non-null.
	FTmvMediaTileResidency& GetResidencyChecked(FTmvMediaFrameMipBuffer& InBuffer)
	{
		FTmvMediaTileResidency* const Residency = InBuffer.GetResidency();
		check(Residency != nullptr);
		return *Residency;
	}
}

FTmvSharedTileSession::FTmvSharedTileSession(FTmvMediaFrameMipBuffer& InBuffer)
	: Buffer(InBuffer)
	, Residency(GetResidencyChecked(InBuffer))
{
}

void FTmvSharedTileSession::EnsureCapacityFor(int32 InVisibleCount)
{
	if (InVisibleCount > Residency.Capacity)
	{
		Buffer.EnsureCapacityFor(InVisibleCount);
	}
}

void FTmvSharedTileSession::PublishVisibility(const TSet<FIntPoint>& InVisibleSet)
{
	if (Residency.IsShared())
	{
		Residency.SharedPool->SetOwnerVisibility(Residency.SharedOwner, InVisibleSet);
		Residency.TileToSlot.Reset();
	}
}

FTmvSharedTileSession::FSlotResolution FTmvSharedTileSession::ResolveSlot(FIntPoint InTile, const TSet<FIntPoint>& InVisibleSet)
{
	// Already resident? In shared mode this queries the pool authoritatively; we
	// still mirror the slot into the local TileToSlot map so the post-decode
	// descriptor rebuild sees it.
	const int32 ExistingSlot = Residency.GetSlot(InTile);
	if (ExistingSlot != INDEX_NONE)
	{
		if (Residency.IsShared())
		{
			Residency.TileToSlot.Add(InTile, ExistingSlot);
		}
		return { ESlotResult::AlreadyResident, ExistingSlot };
	}

	int32 NewSlot = Residency.AllocateSlot(InTile, InVisibleSet);
	if (NewSlot == INDEX_NONE)
	{
		// Pool exhausted of evictable slots (every slot pinned to a currently-
		// visible tile in some owner). In blocking-media mode we cannot ship a
		// sample with this tile missing -> grow + retry.
		//
		// Each iteration recomputes sum-visible and asks the pool to grow to
		// that. Necessary because when a grow is already in flight, EnsureCapacityFor
		// returns immediately without scheduling a second grow; WaitForGrowComplete
		// then returns once the first grow lands, but that grow's target may have
		// been smaller than what we now need (more owners published visibility
		// since the in-flight grow's target was computed). A further iteration
		// captures the new demand. Bounded to keep pathological cases from spinning.
		for (int32 Attempt = 0; Attempt < MaxGrowRetries && NewSlot == INDEX_NONE; ++Attempt)
		{
			int32 GrowTarget = Residency.Capacity > 0 ? Residency.Capacity : 64;
			if (Residency.IsShared() && Residency.SharedPool.IsValid())
			{
				const int32 SumVisible = Residency.SharedPool->GetTotalVisibleTileCount();
				// Ensure we grow STRICTLY beyond current pool capacity even if
				// SumVisible momentarily fits - otherwise the pool's
				// EnsureCapacityFor no-ops on a "Capacity >= InMinCapacity" check
				// and we'd loop forever returning INDEX_NONE.
				const int32 PoolCap = Residency.SharedPool->GetCapacity();
				GrowTarget = FMath::Max3(GrowTarget, SumVisible, PoolCap + 1);
			}
			Buffer.EnsureCapacityFor(GrowTarget);
			if (Residency.IsShared() && Residency.SharedPool.IsValid())
			{
				Residency.SharedPool->WaitForGrowComplete();
				Residency.Capacity = Residency.SharedPool->GetCapacity();
				NewSlot = Residency.AllocateSlot(InTile, InVisibleSet);
			}
			else
			{
				// Non-shared path: one shot, no retry loop.
				break;
			}
		}
		if (NewSlot == INDEX_NONE)
		{
			return { ESlotResult::Overflowed, INDEX_NONE };
		}
	}

	return { ESlotResult::Allocated, NewSlot };
}
