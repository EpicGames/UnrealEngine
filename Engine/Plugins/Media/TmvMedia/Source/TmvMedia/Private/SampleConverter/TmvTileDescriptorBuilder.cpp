// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleConverter/TmvTileDescriptorBuilder.h"

#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "SampleConverter/TmvMediaTileResidency.h"
#include "TmvMediaFrameInfo.h"
#include "TmvMediaShaderDefines.h"

namespace UE::TmvMedia::TileDescriptorBuilderPrivate
{
	/**
	 * Layout of one tile in a tile-major mip buffer: per-plane intra-tile byte
	 * offsets and per-plane row strides. Identical for every tile in a mip
	 * because planes are interleaved within each tile at fixed offsets - the
	 * only thing that differs per tile is the byte base (TileBytes * index-or-slot).
	 */
	struct FIntraTileLayout
	{
		static constexpr int32 MaxPlanes = 4;
		int32 NumPlanes = 0;
		int64 IntraTileOffset[MaxPlanes] = { 0, 0, 0, 0 };
		int64 Stride[MaxPlanes]          = { 0, 0, 0, 0 };
	};

	FIntraTileLayout ComputeIntraTileLayout(const FTmvMediaFrameMipInfo& InMipInfo)
	{
		FIntraTileLayout Out;
		Out.NumPlanes = FMath::Min(InMipInfo.Planes.Num(), FIntraTileLayout::MaxPlanes);
		int64 Cursor = 0;
		for (int32 PlaneIndex = 0; PlaneIndex < Out.NumPlanes; ++PlaneIndex)
		{
			Out.IntraTileOffset[PlaneIndex] = Cursor;
			Out.Stride[PlaneIndex]          = static_cast<int64>(InMipInfo.Planes[PlaneIndex].Stride);
			Cursor += static_cast<int64>(InMipInfo.Planes[PlaneIndex].GetMemorySizeInBytes());
		}
		return Out;
	}

	/**
	 * Shader expects offsets/strides in SHORTS, not bytes: the unswizzled buffer
	 * is a StructuredBuffer<uint> with uint16-packed elements (GetBufferValue
	 * indexes UnswizzledBuffer[Offset/2]).
	 */
	void FillTileDesc(FTmvMediaShaderTileDesc& OutDesc, int64 InTileByteBase, const FIntraTileLayout& InLayout)
	{
		for (int32 PlaneIndex = 0; PlaneIndex < InLayout.NumPlanes; ++PlaneIndex)
		{
			OutDesc.Offsets[PlaneIndex] = static_cast<int32>((InTileByteBase + InLayout.IntraTileOffset[PlaneIndex]) / 2);
			OutDesc.Strides[PlaneIndex] = static_cast<int32>(InLayout.Stride[PlaneIndex] / 2);
		}
	}
}

void FTmvTileDescriptorBuilder::BuildNaturalIndex(const FTmvMediaFrameMipInfo& InMipInfo, TArray<FTmvMediaShaderTileDesc>& OutDescs)
{
	using namespace UE::TmvMedia::TileDescriptorBuilderPrivate;

	const int32 NumTilesX = InMipInfo.NumTiles.X;
	const int32 NumTilesY = InMipInfo.NumTiles.Y;
	const int64 TileBytes = static_cast<int64>(InMipInfo.GetAllPlaneMemorySizeInBytes());
	if (NumTilesX <= 0 || NumTilesY <= 0 || TileBytes <= 0)
	{
		return;
	}

	OutDescs.SetNumZeroed(NumTilesX * NumTilesY);

	// Virtualized mips are filled from their residency map post-decode (slots
	// don't correspond to tile coords), so leave the zeroed array as a placeholder.
	if (InMipInfo.IsVirtualized())
	{
		return;
	}

	const FIntraTileLayout Layout = ComputeIntraTileLayout(InMipInfo);
	for (int32 TileY = 0; TileY < NumTilesY; ++TileY)
	{
		for (int32 TileX = 0; TileX < NumTilesX; ++TileX)
		{
			const int32 Idx = TileY * NumTilesX + TileX;
			FillTileDesc(OutDescs[Idx], TileBytes * Idx, Layout);
		}
	}
}

void FTmvTileDescriptorBuilder::BuildFromResidency(const FTmvMediaFrameMipInfo& InMipInfo, const FTmvMediaTileResidency& InResidency, TArray<FTmvMediaShaderTileDesc>& OutDescs)
{
	using namespace UE::TmvMedia::TileDescriptorBuilderPrivate;

	const int32 NumTilesX = InMipInfo.NumTiles.X;
	const int32 NumTilesY = InMipInfo.NumTiles.Y;
	const int64 TileBytes = static_cast<int64>(InMipInfo.GetAllPlaneMemorySizeInBytes());
	if (NumTilesX <= 0 || NumTilesY <= 0 || TileBytes <= 0)
	{
		return;
	}

	if (OutDescs.Num() != NumTilesX * NumTilesY)
	{
		OutDescs.SetNumZeroed(NumTilesX * NumTilesY);
	}
	else
	{
		// Clear any stale slot offsets from a prior frame's residency (the
		// descriptor cache hashes content, so we only repay this if the
		// residency actually shifted).
		FMemory::Memzero(OutDescs.GetData(), static_cast<SIZE_T>(OutDescs.Num()) * sizeof(FTmvMediaShaderTileDesc));
	}

	const FIntraTileLayout Layout = ComputeIntraTileLayout(InMipInfo);
	for (const TPair<FIntPoint, int32>& TileSlotPair : InResidency.TileToSlot)
	{
		const FIntPoint Tile = TileSlotPair.Key;
		const int32 Slot = TileSlotPair.Value;
		if (Tile.X < 0 || Tile.X >= NumTilesX || Tile.Y < 0 || Tile.Y >= NumTilesY)
		{
			continue;
		}
		const int32 TileIdx = Tile.Y * NumTilesX + Tile.X;
		FillTileDesc(OutDescs[TileIdx], TileBytes * static_cast<int64>(Slot), Layout);
	}
}
