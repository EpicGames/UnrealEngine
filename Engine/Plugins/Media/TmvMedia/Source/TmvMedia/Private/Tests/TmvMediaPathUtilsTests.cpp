// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Engine/Engine.h"
#include "HAL/FileManager.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "Utils/TmvMediaPathUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::TmvMedia::Tests
{
	/** Test Utility to create temporary directories and delete them when done. */
	struct FTestFilesAndDirectories
	{
		~FTestFilesAndDirectories()
		{
			for (const FString& File : CreatedFiles)
			{
				IFileManager::Get().Delete(*File);
			}

			for(const FString& Directory : ReverseIterate(CreatedDirectories))
			{
				IFileManager::Get().DeleteDirectory(*Directory);
			}
		}

		bool MakeDirectory(const FString& InPath)
		{
			if (FPaths::DirectoryExists(InPath))
			{
				return true;
			}
			if (IFileManager::Get().MakeDirectory(*InPath))
			{
				CreatedDirectories.Add(InPath);
				return true;
			}
			return false;
		}

		bool MakeFile(const FString& InFilepath)
		{
			if (FPaths::FileExists(InFilepath))
			{
				return true;
			}
			const TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*InFilepath));
			if (FileWriter)
			{
				int32 value = 1234;
				(*FileWriter) << value;
				CreatedFiles.Add(InFilepath);
				return true;
			}
			return false;
		}


	private:
		TArray<FString> CreatedDirectories;
		TArray<FString> CreatedFiles;
	};
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTmvMediaSanitizePathTests, "System.Plugins.TmvMedia.SanitizePath", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FTmvMediaSanitizePathTests::RunTest(const FString& Parameters)
{
	using namespace UE::TmvMedia::Tests;
	
	// The test needs some existing files in different folders.
	FTestFilesAndDirectories TestFilesAndDirectories;

	auto MakeTempTestFileAndPath = [&TestFilesAndDirectories](const FString& InBasePath)
	{
		const FString MoviesDirectory = FPaths::ConvertRelativePathToFull(FPaths::Combine(InBasePath, TEXT("Movies")));
		const FString SequenceDirectory = FPaths::Combine(MoviesDirectory, TEXT("TmvMediaTestSequence"));
		const FString SequenceImgPath = FPaths::Combine(SequenceDirectory, TEXT("Image0000.exr"));
		if (TestFilesAndDirectories.MakeDirectory(MoviesDirectory))
		{
			if (TestFilesAndDirectories.MakeDirectory(SequenceDirectory))
			{
				return TestFilesAndDirectories.MakeFile(SequenceImgPath);
			}
		}
		return false;
	};

	// Prepare temporary files under different base paths. We want the Content vs Project one to test priority.
	bool bSetupSuccess = true;
	bSetupSuccess &= MakeTempTestFileAndPath(FPaths::ProjectContentDir());
	bSetupSuccess &= MakeTempTestFileAndPath(FPaths::ProjectDir());
	bSetupSuccess &= MakeTempTestFileAndPath(FPaths::EngineDir());

	// If the setup failed, issue a warning to help troubleshooting CIS.
	if (!bSetupSuccess)
	{
		AddWarning(TEXT("Failed to create the test media files and folders."));
	}

	// ---- Tests Begin here

	// Testing GetSanitizedPath
	auto TestSanitizePath = [this](const FString& InPath, const FString& InExpectedResult, FString& OutError)
	{
		const FString SanitizedPath = UE::TmvMedia::PathUtils::GetSanitizedPath(InPath);
		if (SanitizedPath != InExpectedResult)
		{
			OutError = FString::Printf(TEXT("GetSanitizedPath failed: Input: \"%s\" Result: \"%s\" Expected \"%s\""), *InPath, *SanitizedPath, *InExpectedResult);
			return false;
		}
		return true;
	};

	// Testing ConvertPathToFull, this is the complement to GetSanitizedPath.
	auto TestConvertPathToFull = [this](const FString& InSanitizedPath, const FString& InExpectedResult, FString& OutError)
	{
		const FString FullPath = UE::TmvMedia::PathUtils::ConvertSanitizedPathToFull(InSanitizedPath);
		if (FullPath != InExpectedResult)
		{
			OutError = FString::Printf(TEXT("ConvertPathToFull failed: Input: \"%s\" Result: \"%s\" Expected: \"%s\""), *InSanitizedPath, *FullPath, *InExpectedResult);
			return false;
		}
		return true;
	};

	FString Error;
	
	// Empty path
	AddErrorIfFalse(TestSanitizePath(TEXT(""), TEXT(""), Error), Error);
	
	{
		// ------------- Already sanitized paths ----------------
		{
			// directory only - existing
			const FString ExpectedSanitized = TEXT("./Movies/TmvMediaTestSequence");
			AddErrorIfFalse(TestSanitizePath(TEXT("./Movies/TmvMediaTestSequence"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("./Movies/TmvMediaTestSequence/"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT(".\\Movies\\TmvMediaTestSequence"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT(".\\Movies\\TmvMediaTestSequence\\"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("\".\\Movies\\TmvMediaTestSequence\""), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("Movies/TmvMediaTestSequence"));
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}		
		{
			// directory only - non-existing
			const FString ExpectedSanitized = TEXT("./Movies/TmvMediaTestSequenceNotExisting");
			AddErrorIfFalse(TestSanitizePath(TEXT("./Movies/TmvMediaTestSequenceNotExisting"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("./Movies/TmvMediaTestSequenceNotExisting/"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT(".\\Movies\\TmvMediaTestSequenceNotExisting"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT(".\\Movies\\TmvMediaTestSequenceNotExisting\\"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("\".\\Movies\\TmvMediaTestSequenceNotExisting\""), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("Movies/TmvMediaTestSequenceNotExisting"));
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}
		{
			// Already sanitized - with filename - existing
			const FString ExpectedSanitized = TEXT("./Movies/TmvMediaTestSequence/Image0000.exr");
			AddErrorIfFalse(TestSanitizePath(TEXT("./Movies/TmvMediaTestSequence/Image0000.exr"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT(".\\Movies\\TmvMediaTestSequence\\Image0000.exr"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("\".\\Movies\\TmvMediaTestSequence\\Image0000.exr\""), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("Movies/TmvMediaTestSequence/Image0000.exr"));
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}
		{
			// Already sanitized - with filename - non-existing
			const FString ExpectedSanitized = TEXT("./Movies/TmvMediaTestSequenceNotExisting/Image0000.exr");
			AddErrorIfFalse(TestSanitizePath(TEXT("./Movies/TmvMediaTestSequenceNotExisting/Image0000.exr"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT(".\\Movies\\TmvMediaTestSequenceNotExisting\\Image0000.exr"), ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("\".\\Movies\\TmvMediaTestSequenceNotExisting\\Image0000.exr\""), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("Movies/TmvMediaTestSequenceNotExisting/Image0000.exr"));
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}
	}
	
	{
		// ------------- Project Relative ----------------
		{
			// directory only - existing
			const FString ExpectedSanitized = TEXT("./Movies/TmvMediaTestSequence");
			const FString ProjectRelative = FPaths::Combine(FPaths::ProjectDir(), TEXT("Movies/TmvMediaTestSequence"));
			AddErrorIfFalse(TestSanitizePath(ProjectRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(FPaths::ConvertRelativePathToFull(ProjectRelative), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::ConvertRelativePathToFull(ProjectRelative);
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}
		{
			// directory only - non-existing ---- Experimental
			const FString ExpectedSanitized = TEXT("./Movies/TmvMediaTestSequenceNotExisting");
			const FString ProjectRelative = FPaths::Combine(FPaths::ProjectDir(), TEXT("Movies/TmvMediaTestSequenceNotExisting"));
			AddErrorIfFalse(TestSanitizePath(ProjectRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(FPaths::ConvertRelativePathToFull(ProjectRelative), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::ConvertRelativePathToFull(ProjectRelative);
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}
		{
			// with filename - existing
			const FString ExpectedSanitized = TEXT("./Movies/TmvMediaTestSequence/Image0000.exr");
			const FString ProjectRelative = FPaths::Combine(FPaths::ProjectDir(), TEXT("Movies/TmvMediaTestSequence/Image0000.exr"));
			AddErrorIfFalse(TestSanitizePath(ProjectRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(FPaths::ConvertRelativePathToFull(ProjectRelative), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::ConvertRelativePathToFull(ProjectRelative);
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}
		{
			// with filename - non-existing ---- Experimental
			const FString ExpectedSanitized = TEXT("./Movies/TmvMediaTestSequenceNotExisting/Image0000.exr");
			const FString ProjectRelative = FPaths::Combine(FPaths::ProjectDir(), TEXT("Movies/TmvMediaTestSequenceNotExisting/Image0000.exr"));
			AddErrorIfFalse(TestSanitizePath(ProjectRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(FPaths::ConvertRelativePathToFull(ProjectRelative), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::ConvertRelativePathToFull(ProjectRelative);
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}
	}

	{
		// ------------- Content relative -------------
		// Note: Non-existing content relative can't be done with this syntax. Testing only existing files and directories.
		{
			// directory only - existing
			const FString ExpectedSanitized = TEXT("./Content/Movies/TmvMediaTestSequence");
			const FString ContentRelative = FPaths::Combine(FPaths::ProjectContentDir(), TEXT("Movies/TmvMediaTestSequence"));
			AddErrorIfFalse(TestSanitizePath(ContentRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(FPaths::ConvertRelativePathToFull(ContentRelative), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::ConvertRelativePathToFull(ContentRelative);
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}
		{
			// With filename - existing
			const FString ExpectedSanitized = TEXT("./Content/Movies/TmvMediaTestSequence/Image0000.exr");
			const FString ContentRelative = FPaths::Combine(FPaths::ProjectContentDir(), TEXT("Movies/TmvMediaTestSequence/Image0000.exr"));
			AddErrorIfFalse(TestSanitizePath(ContentRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(FPaths::ConvertRelativePathToFull(ContentRelative), ExpectedSanitized, Error), Error);

			const FString ExpectedFull = FPaths::ConvertRelativePathToFull(ContentRelative);
			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, ExpectedFull, Error), Error);
		}
	}

	{
		// ------------- Outside of project (ex: engine) -------------
		// expect absolute path
		{
			// directory only - existing
			const FString EngineRelative = FPaths::Combine(FPaths::EngineDir(), TEXT("Movies/TmvMediaTestSequence"));
			const FString EngineAbsolute = FPaths::ConvertRelativePathToFull(EngineRelative);
			const FString ExpectedSanitized = EngineAbsolute;
			
			AddErrorIfFalse(TestSanitizePath(EngineRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(EngineAbsolute, ExpectedSanitized, Error), Error);

			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, EngineAbsolute, Error), Error);
		}

		{
			// directory only - non-existing
			const FString EngineRelative = FPaths::Combine(FPaths::EngineDir(), TEXT("Movies/TmvMediaTestSequenceNotExisting"));
			const FString EngineAbsolute = FPaths::ConvertRelativePathToFull(EngineRelative);
			const FString ExpectedSanitized = EngineAbsolute;
			
			AddErrorIfFalse(TestSanitizePath(EngineRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(EngineAbsolute, ExpectedSanitized, Error), Error);

			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, EngineAbsolute, Error), Error);
		}
		
		{
			// with filename - existing
			const FString EngineRelative = FPaths::Combine(FPaths::EngineDir(), TEXT("Movies/TmvMediaTestSequence/Image0000.exr"));
			const FString EngineAbsolute = FPaths::ConvertRelativePathToFull(EngineRelative);
			const FString ExpectedSanitized = EngineAbsolute;
			
			AddErrorIfFalse(TestSanitizePath(EngineRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(EngineAbsolute, ExpectedSanitized, Error), Error);

			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, EngineAbsolute, Error), Error);
		}

		{
			// with filename - non-existing
			const FString EngineRelative = FPaths::Combine(FPaths::EngineDir(), TEXT("Movies/TmvMediaTestSequenceNotExisting/Image0000.exr"));
			const FString EngineAbsolute = FPaths::ConvertRelativePathToFull(EngineRelative);
			const FString ExpectedSanitized = EngineAbsolute;
			
			AddErrorIfFalse(TestSanitizePath(EngineRelative, ExpectedSanitized, Error), Error);
			AddErrorIfFalse(TestSanitizePath(EngineAbsolute, ExpectedSanitized, Error), Error);

			AddErrorIfFalse(TestConvertPathToFull(ExpectedSanitized, EngineAbsolute, Error), Error);
		}
	}

	{
		// ------------- Project relative with Token -------------
		// Note: also testing "sanitization" of paths with tokens.
		{
			// directory only - existing
			FString Expected = TEXT("{project_dir}/Movies/TmvMediaTestSequence");

			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}/Movies/TmvMediaTestSequence"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}/Movies/TmvMediaTestSequence/"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}\\Movies\\TmvMediaTestSequence"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}\\Movies\\TmvMediaTestSequence\\"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("\"{project_dir}\\Movies\\TmvMediaTestSequence\""), Expected, Error), Error);

			AddErrorIfFalse(TestSanitizePath(Expected, Expected, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("Movies/TmvMediaTestSequence"));
			AddErrorIfFalse(TestConvertPathToFull(Expected, ExpectedFull, Error), Error);
		}
		{
			// directory only - non-existing
			FString Expected = TEXT("{project_dir}/Movies/TmvMediaTestSequenceNotExisting");

			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}/Movies/TmvMediaTestSequenceNotExisting"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}/Movies/TmvMediaTestSequenceNotExisting/"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}\\Movies\\TmvMediaTestSequenceNotExisting"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}\\Movies\\TmvMediaTestSequenceNotExisting\\"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("\"{project_dir}\\Movies\\TmvMediaTestSequenceNotExisting\""), Expected, Error), Error);

			AddErrorIfFalse(TestSanitizePath(Expected, Expected, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("Movies/TmvMediaTestSequenceNotExisting"));
			AddErrorIfFalse(TestConvertPathToFull(Expected, ExpectedFull, Error), Error);
		}
		{
			// with filename - existing
			FString Expected = TEXT("{project_dir}/Movies/TmvMediaTestSequence/Image0000.exr");
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}/Movies/TmvMediaTestSequence/Image0000.exr"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}\\Movies\\TmvMediaTestSequence\\Image0000.exr"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("\"{project_dir}\\Movies\\TmvMediaTestSequence\\Image0000.exr\""), Expected, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("Movies/TmvMediaTestSequence/Image0000.exr"));
			AddErrorIfFalse(TestConvertPathToFull(Expected, ExpectedFull, Error), Error);
		}
		{
			// with filename - non-existing
			FString Expected = TEXT("{project_dir}/Movies/TmvMediaTestSequenceNotExisting/Image0000.exr");
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}/Movies/TmvMediaTestSequenceNotExisting/Image0000.exr"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("{project_dir}\\Movies\\TmvMediaTestSequenceNotExisting\\Image0000.exr"), Expected, Error), Error);
			AddErrorIfFalse(TestSanitizePath(TEXT("\"{project_dir}\\Movies\\TmvMediaTestSequenceNotExisting\\Image0000.exr\""), Expected, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("Movies/TmvMediaTestSequenceNotExisting/Image0000.exr"));
			AddErrorIfFalse(TestConvertPathToFull(Expected, ExpectedFull, Error), Error);
		}
	}

	{
		// ------------- Engine relative with Token -------------
		{
			// directory only - existing
			FString Expected = TEXT("{engine_dir}/Movies/TmvMediaTestSequence");
			AddErrorIfFalse(TestSanitizePath(Expected, Expected, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), TEXT("Movies/TmvMediaTestSequence"));
			AddErrorIfFalse(TestConvertPathToFull(Expected, ExpectedFull, Error), Error);
		}
		{
			// directory only - non-existing
			FString Expected = TEXT("{engine_dir}/Movies/TmvMediaTestSequenceNotExisting");
			AddErrorIfFalse(TestSanitizePath(Expected, Expected, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), TEXT("Movies/TmvMediaTestSequenceNotExisting"));
			AddErrorIfFalse(TestConvertPathToFull(Expected, ExpectedFull, Error), Error);
		}
		{
			// with filename - existing
			FString Expected = TEXT("{engine_dir}/Movies/TmvMediaTestSequence/Image0000.exr");
			AddErrorIfFalse(TestSanitizePath(Expected, Expected, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), TEXT("Movies/TmvMediaTestSequence/Image0000.exr"));
			AddErrorIfFalse(TestConvertPathToFull(Expected, ExpectedFull, Error), Error);
		}
		{
			// with filename - existing
			FString Expected = TEXT("{engine_dir}/Movies/TmvMediaTestSequenceNotExisting/Image0000.exr");
			AddErrorIfFalse(TestSanitizePath(Expected, Expected, Error), Error);

			const FString ExpectedFull = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), TEXT("Movies/TmvMediaTestSequenceNotExisting/Image0000.exr"));
			AddErrorIfFalse(TestConvertPathToFull(Expected, ExpectedFull, Error), Error);
		}
	}

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTmvMediaMakeFrameFilenameTests, "System.Plugins.TmvMedia.MakeFrameFilename", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FTmvMediaMakeFrameFilenameTests::RunTest(const FString& Parameters)
{
	// Testing MakeFrameFilename
	auto TestMakeFrameFilename = [this](const FString& InBaseName, int32 InFrameNumber, int32 InZeroPad, const FString& InExtension, const FString& InExpectedResult, FString& OutError)
	{
		const FString FrameFilename = UE::TmvMedia::PathUtils::MakeFrameFilename(InBaseName, InFrameNumber, InZeroPad, InExtension);
		if (FrameFilename != InExpectedResult)
		{
			OutError = FString::Printf(TEXT("MakeFrameFilename Test Failed: BaseName: \"%s\" FrameNumber: %d  ZeroPad: %d Ext: \"%s\" Result: \"%s\" Expected \"%s\""),
				*InBaseName, InFrameNumber, InZeroPad, *InExtension, *FrameFilename, *InExpectedResult);
			return false;
		}
		return true;
	};

	FString Error;

	// Test zero pad
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 0, 1, TEXT(".exr"), TEXT("Frame_0.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 0, 2, TEXT(".exr"), TEXT("Frame_00.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 0, 3, TEXT(".exr"), TEXT("Frame_000.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 0, 4, TEXT(".exr"), TEXT("Frame_0000.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 0, 5, TEXT(".exr"), TEXT("Frame_00000.exr"), Error), Error);

	// Test frame index
	constexpr int32 ZeroPad = 5;
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 0, ZeroPad, TEXT(".exr"), TEXT("Frame_00000.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 0, ZeroPad, TEXT("exr"), TEXT("Frame_00000.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 123, ZeroPad, TEXT(".exr"), TEXT("Frame_00123.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 123, ZeroPad, TEXT(".tmv.exr"), TEXT("Frame_00123.tmv.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame."), 123, ZeroPad, TEXT(".exr"), TEXT("Frame.00123.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame."), 123, ZeroPad, TEXT(".tmv.exr"), TEXT("Frame.00123.tmv.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_"), 123456, ZeroPad, TEXT(".exr"), TEXT("Frame_123456.exr"), Error), Error);

	// {frame_number} token and dot naming
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_{frame_number}"), 123, ZeroPad, TEXT(".exr"), TEXT("Frame_00123.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_{frame_number}"), 123, ZeroPad, TEXT("exr"), TEXT("Frame_00123.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_{frame_number}"), 123, ZeroPad, TEXT("apv1"), TEXT("Frame_00123.apv1"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_{frame_number}"), 123, ZeroPad, TEXT(".apv1"), TEXT("Frame_00123.apv1"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_{frame_number}"), 123, ZeroPad, TEXT(".apv1.exr"), TEXT("Frame_00123.apv1.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame.{frame_number}"), 123, ZeroPad, TEXT("exr"), TEXT("Frame.00123.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_{frame_number}."), 123, ZeroPad, TEXT(".exr"), TEXT("Frame_00123.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_{frame_number}."), 123, ZeroPad, TEXT("exr"), TEXT("Frame_00123.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame_{frame_number}_PostFix"), 123, ZeroPad, TEXT(".exr"), TEXT("Frame_00123_PostFix.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("Frame.{frame_number}.PostFix"), 123, ZeroPad, TEXT(".exr"), TEXT("Frame.00123.PostFix.exr"), Error), Error);

	// Test invalid characters /?:&\\*\"<>|%#@^ in either base name or extension.
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("?Fra?me_?"), 123, ZeroPad, TEXT(".exr"), TEXT("Frame_00123.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("&Fra*me_>"), 123, ZeroPad, TEXT(".exr"), TEXT("Frame_00123.exr"), Error), Error);
	AddErrorIfFalse(TestMakeFrameFilename(TEXT("\\Fra/me_\""), 123, ZeroPad, TEXT(".&ex*r"), TEXT("Frame_00123.exr"), Error), Error);
	
	return !HasAnyErrors();
}

#endif
