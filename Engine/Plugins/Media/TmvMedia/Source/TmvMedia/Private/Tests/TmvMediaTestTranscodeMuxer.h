// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transcoder/TmvMediaTranscodeMuxer.h"

#include "TmvMediaTestTranscodeMuxer.generated.h"

/**
 * Concrete stub of UTmvMediaTranscodeMuxer used only by the TmvMedia tests.
 *
 * The base class is UCLASS(Abstract) and declares two PURE_VIRTUAL methods, so it cannot itself
 * be instantiated by FTmvMediaTranscodeJobBuilder. This subclass provides empty implementations so
 * tests that need to verify "the override class is the one that gets instantiated" don't have to
 * piggyback on production muxers (UTmvMediaContainerTranscodeMuxer / UTmvMediaFileSequenceTranscodeMuxer).
 */
UCLASS()
class UTmvMediaTestTranscodeMuxer : public UTmvMediaTranscodeMuxer
{
	GENERATED_BODY()

public:
	//~ Begin UTmvMediaTranscodeMuxer
	virtual int32 OpenStream(UTmvMediaTranscodeJob* /*InParentJob*/, const FString& /*InStreamName*/, const FString& /*InExtension*/) override
	{
		return INDEX_NONE;
	}

	virtual void ReceiveAccessUnit(
		UTmvMediaTranscodeJob* /*InParentJob*/,
		int32 /*InStreamId*/,
		const FTmvMediaFrameTimeInfo& /*TimeInfo*/,
		TSharedPtr<TArray64<uint8>>&& /*InAccessUnit*/) override
	{
	}
	//~ End UTmvMediaTranscodeMuxer
};
