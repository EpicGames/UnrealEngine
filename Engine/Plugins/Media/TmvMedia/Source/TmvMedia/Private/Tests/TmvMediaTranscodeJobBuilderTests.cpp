// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "CQTest.h"

#include "Encoder/TmvMediaEncoderOptions.h"
#include "TmvMediaTestTranscodeMuxer.h"
#include "Transcoder/ITmvMediaTranscodeJobManager.h"
#include "Transcoder/TmvMediaContainerTranscodeMuxer.h"
#include "Transcoder/TmvMediaFileSequenceFrameProducer.h"
#include "Transcoder/TmvMediaFileSequenceTranscodeMuxer.h"
#include "Transcoder/TmvMediaFrameConverter.h"
#include "Transcoder/TmvMediaFrameEncoder.h"
#include "Transcoder/TmvMediaFrameProducer.h"
#include "Transcoder/TmvMediaPlayerFrameProducer.h"
#include "Transcoder/TmvMediaTmvFrameEncoder.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeJobBuilder.h"
#include "Transcoder/TmvMediaTranscodeList.h"
#include "Transcoder/TmvMediaTranscodeMuxer.h"
#include "Utils/TmvMediaMessageContext.h"

namespace UE::TmvMedia::Tests
{
	/**
	 * Builds a minimal FTmvMediaTranscodeListItem with both paths set so the builder
	 * always has a valid item. EncoderOptions is intentionally left default-constructed
	 * (empty TInstancedStruct) — SetEncoderOptions merely stores the struct and is safe to
	 * call before Start(), so no crash occurs and no cross-module dep on ApvMedia is needed.
	 */
	static FTmvMediaTranscodeListItem MakeMinimalJobItem()
	{
		FTmvMediaTranscodeListItem Item;
		Item.Name = TEXT("TestJob");
		Item.Settings.InputPath.FilePath = TEXT("/TestInput/sequence.exr");
		Item.Settings.OutputPath.Path   = TEXT("/TestOutput/");
		// bUseMediaPlayer defaults to true, OutputFormat defaults to Container.
		return Item;
	}

	/**
	 * UTmvMediaTranscodeJob's constructor registers with the global manager, so anything the
	 * builder produces would show up in the editor's Transcoder Job Monitor until GC. Drop test
	 * jobs from the registry on construction so they stay invisible to production UI.
	 */
	static UTmvMediaTranscodeJob* DetachJob(UTmvMediaTranscodeJob* InJob)
	{
		if (InJob)
		{
			ITmvMediaTranscodeJobManager::SafeUnregisterTranscodeJob(InJob);
		}
		return InJob;
	}
} // namespace UE::TmvMedia::Tests

// ---------------------------------------------------------------------------
// Test fixture
// ---------------------------------------------------------------------------

TEST_CLASS(FTmvMediaTranscodeJobBuilderTests, "System.Plugins.TmvMedia.TranscodeJobBuilder")
{
	// -----------------------------------------------------------------------
	// Producer selection
	// -----------------------------------------------------------------------

	TEST_METHOD(DefaultProducerIsMediaPlayerWhenSettingbUseMediaPlayerTrue)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeMinimalJobItem();
		Item.Settings.bUseMediaPlayer = true;

		UTmvMediaTranscodeJob* Job = DetachJob(FTmvMediaTranscodeJobBuilder(Item).Build());

		ASSERT_THAT(IsNotNull(Job));
		ASSERT_THAT(IsNotNull(Job->GetStage<UTmvMediaPlayerFrameProducer>()));
		// Confirm the wrong type is NOT present.
		ASSERT_THAT(IsNull(Job->GetStage<UTmvMediaFileSequenceFrameProducer>()));
	}

	TEST_METHOD(DefaultProducerIsFileSequenceWhenSettingbUseMediaPlayerFalse)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeMinimalJobItem();
		Item.Settings.bUseMediaPlayer = false;

		UTmvMediaTranscodeJob* Job = DetachJob(FTmvMediaTranscodeJobBuilder(Item).Build());

		ASSERT_THAT(IsNotNull(Job));
		ASSERT_THAT(IsNotNull(Job->GetStage<UTmvMediaFileSequenceFrameProducer>()));
		ASSERT_THAT(IsNull(Job->GetStage<UTmvMediaPlayerFrameProducer>()));
	}

	// -----------------------------------------------------------------------
	// Muxer selection
	// -----------------------------------------------------------------------

	TEST_METHOD(DefaultMuxerIsContainerWhenOutputFormatIsContainer)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeMinimalJobItem();
		Item.Settings.OutputFormat = ETmvMediaTranscodeOutputFormat::Container;

		UTmvMediaTranscodeJob* Job = DetachJob(FTmvMediaTranscodeJobBuilder(Item).Build());

		ASSERT_THAT(IsNotNull(Job));
		ASSERT_THAT(IsNotNull(Job->GetStage<UTmvMediaContainerTranscodeMuxer>()));
		ASSERT_THAT(IsNull(Job->GetStage<UTmvMediaFileSequenceTranscodeMuxer>()));
	}

	TEST_METHOD(DefaultMuxerIsFileSequenceWhenOutputFormatIsNotContainer)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeMinimalJobItem();
		Item.Settings.OutputFormat = ETmvMediaTranscodeOutputFormat::FileSequence;

		UTmvMediaTranscodeJob* Job = DetachJob(FTmvMediaTranscodeJobBuilder(Item).Build());

		ASSERT_THAT(IsNotNull(Job));
		ASSERT_THAT(IsNotNull(Job->GetStage<UTmvMediaFileSequenceTranscodeMuxer>()));
		ASSERT_THAT(IsNull(Job->GetStage<UTmvMediaContainerTranscodeMuxer>()));
	}

	// -----------------------------------------------------------------------
	// All four stage types must be present after Build (order is irrelevant: the
	// transcoding pipeline always retrieves stages by type via GetStage<T>()).
	// -----------------------------------------------------------------------

	TEST_METHOD(BuildAddsAllFourStageTypes)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeMinimalJobItem();

		UTmvMediaTranscodeJob* Job = DetachJob(FTmvMediaTranscodeJobBuilder(Item).Build());

		ASSERT_THAT(IsNotNull(Job));
		ASSERT_THAT(AreEqual(4, Job->Stages.Num()));

		// Each base type is queried via the same accessor production code uses.
		ASSERT_THAT(IsNotNull(Job->GetStage<UTmvMediaFrameProducer>()));
		ASSERT_THAT(IsNotNull(Job->GetStage<UTmvMediaFrameConverter>()));
		ASSERT_THAT(IsNotNull(Job->GetStage<UTmvMediaFrameEncoder>()));
		ASSERT_THAT(IsNotNull(Job->GetStage<UTmvMediaTranscodeMuxer>()));
	}

	// -----------------------------------------------------------------------
	// Override tests
	// -----------------------------------------------------------------------

	TEST_METHOD(OverrideProducerClassTakesPrecedence)
	{
		using namespace UE::TmvMedia::Tests;
		// Pass the abstract base class as the override (mirrors what MovieGraph does).
		// The builder must use the override rather than selecting a subclass from settings.
		FTmvMediaTranscodeListItem Item = MakeMinimalJobItem();
		Item.Settings.bUseMediaPlayer = true;  // would normally produce UTmvMediaPlayerFrameProducer

		UTmvMediaTranscodeJob* Job = DetachJob(FTmvMediaTranscodeJobBuilder(Item)
			.WithFrameProducerClass(UTmvMediaFrameProducer::StaticClass())
			.Build());

		ASSERT_THAT(IsNotNull(Job));

		// The producer stage's concrete class must be exactly UTmvMediaFrameProducer (the base),
		// not the UTmvMediaPlayerFrameProducer subclass that settings alone would pick.
		UTmvMediaFrameProducer* ProducerStage = Job->GetStage<UTmvMediaFrameProducer>();
		ASSERT_THAT(IsNotNull(ProducerStage));
		ASSERT_THAT(IsTrue(ProducerStage->GetClass() == UTmvMediaFrameProducer::StaticClass()));
		ASSERT_THAT(IsFalse(ProducerStage->GetClass()->IsChildOf(UTmvMediaPlayerFrameProducer::StaticClass())));
	}

	TEST_METHOD(OverrideMuxerClassTakesPrecedence)
	{
		using namespace UE::TmvMedia::Tests;
		// Use a test-only concrete subclass of the abstract base muxer. This isolates the test
		// from the production muxer subclasses: we know the builder used the override class
		// (and not the OutputFormat-driven default) precisely because no production code path
		// would ever instantiate UTmvMediaTestTranscodeMuxer.
		FTmvMediaTranscodeListItem Item = MakeMinimalJobItem();
		Item.Settings.OutputFormat = ETmvMediaTranscodeOutputFormat::FileSequence;

		UTmvMediaTranscodeJob* Job = DetachJob(FTmvMediaTranscodeJobBuilder(Item)
			.WithMuxerClass(UTmvMediaTestTranscodeMuxer::StaticClass())
			.Build());

		ASSERT_THAT(IsNotNull(Job));

		UTmvMediaTranscodeMuxer* MuxerStage = Job->GetStage<UTmvMediaTranscodeMuxer>();
		ASSERT_THAT(IsNotNull(MuxerStage));
		ASSERT_THAT(IsTrue(MuxerStage->GetClass() == UTmvMediaTestTranscodeMuxer::StaticClass()));
		// And confirm the OutputFormat-driven default was suppressed.
		ASSERT_THAT(IsFalse(MuxerStage->GetClass()->IsChildOf(UTmvMediaFileSequenceTranscodeMuxer::StaticClass())));
		ASSERT_THAT(IsFalse(MuxerStage->GetClass()->IsChildOf(UTmvMediaContainerTranscodeMuxer::StaticClass())));
	}

	// -----------------------------------------------------------------------
	// Encoder / OutputFormat compatibility validation.
	// The builder must refuse to build when the user picked Container output but the configured
	// encoder cannot be muxed (GetCodecFourCC() == 0). It must also populate OutErrors so callers
	// can surface the failure (e.g. via a toaster notification).
	// -----------------------------------------------------------------------

	TEST_METHOD(BuildRefusesContainerWhenEncoderHasNoFourCCAndPopulatesOutErrors)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeMinimalJobItem();
		Item.Settings.OutputFormat = ETmvMediaTranscodeOutputFormat::Container;

		// Instantiate the base FTmvMediaEncoderOptions to stand in for an encoder that doesn't
		// override GetCodecFourCC() (so it returns 0). Mirrors the EXR encoder's behavior without
		// pulling in a cross-module dep.
		Item.EncoderOptions.InitializeAsScriptStruct(FTmvMediaEncoderOptions::StaticStruct());

		// UE_TMV_MEDIA_MESSAGE_LOG emits an Error-level log, which CQTest otherwise reports as a
		// test failure. Register the expected substring so the log is silenced for this case.
		TestRunner->AddExpectedError(TEXT("cannot be muxed into a container"), EAutomationExpectedErrorFlags::Contains, 1);

		FTmvMediaMessageContext BuildErrors;
		UTmvMediaTranscodeJob* Job = DetachJob(FTmvMediaTranscodeJobBuilder(Item).Build(&BuildErrors));

		ASSERT_THAT(IsNull(Job));
		ASSERT_THAT(AreEqual(1, BuildErrors.Messages.Num()));
		// Sanity-check the error mentions the job name so a user reading the toast can correlate it.
		ASSERT_THAT(IsTrue(BuildErrors.Messages[0].ToString().Contains(Item.Name)));
	}

	// -----------------------------------------------------------------------
	// PostBuild delegate
	// -----------------------------------------------------------------------

	TEST_METHOD(PostBuildDelegateIsInvokedWithBuiltJob)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeMinimalJobItem();

		bool bDelegateWasCalled       = false;
		int32 StageCountAtCallTime    = 0;
		UTmvMediaTranscodeJob* DelegateJobPtr = nullptr;

		FOnTranscodeJobBuilt PostBuildDelegate = FOnTranscodeJobBuilt::CreateLambda(
			[&bDelegateWasCalled, &StageCountAtCallTime, &DelegateJobPtr](UTmvMediaTranscodeJob* InJob)
			{
				bDelegateWasCalled   = true;
				DelegateJobPtr       = InJob;
				StageCountAtCallTime = IsValid(InJob) ? InJob->Stages.Num() : 0;
			});

		UTmvMediaTranscodeJob* ReturnedJob = DetachJob(FTmvMediaTranscodeJobBuilder(Item)
			.OnPostBuild(MoveTemp(PostBuildDelegate))
			.Build());

		// Delegate must have fired exactly once.
		ASSERT_THAT(IsTrue(bDelegateWasCalled));

		// The job pointer the delegate received must be the same object Build() returned.
		ASSERT_THAT(IsNotNull(ReturnedJob));
		ASSERT_THAT(IsTrue(DelegateJobPtr == ReturnedJob));

		// All four stages must already be present at the time the delegate fires.
		// (Producer, converter, encoder, muxer — the delegate is called after SetEncoderOptions.)
		ASSERT_THAT(AreEqual(4, StageCountAtCallTime));
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS
