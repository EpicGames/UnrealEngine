// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "CQTest.h"

#include "Misc/Guid.h"
#include "Transcoder/ITmvMediaTranscodeJobManager.h"
#include "Transcoder/ITmvMediaTranscodeJobRunner.h"
#include "Transcoder/TmvMediaFrameProducer.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeJobRunner.h"
#include "Transcoder/TmvMediaTranscodeStage.h"

namespace UE::TmvMedia::Tests
{
	/**
	 * UTmvMediaTranscodeJob's constructor (and SetId) call SafeRegisterTranscodeJob, which means
	 * any NewObject<UTmvMediaTranscodeJob> shows up in the editor's Transcoder Job Monitor until
	 * GC reclaims it. Pull test jobs out of the global registry immediately so they remain
	 * invisible to production UI.
	 */
	static void DetachJobFromGlobalManager(UTmvMediaTranscodeJob* InJob)
	{
		ITmvMediaTranscodeJobManager::SafeUnregisterTranscodeJob(InJob);
	}

	/**
	 * Build a job with a single base FrameProducer stage. The default stage Start/RequestStop
	 * complete synchronously, so the job is fully drivable from the test without any real
	 * media: Start() returns true, the stage idles in Started, and a manual SetStageStatus
	 * lets the test signal completion when it wants Tick to discard the job.
	 */
	static UTmvMediaTranscodeJob* MakeRunnableJob(const TCHAR* InName = TEXT("Runner.Test.Job"))
	{
		UTmvMediaTranscodeJob* Job = NewObject<UTmvMediaTranscodeJob>();
		Job->JobName = InName;
		Job->SetId(FGuid::NewGuid());
		DetachJobFromGlobalManager(Job);
		UTmvMediaFrameProducer* Producer = NewObject<UTmvMediaFrameProducer>(Job);
		Job->AddStage(Producer);
		return Job;
	}

	/**
	 * Build a job with no FrameProducer stage. UTmvMediaTranscodeJob::Start returns false
	 * synchronously in that case, which is exactly the sync-fail path the runner has to handle.
	 */
	static UTmvMediaTranscodeJob* MakeUnstartableJob(const TCHAR* InName = TEXT("Runner.Test.UnstartableJob"))
	{
		UTmvMediaTranscodeJob* Job = NewObject<UTmvMediaTranscodeJob>();
		Job->JobName = InName;
		Job->SetId(FGuid::NewGuid());
		DetachJobFromGlobalManager(Job);
		return Job;
	}
} // namespace UE::TmvMedia::Tests

// ---------------------------------------------------------------------------
// Test fixture
//
// Each test gets a fresh FTmvMediaTranscodeJobRunner instance. The runner is a
// FTickableGameObject, so the engine will tick it between latent commands; tests
// that want explicit control call Runner->Tick() directly. The shared delegate
// counters reset automatically between TEST_METHOD invocations.
// ---------------------------------------------------------------------------

TEST_CLASS(FTmvMediaTranscodeJobRunnerTests, "System.Plugins.TmvMedia.TranscodeJobRunner")
{
	TUniquePtr<FTmvMediaTranscodeJobRunner> Runner;
	int32 OnJobStartedCount = 0;
	int32 OnJobFinishedCount = 0;
	int32 OnAllJobsFinishedCount = 0;

	BEFORE_EACH()
	{
		Runner = MakeUnique<FTmvMediaTranscodeJobRunner>();
		Runner->GetOnJobStarted().AddLambda([this](UTmvMediaTranscodeJob*) { ++OnJobStartedCount; });
		Runner->GetOnJobFinished().AddLambda([this](UTmvMediaTranscodeJob*) { ++OnJobFinishedCount; });
		Runner->GetOnAllJobsFinished().AddLambda([this]() { ++OnAllJobsFinishedCount; });
	}

	AFTER_EACH()
	{
		// Destructor runs CancelAll; broadcasts go to the lambdas above, which die with the runner.
		Runner.Reset();
	}

	// -----------------------------------------------------------------------
	// EnqueueJob
	// -----------------------------------------------------------------------

	TEST_METHOD(EnqueueJob_AutoStartsWhenIdle)
	{
		using namespace UE::TmvMedia::Tests;
		UTmvMediaTranscodeJob* Job = MakeRunnableJob();

		Runner->EnqueueJob(Job);

		ASSERT_THAT(IsTrue(Runner->HasActiveOrPendingJobs()));
		ASSERT_THAT(IsTrue(Runner->IsJobActiveOrPending(Job->GetId())));
		ASSERT_THAT(AreEqual(1, OnJobStartedCount));
		ASSERT_THAT(AreEqual(0, OnJobFinishedCount));
		ASSERT_THAT(AreEqual(0, OnAllJobsFinishedCount));
	}

	TEST_METHOD(EnqueueMultipleJobs_StartsFirstAndQueuesRest)
	{
		using namespace UE::TmvMedia::Tests;
		UTmvMediaTranscodeJob* JobA = MakeRunnableJob(TEXT("A"));
		UTmvMediaTranscodeJob* JobB = MakeRunnableJob(TEXT("B"));
		UTmvMediaTranscodeJob* JobC = MakeRunnableJob(TEXT("C"));

		Runner->EnqueueJob(JobA);
		Runner->EnqueueJob(JobB);
		Runner->EnqueueJob(JobC);

		// Only one OnJobStarted broadcast — pending jobs do not start until the active job completes.
		ASSERT_THAT(AreEqual(1, OnJobStartedCount));
		ASSERT_THAT(IsTrue(Runner->IsJobActiveOrPending(JobA->GetId())));
		ASSERT_THAT(IsTrue(Runner->IsJobActiveOrPending(JobB->GetId())));
		ASSERT_THAT(IsTrue(Runner->IsJobActiveOrPending(JobC->GetId())));
	}

	TEST_METHOD(EnqueueJob_NullIsNoOp)
	{
		Runner->EnqueueJob(nullptr);

		ASSERT_THAT(IsFalse(Runner->HasActiveOrPendingJobs()));
		ASSERT_THAT(AreEqual(0, OnJobStartedCount));
		ASSERT_THAT(AreEqual(0, OnJobFinishedCount));
	}

	// -----------------------------------------------------------------------
	// CancelAll
	// -----------------------------------------------------------------------

	TEST_METHOD(CancelAll_DrainsActiveAndPendingJobs)
	{
		using namespace UE::TmvMedia::Tests;
		Runner->EnqueueJob(MakeRunnableJob(TEXT("Active")));
		Runner->EnqueueJob(MakeRunnableJob(TEXT("PendingA")));
		Runner->EnqueueJob(MakeRunnableJob(TEXT("PendingB")));
		ASSERT_THAT(IsTrue(Runner->HasActiveOrPendingJobs()));

		Runner->CancelAll();

		ASSERT_THAT(IsFalse(Runner->HasActiveOrPendingJobs()));
		// Active + 2 pending → 3 OnJobFinished broadcasts.
		ASSERT_THAT(AreEqual(3, OnJobFinishedCount));
		// CancelAll does not broadcast OnAllJobsFinished — that delegate fires only from Tick().
		ASSERT_THAT(AreEqual(0, OnAllJobsFinishedCount));
	}

	// -----------------------------------------------------------------------
	// CancelJob (by FGuid)
	// -----------------------------------------------------------------------

	TEST_METHOD(CancelJob_RemovesPendingJobByIdAndLeavesActiveRunning)
	{
		using namespace UE::TmvMedia::Tests;
		UTmvMediaTranscodeJob* Active = MakeRunnableJob(TEXT("Active"));
		UTmvMediaTranscodeJob* Pending = MakeRunnableJob(TEXT("Pending"));
		Runner->EnqueueJob(Active);
		Runner->EnqueueJob(Pending);
		const FGuid PendingId = Pending->GetId();
		const FGuid ActiveId = Active->GetId();

		Runner->CancelJob(PendingId);

		ASSERT_THAT(IsFalse(Runner->IsJobActiveOrPending(PendingId)));
		ASSERT_THAT(IsTrue(Runner->IsJobActiveOrPending(ActiveId)));
		ASSERT_THAT(AreEqual(1, OnJobFinishedCount));
		ASSERT_THAT(AreEqual(0, OnAllJobsFinishedCount));
	}

	TEST_METHOD(CancelJob_StopsActiveJobByIdAndDiscardsSynchronously)
	{
		using namespace UE::TmvMedia::Tests;
		UTmvMediaTranscodeJob* Job = MakeRunnableJob();
		Runner->EnqueueJob(Job);

		Runner->CancelJob(Job->GetId());

		// CancelJob's active path is symmetric with CancelAll and with the pending branch:
		// the job is RequestStop'd, finalized, and OnJobFinished broadcast synchronously.
		// IsJobActiveOrPending returns false on return without needing a follow-up Tick().
		ASSERT_THAT(IsFalse(Runner->HasActiveOrPendingJobs()));
		ASSERT_THAT(AreEqual(1, OnJobFinishedCount));
		ASSERT_THAT(AreEqual(1, OnAllJobsFinishedCount));
	}

	TEST_METHOD(CancelJob_ActiveJobAdvancesToNextPending)
	{
		using namespace UE::TmvMedia::Tests;
		UTmvMediaTranscodeJob* Active = MakeRunnableJob(TEXT("Active"));
		UTmvMediaTranscodeJob* Pending = MakeRunnableJob(TEXT("Pending"));
		Runner->EnqueueJob(Active);
		Runner->EnqueueJob(Pending);
		ASSERT_THAT(AreEqual(1, OnJobStartedCount));

		Runner->CancelJob(Active->GetId());

		// After finalizing the cancelled active job synchronously, the runner promotes the
		// next pending entry — otherwise it would stall (IsTickable gates on CurrentJob.IsValid()).
		ASSERT_THAT(IsFalse(Runner->IsJobActiveOrPending(Active->GetId())));
		ASSERT_THAT(IsTrue(Runner->IsJobActiveOrPending(Pending->GetId())));
		ASSERT_THAT(AreEqual(2, OnJobStartedCount));
		ASSERT_THAT(AreEqual(1, OnJobFinishedCount));
		ASSERT_THAT(AreEqual(0, OnAllJobsFinishedCount));
	}

	TEST_METHOD(CancelJob_UnknownIdIsNoOp)
	{
		using namespace UE::TmvMedia::Tests;
		Runner->EnqueueJob(MakeRunnableJob());

		Runner->CancelJob(FGuid::NewGuid()); // never enqueued

		ASSERT_THAT(IsTrue(Runner->HasActiveOrPendingJobs()));
		ASSERT_THAT(AreEqual(0, OnJobFinishedCount));
	}

	// -----------------------------------------------------------------------
	// Tick / completion flow
	// -----------------------------------------------------------------------

	TEST_METHOD(Tick_CompletesJobAndBroadcastsOnAllJobsFinished)
	{
		using namespace UE::TmvMedia::Tests;
		UTmvMediaTranscodeJob* Job = MakeRunnableJob();
		Runner->EnqueueJob(Job);

		// Force the producer to Stopped; Tick will mirror it across all stages and complete the job.
		UTmvMediaFrameProducer* Producer = Job->GetStage<UTmvMediaFrameProducer>();
		ASSERT_THAT(IsNotNull(Producer));
		Producer->SetStageStatus(ETmvMediaTranscodeStageStatus::Stopped, Job);

		Runner->Tick(0.0f);

		ASSERT_THAT(IsFalse(Runner->HasActiveOrPendingJobs()));
		ASSERT_THAT(AreEqual(1, OnJobStartedCount));
		ASSERT_THAT(AreEqual(1, OnJobFinishedCount));
		ASSERT_THAT(AreEqual(1, OnAllJobsFinishedCount));
	}

	TEST_METHOD(Tick_StartsNextPendingJobAfterCurrentCompletes)
	{
		using namespace UE::TmvMedia::Tests;
		UTmvMediaTranscodeJob* JobA = MakeRunnableJob(TEXT("A"));
		UTmvMediaTranscodeJob* JobB = MakeRunnableJob(TEXT("B"));
		Runner->EnqueueJob(JobA);
		Runner->EnqueueJob(JobB);
		ASSERT_THAT(AreEqual(1, OnJobStartedCount));

		// Complete A.
		JobA->GetStage<UTmvMediaFrameProducer>()->SetStageStatus(ETmvMediaTranscodeStageStatus::Stopped, JobA);
		Runner->Tick(0.0f);

		// B should be promoted to active. OnAllJobsFinished must NOT have fired because the queue is not empty.
		ASSERT_THAT(IsTrue(Runner->IsJobActiveOrPending(JobB->GetId())));
		ASSERT_THAT(IsFalse(Runner->IsJobActiveOrPending(JobA->GetId())));
		ASSERT_THAT(AreEqual(2, OnJobStartedCount));
		ASSERT_THAT(AreEqual(1, OnJobFinishedCount));
		ASSERT_THAT(AreEqual(0, OnAllJobsFinishedCount));
	}

	// -----------------------------------------------------------------------
	// Synchronous Start() failure
	// -----------------------------------------------------------------------

	TEST_METHOD(SyncStartFailure_BroadcastsOnJobFinishedButNotOnAllJobsFinished)
	{
		using namespace UE::TmvMedia::Tests;
		// Two error logs are expected for this path: the job's own "missing Frame Producer" error
		// from UTmvMediaTranscodeJob::Start, then the runner's "Skipping" follow-up.
		this->Assert.ExpectError(TEXT("missing the Frame Producer stage"));
		this->Assert.ExpectError(TEXT("Skipping"));

		Runner->EnqueueJob(MakeUnstartableJob());

		// The contract on GetOnAllJobsFinished states the delegate fires only from Tick(),
		// never from EnqueueJob draining the queue synchronously. The commandlet's exit
		// handler depends on that.
		ASSERT_THAT(IsFalse(Runner->HasActiveOrPendingJobs()));
		ASSERT_THAT(AreEqual(0, OnJobStartedCount));
		ASSERT_THAT(AreEqual(1, OnJobFinishedCount));
		ASSERT_THAT(AreEqual(0, OnAllJobsFinishedCount));
	}

	// -----------------------------------------------------------------------
	// Timeout watchdog
	// -----------------------------------------------------------------------

	TEST_METHOD(Tick_EnforcesTimeoutByRequestingStopOnLongRunningJob)
	{
		using namespace UE::TmvMedia::Tests;
		// The watchdog emits two error logs: "exceeded timeout" when it fires, then
		// "aborted after timeout" once DiscardCurrentJob runs in the same Tick.
		this->Assert.ExpectError(TEXT("exceeded timeout"));
		this->Assert.ExpectError(TEXT("aborted after timeout"));

		UTmvMediaTranscodeJob* Job = MakeRunnableJob();
		FTmvMediaTranscodeJobRunOptions Options;
		Options.TimeoutSeconds = 0.05; // 50ms

		Runner->EnqueueJob(Job, Options);
		const FGuid JobId = Job->GetId();
		ASSERT_THAT(IsTrue(Runner->HasActiveOrPendingJobs()));

		// Poll Tick until the watchdog cancels the job. FApp::GetCurrentTime advances each frame,
		// so the timeout fires once (CurrentTime - CurrentJobStartTime) exceeds TimeoutSeconds.
		// Upper-bound the wait at 2s to keep the test responsive if the watchdog regresses.
		TestCommandBuilder
			.Until([this, JobId]()
				{
					Runner->Tick(0.0f);
					return !Runner->IsJobActiveOrPending(JobId);
				},
				FTimespan::FromSeconds(2.0))
			.Then([this]()
				{
					ASSERT_THAT(AreEqual(1, OnJobFinishedCount));
					ASSERT_THAT(AreEqual(1, OnAllJobsFinishedCount));
				});
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR
