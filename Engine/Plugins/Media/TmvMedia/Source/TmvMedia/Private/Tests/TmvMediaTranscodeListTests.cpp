// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "CQTest.h"

#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeList.h"

namespace UE::TmvMedia::Tests
{
	/**
	 * Builds a minimally-valid FTmvMediaTranscodeListItem.
	 * Both InputPath.FilePath and OutputPath.Path are set so ValidateJobItem returns true.
	 * bMakeOutputAsset is false by default, satisfying the third validation check.
	 */
	static FTmvMediaTranscodeListItem MakeValidJobItem()
	{
		FTmvMediaTranscodeListItem Item;
		Item.Name = TEXT("ValidTestJob");
		Item.Settings.InputPath.FilePath = TEXT("/TestInput/source.exr");
		Item.Settings.OutputPath.Path   = TEXT("/TestOutput/");
		Item.Settings.bMakeOutputAsset  = false;
		return Item;
	}
} // namespace UE::TmvMedia::Tests

// ---------------------------------------------------------------------------
// Test fixture for UTmvMediaTranscodeList::ValidateJobItem
// ---------------------------------------------------------------------------

TEST_CLASS(FTmvMediaTranscodeListValidateJobItemTests, "System.Plugins.TmvMedia.TranscodeList")
{
	// -----------------------------------------------------------------------
	// Happy path
	// -----------------------------------------------------------------------

	TEST_METHOD(FullyValidItemPasses)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeValidJobItem();

		FString OutError;
		const bool bResult = UTmvMediaTranscodeList::ValidateJobItem(Item, &OutError);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(IsTrue(OutError.IsEmpty()));
	}

	// -----------------------------------------------------------------------
	// Individual failure paths
	// -----------------------------------------------------------------------

	TEST_METHOD(MissingInputPathFailsWithInputReason)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeValidJobItem();
		// Clear only the input path; output path remains valid.
		Item.Settings.InputPath.FilePath = TEXT("");

		FString OutError;
		const bool bResult = UTmvMediaTranscodeList::ValidateJobItem(Item, &OutError);

		ASSERT_THAT(IsFalse(bResult));
		ASSERT_THAT(IsTrue(OutError.Contains(TEXT("Input path is not set."))));
		// Output path is set, so the output-path reason must NOT be present.
		ASSERT_THAT(IsFalse(OutError.Contains(TEXT("Output path is not set."))));
	}

	TEST_METHOD(MissingOutputPathFailsWithOutputReason)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeValidJobItem();
		// Clear only the output path; input path remains valid.
		Item.Settings.OutputPath.Path = TEXT("");

		FString OutError;
		const bool bResult = UTmvMediaTranscodeList::ValidateJobItem(Item, &OutError);

		ASSERT_THAT(IsFalse(bResult));
		ASSERT_THAT(IsTrue(OutError.Contains(TEXT("Output path is not set."))));
		ASSERT_THAT(IsFalse(OutError.Contains(TEXT("Input path is not set."))));
	}

	TEST_METHOD(MakeOutputAssetWithoutDirectoryFailsWithAssetReason)
	{
		using namespace UE::TmvMedia::Tests;
		FTmvMediaTranscodeListItem Item = MakeValidJobItem();
		// Both input and output paths are set, so the first two checks pass.
		// Trigger only the third check.
		Item.Settings.bMakeOutputAsset              = true;
		Item.Settings.OutputAssetDirectory.Path     = TEXT(""); // intentionally empty

		FString OutError;
		const bool bResult = UTmvMediaTranscodeList::ValidateJobItem(Item, &OutError);

		ASSERT_THAT(IsFalse(bResult));
		ASSERT_THAT(IsTrue(OutError.Contains(TEXT("bMakeOutputAsset is set but OutputAssetDirectory is empty."))));
		ASSERT_THAT(IsFalse(OutError.Contains(TEXT("Input path is not set."))));
		ASSERT_THAT(IsFalse(OutError.Contains(TEXT("Output path is not set."))));
	}

	// -----------------------------------------------------------------------
	// Multiple failures accumulate
	// -----------------------------------------------------------------------

	TEST_METHOD(EmptySettingsFailsWithAllThreeReasons)
	{
		// Default-constructed item: InputPath.FilePath is empty, OutputPath.Path is empty.
		// Set bMakeOutputAsset=true and leave OutputAssetDirectory.Path empty to trigger the third check.
		FTmvMediaTranscodeListItem Item;
		Item.Settings.bMakeOutputAsset          = true;
		Item.Settings.OutputAssetDirectory.Path = TEXT(""); // empty

		FString OutError;
		const bool bResult = UTmvMediaTranscodeList::ValidateJobItem(Item, &OutError);

		ASSERT_THAT(IsFalse(bResult));
		ASSERT_THAT(IsTrue(OutError.Contains(TEXT("Input path is not set."))));
		ASSERT_THAT(IsTrue(OutError.Contains(TEXT("Output path is not set."))));
		ASSERT_THAT(IsTrue(OutError.Contains(TEXT("bMakeOutputAsset is set but OutputAssetDirectory is empty."))));
	}

	// -----------------------------------------------------------------------
	// Null OutError safety
	// -----------------------------------------------------------------------

	TEST_METHOD(NullOutErrorIsHandled)
	{
		// A default-constructed item (no paths set) with a null OutError must not crash,
		// and must still return the correct bool.
		FTmvMediaTranscodeListItem Item;

		// Pass nullptr — should not crash.
		const bool bResult = UTmvMediaTranscodeList::ValidateJobItem(Item, nullptr);

		ASSERT_THAT(IsFalse(bResult));
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS
