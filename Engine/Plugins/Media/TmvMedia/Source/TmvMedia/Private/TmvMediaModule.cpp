// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decoder/ITmvMediaDecoderFactory.h"
#include "Decoder/ITmvMediaDemuxerFactory.h"
#include "Encoder/ITmvMediaEncoderFactory.h"
#include "Encoder/ITmvMediaMuxerFactory.h"
#include "ITmvMediaModule.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "SampleConverter/TmvSharedTilePool.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaTranscodeJobManager.h"
#include "Transcoder/TmvMediaTranscodeJobRunner.h"

DEFINE_LOG_CATEGORY(LogTmvMedia);

class FTmvMediaModule : public ITmvMediaModule
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		TranscodeJobManager = MakeShared<FTmvMediaTranscodeJobManager>();
		TranscodeJobRunner = MakeShared<FTmvMediaTranscodeJobRunner>();

		// Backstop for the shared tile pool: pools are normally dropped as each
		// clip's last owner unregisters (FTmvSharedTilePoolManager::ReleaseOwner),
		// but free any that survive (e.g. a clip still open at exit) while the RHI
		// is still alive, rather than at static-destruction time.
		EnginePreExitHandle = FCoreDelegates::OnEnginePreExit.AddLambda([]()
		{
			FTmvSharedTilePoolManager::Get().ReleaseAll();
		});
	}

	virtual void ShutdownModule() override
	{
		if (EnginePreExitHandle.IsValid())
		{
			FCoreDelegates::OnEnginePreExit.Remove(EnginePreExitHandle);
			EnginePreExitHandle.Reset();
		}
		FTmvSharedTilePoolManager::Get().ReleaseAll();

		TranscodeJobRunner.Reset();
		TranscodeJobManager.Reset();
	}
	//~ End IModuleInterface

	//~ Begin ITmvMediaModule
	virtual void RegisterDecoderFactory(TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> InDecoderFactory) override
	{
		if (InDecoderFactory.IsValid())
		{
			FScopeLock Lock(&FactoriesCS);
			if (DecoderFactoryNames.Contains(InDecoderFactory->GetName()))
			{
				UE_LOGF(LogTmvMedia, Error, "Failed to add decoder factory \"%ls\", it is already registered.", *InDecoderFactory->GetName());
				return;
			}

			DecoderFactoryNames.Add(InDecoderFactory->GetName());
			DecoderFactories.Emplace(MoveTemp(InDecoderFactory));
		}
	}
	
	virtual void UnregisterDecoderFactory(TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> InDecoderFactory) override
	{
		if (InDecoderFactory.IsValid())
		{
			FScopeLock Lock(&FactoriesCS);
			DecoderFactoryNames.Remove(InDecoderFactory->GetName());
			DecoderFactories.RemoveSingleSwap(InDecoderFactory);
		}
	}
	
	virtual void GetDecoderFactories(TArray<TWeakPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe>>& OutDecoderFactories) const override
	{
		FScopeLock Lock(&FactoriesCS);
		OutDecoderFactories.Append(DecoderFactories);
	}
	
	virtual TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> GetBestDecoderFactoryForFormat(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions) const override
	{
		TArray<TWeakPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe>> Factories;
		GetDecoderFactories(Factories);
		
		// Get the list of registered factories
		TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> BestFactory;
		int32 BestPriority = 0;
		for(const TWeakPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe>& FactoryWeak : Factories)
		{
			TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> Factory = FactoryWeak.Pin();
			int32 SupportedPriority = Factory.IsValid() ? Factory->SupportsFormat(InCodecFormat, InOptions) : 0;
			if (SupportedPriority > BestPriority)
			{
				BestPriority = SupportedPriority;
				BestFactory = Factory;
			}
		}
		return BestFactory;
	}

	virtual void RegisterEncoderFactory(TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> InEncoderFactory) override
	{
		if (InEncoderFactory.IsValid())
		{
			FScopeLock Lock(&FactoriesCS);
			if (EncoderFactoryNames.Contains(InEncoderFactory->GetName()))
			{
				UE_LOGF(LogTmvMedia, Error, "Failed to add encoder factory \"%ls\", it is already registered.", *InEncoderFactory->GetName().ToString());
				return;
			}
			EncoderFactoryNames.Add(InEncoderFactory->GetName());
			EncoderFactories.Emplace(MoveTemp(InEncoderFactory));
		}
	}

	virtual void UnregisterEncoderFactory(TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> InEncoderFactory) override
	{
		if (InEncoderFactory.IsValid())
		{
			FScopeLock Lock(&FactoriesCS);
			EncoderFactoryNames.Remove(InEncoderFactory->GetName());
			EncoderFactories.RemoveSingleSwap(InEncoderFactory);
		}
	}

	virtual void GetEncoderFactories(TArray<TWeakPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe>>& OutEncoderFactories) const override
	{
		FScopeLock Lock(&FactoriesCS);
		OutEncoderFactories.Append(EncoderFactories);
	}

	virtual TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> FindEncoderFactory(FName InEncoderFactoryName) const override
	{
		FScopeLock Lock(&FactoriesCS);
		for (TWeakPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> EncoderFactoryWeak : EncoderFactories)
		{
			if (TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> EncoderFactory = EncoderFactoryWeak.Pin())
			{
				if (EncoderFactory->GetName() == InEncoderFactoryName)
				{
					return EncoderFactory;
				}
			}
		}
		return nullptr;
	}

	virtual void RegisterMuxerFactory(TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> InMuxerFactory) override
	{
		if (InMuxerFactory.IsValid())
		{
			FScopeLock Lock(&FactoriesCS);
			if (MuxerFactoryNames.Contains(InMuxerFactory->GetName()))
			{
				UE_LOGF(LogTmvMedia, Error, "Failed to add muxer factory \"%ls\", it is already registered.", *InMuxerFactory->GetName().ToString());
				return;
			}
			MuxerFactoryNames.Add(InMuxerFactory->GetName());
			MuxerFactories.Emplace(MoveTemp(InMuxerFactory));
		}
	}

	virtual void UnregisterMuxerFactory(TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> InMuxerFactory) override
	{
		if (InMuxerFactory.IsValid())
		{
			FScopeLock Lock(&FactoriesCS);
			MuxerFactoryNames.Remove(InMuxerFactory->GetName());
			MuxerFactories.RemoveSingleSwap(InMuxerFactory);
		}
	}

	virtual void GetMuxerFactories(TArray<TWeakPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>>& OutMuxerFactories) const override
	{
		FScopeLock Lock(&FactoriesCS);
		OutMuxerFactories.Append(MuxerFactories);
	}

	virtual void RegisterDemuxerFactory(TSharedPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe> InDemuxerFactory) override
	{
		if (InDemuxerFactory.IsValid())
		{
			FScopeLock Lock(&FactoriesCS);
			if (DemuxerFactoryNames.Contains(InDemuxerFactory->GetName()))
			{
				UE_LOGF(LogTmvMedia, Error, "Failed to add demuxer factory \"%ls\", it is already registered.", *InDemuxerFactory->GetName().ToString());
				return;
			}
			DemuxerFactoryNames.Add(InDemuxerFactory->GetName());
			DemuxerFactories.Emplace(MoveTemp(InDemuxerFactory));
		}
	}

	virtual void UnregisterDemuxerFactory(TSharedPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe> InDemuxerFactory) override
	{
		if (InDemuxerFactory.IsValid())
		{
			FScopeLock Lock(&FactoriesCS);
			DemuxerFactoryNames.Remove(InDemuxerFactory->GetName());
			DemuxerFactories.RemoveSingleSwap(InDemuxerFactory);
		}
	}

	virtual void GetDemuxerFactories(TArray<TWeakPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe>>& OutDemuxerFactories) const override
	{
		FScopeLock Lock(&FactoriesCS);
		OutDemuxerFactories.Append(DemuxerFactories);
	}
	
	virtual TSharedPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe> FindDemuxerFactoryForExtension(const FString& InFileExtension) const override
	{
		FScopeLock Lock(&FactoriesCS);
		for (const TWeakPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe>& FactoryWeak : DemuxerFactories)
		{
			if (TSharedPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe> Factory = FactoryWeak.Pin())
			{
				if (Factory->GetSupportedContainerFormats().Contains(InFileExtension))
				{
					return Factory;
				}
			}
		}
		return nullptr;
	}

	virtual ITmvMediaTranscodeJobManager* GetTranscodeJobManager() const override
	{
		return TranscodeJobManager.Get();
	}

	virtual ITmvMediaTranscodeJobRunner* GetTranscodeJobRunner() const override
	{
		return TranscodeJobRunner.Get();
	}
	//~ End ITmvMediaModule

private:
	mutable FCriticalSection FactoriesCS;
	TArray<TWeakPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe>> DecoderFactories;
	TSet<FString> DecoderFactoryNames;
	TArray<TWeakPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe>> EncoderFactories;
	TSet<FName> EncoderFactoryNames;
	TArray<TWeakPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>> MuxerFactories;
	TSet<FName> MuxerFactoryNames;
	TArray<TWeakPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe>> DemuxerFactories;
	TSet<FName> DemuxerFactoryNames;

	TSharedPtr<FTmvMediaTranscodeJobManager> TranscodeJobManager;
	TSharedPtr<FTmvMediaTranscodeJobRunner> TranscodeJobRunner;

	FDelegateHandle EnginePreExitHandle;
};

ITmvMediaModule* ITmvMediaModule::Get()
{
	return FModuleManager::GetModulePtr<FTmvMediaModule>("TmvMedia");
}

ITmvMediaModule& ITmvMediaModule::GetOrLoad()
{
	return FModuleManager::LoadModuleChecked<FTmvMediaModule>("TmvMedia");
}

IMPLEMENT_MODULE(FTmvMediaModule, TmvMedia);