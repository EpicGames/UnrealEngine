// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaContainerTranscodeMuxer.h"

#include "Encoder/ITmvMediaMuxerFactory.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "ITmvMediaModule.h"
#include "Misc/Paths.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Transcoder/TmvMediaFrameProducer.h"
#include "Transcoder/TmvMediaTranscodeJob.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaContainerTranscodeMuxer)

#define LOCTEXT_NAMESPACE "TmvMediaContainerTranscodeMuxer"

static TAutoConsoleVariable<int32> CVarTmvTranscoderMuxerOutputMode(
	TEXT("TmvTranscoder.MuxerOutputMode"),
	0,
	TEXT("Output mode for the container muxer.\n")
	TEXT("  0: Standard (moov at end)\n")
	TEXT("  1: WebOptimized (moov at front, requires a temp file + 2nd pass)\n")
	TEXT("  2: Fragmented (moof/mdat interleaved)\n"),
	ECVF_Default);

UTmvMediaContainerTranscodeMuxer::UTmvMediaContainerTranscodeMuxer()
	: MuxerFinishedEvent(EEventMode::ManualReset)
{
}

bool UTmvMediaContainerTranscodeMuxer::Start(UTmvMediaTranscodeJob* InParentJob)
{
	if (!InParentJob)
	{
		UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::Start: Invalid parent job.");
		return false;
	}

	bStopRequested = false;
	MuxerFinishedEvent->Reset();

	// Find the specified muxer factory.
	const FName MuxerName(InParentJob->Settings.Muxer.Name);
	const TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> MuxerFactory = FindMuxerFactoryByName(MuxerName);
	if (!MuxerFactory.IsValid())
	{
		const FText Message = FText::Format(LOCTEXT("NoMuxerFactory", "No container muxer factory registered for \"{0}\"."), FText::FromName(MuxerName));
		UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::Start: %ls", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return false;
	}

	Muxer = MuxerFactory->CreateMuxer();
	if (!Muxer.IsValid())
	{
		const FText Message = LOCTEXT("CreateMuxerFailed", "Failed to create container muxer.");
		UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::Start: %ls", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return false;
	}

	// Configure the muxer with output file path.
	const FString OutputFilename = GetContainerOutputFilePath(InParentJob, MuxerFactory);
	const FString OutPath = FPaths::GetPath(OutputFilename);

	// Ensure output directory exists.
	if (!FPaths::DirectoryExists(OutPath))
	{
		if (!FPlatformFileManager::Get().GetPlatformFile().CreateDirectoryTree(*OutPath))
		{
			const FText Message = FText::Format(LOCTEXT("FailedCreateOutputDir", "Failed to create output directory \"{0}\""), FText::FromString(OutPath));
			UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::Start: %ls", *Message.ToString());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			return false;
		}
	}

	FTmvMediaMuxerConfig Config;
	Config.OutputFilename = OutputFilename;
	switch (CVarTmvTranscoderMuxerOutputMode.GetValueOnAnyThread())
	{
	case 1:
		Config.OutputMode = FTmvMediaMuxerConfig::EOutputMode::WebOptimized;
		Config.TemporaryFilename = OutputFilename + TEXT(".tmp");
		break;
	case 2:
		Config.OutputMode = FTmvMediaMuxerConfig::EOutputMode::Fragmented;
		break;
	case 0:
	default:
		Config.OutputMode = FTmvMediaMuxerConfig::EOutputMode::Standard;
		break;
	}

	if (Muxer->Configure(Config) != ETmvMediaContainerResult::Success)
	{
		const FText Message = FText::Format(LOCTEXT("ConfigureFailed", "Failed to configure container muxer: {0}"), FText::FromString(Muxer->GetLastError()));
		UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::Start: %ls", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return false;
	}

	return Super::Start(InParentJob);
}

void UTmvMediaContainerTranscodeMuxer::RequestStop(UTmvMediaTranscodeJob* InParentJob)
{
	bStopRequested = true;

	if (Muxer.IsValid() && bMuxerStarted)
	{
		// Flush any remaining buffered samples now that bStopRequested is true,
		// which will mark the last deliverable sample with bIsFinalSample.
		// For any pending requests that can't be fulfilled from the buffer
		// (worker already consumed the last real sample and is asking for the next),
		// send an empty final sample to unblock the worker.
		{
			FScopeLock Lock(&SampleBufferCS);
			FlushPendingRequests();

			for (const ITmvMediaMuxer::FSampleRequest& Request : PendingRequests)
			{
				FTmvMediaMuxerSample FinalSample;
				FinalSample.SampleNumber = Request.SampleNumber;
				FinalSample.bIsFinalSample = true;
				if (Muxer->AddSample(Request.TrackIndex, FinalSample) != ETmvMediaContainerResult::Success)
				{
					UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::RequestStop: Failed to add final sample for track %d: %ls", Request.TrackIndex, *Muxer->GetLastError());
				}
			}
			PendingRequests.Empty();
		}
	}

	if (Muxer.IsValid() && bMuxerStarted)
	{
		// Wait for the muxer worker to finish writing all samples.
		// The status delegate signals MuxerFinishedEvent on Finished or Failed.
		constexpr uint32 TimeoutMs = 30000;
		bool bTimedOut = false;
		if (!MuxerFinishedEvent->Wait(TimeoutMs))
		{
			UE_LOGF(LogTmvMedia, Warning, "ContainerTranscodeMuxer::RequestStop: Timed out waiting for muxer to finish. Aborting.");
			bTimedOut = true;
		}

		if (Muxer->Finalize() != ETmvMediaContainerResult::Success)
		{
			UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::RequestStop: Finalize failed: %ls", *Muxer->GetLastError());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, FText::FromString(Muxer->GetLastError()));
		}
		else if (bTimedOut)
		{
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, LOCTEXT("ErrorMuxerTimeout", "Timed out waiting for muxer to finish. Aborting."));
		}

		bMuxerStarted = false;
		bMuxerFailed = false;
	}

	if (Muxer.IsValid())
	{
		Muxer.Reset();
	}

	ContainerStreams.Empty();
	StreamBuffers.Empty();
	MuxerTrackToStreamIndex.Empty();
	PendingRequests.Empty();

	Super::RequestStop(InParentJob);
}

int32 UTmvMediaContainerTranscodeMuxer::OpenStream(UTmvMediaTranscodeJob* InParentJob, const FString& InStreamName, const FString& InExtension)
{
	if (!InParentJob)
	{
		return INDEX_NONE;
	}

	FScopeLock Lock(&StreamsCS);

	Streams.Add({InStreamName, InExtension});
	ContainerStreams.AddDefaulted();
	StreamBuffers.Add(MakeUnique<FStreamBufferState>());

	return Streams.Num() - 1;
}

void UTmvMediaContainerTranscodeMuxer::SetStreamTrackConfig(int32 InStreamId, const FTmvMediaMuxerTrackConfig& InTrackConfig)
{
	int32 MuxerTrackIndex = INDEX_NONE;
	
	{
		FScopeLock Lock(&StreamsCS);

		if (!ContainerStreams.IsValidIndex(InStreamId))
		{
			UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::SetStreamTrackConfig: Invalid stream id %d.", InStreamId);
			return;
		}

		if (!Muxer.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::SetStreamTrackConfig: Muxer not initialized.");
			return;
		}

		FContainerStreamInfo& Info = ContainerStreams[InStreamId];
		Info.TrackConfig = InTrackConfig;
		Info.bHasTrackConfig = true;

		// Add track to the underlying muxer.
		MuxerTrackIndex = Info.MuxerTrackIndex = Muxer->AddTrack(InTrackConfig);
		if (MuxerTrackIndex < 0)
		{
			UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::SetStreamTrackConfig: Failed to add track: %ls", *Muxer->GetLastError());
		}
	}

	if (MuxerTrackIndex >= 0)
	{
		// Register the mapping so FlushPendingRequests can resolve track index
		// to stream index without acquiring StreamsCS.
		FScopeLock Lock(&SampleBufferCS);
		MuxerTrackToStreamIndex.Add(MuxerTrackIndex, InStreamId);
	}
}

void UTmvMediaContainerTranscodeMuxer::ReceiveAccessUnit(
		UTmvMediaTranscodeJob* InParentJob,
		int32 InStreamId,
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		TSharedPtr<TArray64<uint8>>&& InAccessUnit)
{
	if (!InAccessUnit.IsValid() || InAccessUnit->Num() == 0)
	{
		return;
	}

	if (!Muxer.IsValid())
	{
		UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::ReceiveAccessUnit: Muxer not initialized.");
		return;
	}

	{
		FScopeLock Lock(&StreamsCS);
		if (!ContainerStreams.IsValidIndex(InStreamId))
		{
			const FText Message = FText::Format(LOCTEXT("InvalidStreamId", "Invalid stream id ({0})"), FText::AsNumber(InStreamId));
			UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::ReceiveAccessUnit: %ls.", *Message.ToString());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			return;
		}

		const FContainerStreamInfo& Info = ContainerStreams[InStreamId];
		if (!Info.bHasTrackConfig || Info.MuxerTrackIndex < 0)
		{
			const FText Message = FText::Format(LOCTEXT("InvalidTrackConfig", "Track config not set for stream {0}"), FText::AsNumber(InStreamId));
			UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::ReceiveAccessUnit: %ls.", *Message.ToString());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			return;
		}
	}

	// Buffer the incoming access unit and lazily start the muxer on the first sample.
	{
		FScopeLock Lock(&SampleBufferCS);

		// Start the muxer on first sample if not yet started.
		// Protected by SampleBufferCS to prevent multiple threads from calling Start() concurrently.
		if (!bMuxerStarted && !bMuxerFailed)
		{
			ITmvMediaMuxer::FSampleRequestDelegate SampleRequestDelegate;
			SampleRequestDelegate.BindWeakLambda(this, [this](const TArray<ITmvMediaMuxer::FSampleRequest>& InRequests)
			{
				// Called from the muxer's worker thread when it needs samples.
				FScopeLock InnerLock(&SampleBufferCS);
				PendingRequests.Append(InRequests);
				// Try to fulfill from already-buffered data.
				FlushPendingRequests();
			});

			ITmvMediaMuxer::FStatusDelegate StatusDelegate;
			TWeakObjectPtr<UTmvMediaTranscodeJob> ParentJobWeak = InParentJob;
			StatusDelegate.BindWeakLambda(this, [this, ParentJobWeak](ITmvMediaMuxer::EStatus InStatus)
			{
				if (InStatus == ITmvMediaMuxer::EStatus::Failed && Muxer.IsValid())
				{
					const FText Message = FText::Format(LOCTEXT("MuxerFailure", "Muxer reported failure: {0}"), FText::FromString(Muxer->GetLastError()));
					UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer: %ls.", *Message.ToString());
					UTmvMediaTranscodeJob::SafeReportError(ParentJobWeak.Get(), Message);
				}
				// Signal that the muxer worker has finished (success or failure).
				// RequestStop() waits on this before calling Finalize().
				MuxerFinishedEvent->Trigger();
			});

			// Pass the source start timecode to the muxer if available and supported.
			if (Muxer->SupportsStartTimecode())
			{
				if (UTmvMediaFrameProducer* Producer = InParentJob->GetStage<UTmvMediaFrameProducer>())
				{
					const TOptional<FTimecode> Timecode = InParentJob->Settings.bEnableStartTimecodeOverride
						? InParentJob->Settings.StartTimecodeOverride : Producer->GetStartTimecode();
					if (Timecode.IsSet())
					{
						// Use the timecode rate if specified, otherwise use the video track's.
						const FFrameRate TimecodeRate = Producer->GetStartTimecodeRate().IsSet() 
							? Producer->GetStartTimecodeRate().GetValue() : Producer->GetVideoTrackInfo().FrameRate;
						Muxer->SetStartTimecode(Timecode.GetValue(), TimecodeRate);
					}
				}
			}

			if (Muxer->Start(SampleRequestDelegate, StatusDelegate) != ETmvMediaContainerResult::Success)
			{
				const FText Message = FText::Format(LOCTEXT("MuxerStartFailure", "Failed to start muxer: {0}"), FText::FromString(Muxer->GetLastError()));
				UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::ReceiveAccessUnit: %ls.", *Message.ToString());
				UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
				bMuxerFailed = true;
				return;
			}
			bMuxerStarted = true;
		}

		if (StreamBuffers.IsValidIndex(InStreamId) && StreamBuffers[InStreamId].IsValid())
		{
			// For the sample number, we need contiguous values that start at index 0.
			const uint32 SampleNumber = static_cast<uint32>(InTimeInfo.FrameIndexNoOffset);

			FStreamBufferState& State = *StreamBuffers[InStreamId];

			FBufferedSample Buffered;
			Buffered.Data = MoveTemp(InAccessUnit);
			Buffered.SampleNumber = SampleNumber;
			State.Samples.Add(SampleNumber, MoveTemp(Buffered));

			if (SampleNumber >= State.HighestReceivedSampleNumber)
			{
				State.HighestReceivedSampleNumber = SampleNumber + 1;
			}
		}

		// Try to fulfill any pending requests now that we have new data.
		FlushPendingRequests();
	}
}

void UTmvMediaContainerTranscodeMuxer::FlushPendingRequests()
{
	// Must be called with SampleBufferCS held.
	// Collect indices of fulfilled/stale requests, then remove after iteration.
	TArray<int32, TInlineAllocator<16>> IndicesToRemove;

	for (int32 i = 0; i < PendingRequests.Num(); ++i)
	{
		const ITmvMediaMuxer::FSampleRequest& Request = PendingRequests[i];

		// Resolve muxer track index to stream index via pre-built map (no StreamsCS needed).
		const int32* StreamIndexPtr = MuxerTrackToStreamIndex.Find(Request.TrackIndex);
		if (!StreamIndexPtr || !StreamBuffers.IsValidIndex(*StreamIndexPtr) || !StreamBuffers[*StreamIndexPtr].IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, 
				"ContainerTranscodeMuxer::FlushPendingRequests: Failed to find StreamBuffer for track index %d."
				" Pending Request for frame %d will be ignored.", Request.TrackIndex, Request.SampleNumber);
			IndicesToRemove.Add(i);
			continue;
		}

		FStreamBufferState& State = *StreamBuffers[*StreamIndexPtr];

		// Discard duplicate requests for samples that have already been delivered.
		if (State.DeliveredSampleNumbers.Contains(Request.SampleNumber))
		{
			IndicesToRemove.Add(i);
			continue;
		}

		if (const FBufferedSample* FoundSample = State.Samples.Find(Request.SampleNumber))
		{
			FTmvMediaMuxerSample Sample;
			Sample.Data = MakeArrayView(FoundSample->Data->GetData(), FoundSample->Data->Num());
			Sample.SampleNumber = FoundSample->SampleNumber;
			Sample.bIsKeyframe = true;

			// Mark as final if stop was requested and this is the last received sample.
			if (bStopRequested && (Request.SampleNumber + 1) >= State.HighestReceivedSampleNumber)
			{
				UE_LOGF(LogTmvMedia, Verbose, "ContainerTranscodeMuxer::FlushPendingRequests: Last sample %u for track %d", FoundSample->SampleNumber, Request.TrackIndex);
				Sample.bIsFinalSample = true;
			}

			if (Muxer->AddSample(Request.TrackIndex, Sample) != ETmvMediaContainerResult::Success)
			{
				UE_LOGF(LogTmvMedia, Error, "ContainerTranscodeMuxer::FlushPendingRequests: Failed to add sample %u for track %d: %ls",
					FoundSample->SampleNumber, Request.TrackIndex, *Muxer->GetLastError());
				// Keep the sample in the buffer and the request pending so it can be retried.
				continue;
			}
			
			UE_LOGF(LogTmvMedia, Verbose, "ContainerTranscodeMuxer::FlushPendingRequests: Added sample %u for track %d", FoundSample->SampleNumber, Request.TrackIndex);

			State.Samples.Remove(Request.SampleNumber);
			State.DeliveredSampleNumbers.Add(Request.SampleNumber);
			IndicesToRemove.Add(i);
		}
	}

	// Remove in reverse order so indices stay valid.
	for (int32 i = IndicesToRemove.Num() - 1; i >= 0; --i)
	{
		PendingRequests.RemoveAtSwap(IndicesToRemove[i]);
	}
}

#undef LOCTEXT_NAMESPACE
