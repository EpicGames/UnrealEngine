// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaFileSequenceFrameProducer.h"

#include "Async/Async.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "ImageUtils.h"
#include "ImageWrapperHelper.h"
#include "Misc/Paths.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaFrameConverter.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Transcoder/TmvMediaTranscodeJob.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaFileSequenceFrameProducer)

#define LOCTEXT_NAMESPACE "TmvMediaFileSequenceFrameProducer"

namespace UE::TmvMedia
{
	/**
	 * Assuming the filename is something like "filename00000.extension"
	 * We are assuming only one extension, not dot naming.
	 * We will look for a numerical value at the end of the filename and return that.
	 */
	bool GetFrameNumberFromFilename(const FString& InFilename, int32& OutFrameNumber)
	{
		// Exclude the extension in case it has a number in it, that doesn't count.
		FString BaseFilename = FPaths::GetBaseFilename(InFilename);

		bool bFoundNumber = false;

		// Find the first digit starting from the right.
		int32 Index = BaseFilename.Len() - 1;
		for (; Index >= 0; --Index)
		{
			if (FChar::IsDigit(BaseFilename[Index]))
			{
				break;
			}
		}

		// Did we find one?
		if (Index >= 0)
		{
			int32 LastNumberIndex = Index;
			// Find the next non digit...
			for (; Index >= 0; --Index)
			{
				bool bIsDigit = FChar::IsDigit(BaseFilename[Index]);
				if (bIsDigit == false)
				{
					break;
				}
			}

			// Index now points to the next non digit.
			// Extract the number.
			Index++;
			if (Index < BaseFilename.Len())
			{
				bFoundNumber = true;
				FString NumberString = BaseFilename.Mid(Index, LastNumberIndex - Index + 1);
				OutFrameNumber = FCString::Atoi(*NumberString);
			}
		}
		return bFoundNumber;
	}
}

UTmvMediaFileSequenceFrameProducer::UTmvMediaFileSequenceFrameProducer()
{
	bHasAsyncQueue = true;
}

bool UTmvMediaFileSequenceFrameProducer::Start(UTmvMediaTranscodeJob* InParentJob)
{
	Super::Start(InParentJob); // set status to "started".
	
	// This could be either the first image of the sequence, or a directory.
	// Note: must be resolved on the main thread because Zenloader can't flush async load on async thread.
	FString SequencePath = InParentJob->Settings.GetAbsoluteInputPath();

	TWeakObjectPtr<UTmvMediaTranscodeJob> ParentJobWeak = InParentJob;
	TWeakObjectPtr<UTmvMediaFileSequenceFrameProducer> SelfWeak(this);

	Async(EAsyncExecution::Thread, [SelfWeak, ParentJobWeak, SequencePath]()
	{
		TStrongObjectPtr<UTmvMediaTranscodeJob> ParentJob = ParentJobWeak.Pin();
		TStrongObjectPtr<UTmvMediaFileSequenceFrameProducer> Self = SelfWeak.Pin();
		if (ParentJob && Self)
		{
			Self->ProcessAllImages(ParentJob.Get(), SequencePath);
			Self->SetStageStatus(ETmvMediaTranscodeStageStatus::Stopped, ParentJob.Get());
		}
	});
	return true;
}

void UTmvMediaFileSequenceFrameProducer::RequestStop(UTmvMediaTranscodeJob* InParentJob)
{
	if (NumActiveFrames > 0)
	{
		// Stopping is not immediate, need to wait for the processing thread to return.
		SetStageStatus(ETmvMediaTranscodeStageStatus::Stopping, InParentJob);
		return;
	}

	Super::RequestStop(InParentJob);
}

void UTmvMediaFileSequenceFrameProducer::ProcessAllImages(UTmvMediaTranscodeJob* InParentJob, const FString& InSequencePath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTmvMediaFileSequenceFrameProducer::ProcessAllImages);

	// This could be either the first image of the sequence, or a directory.
	FString SequencePath = InSequencePath;
	FString FileExtension = TEXT("*");

	if (FPaths::FileExists(SequencePath))
	{
		// If a file with an extension is specified, use it to restrict the file search in the folder.
		FileExtension += FPaths::GetExtension(SequencePath, /*IncludeDot*/ true);
		// Extract the folder path.
		SequencePath = FPaths::GetPath(SequencePath);
	}

	if (!FPaths::DirectoryExists(SequencePath))
	{
		const FText Message = FText::Format(LOCTEXT("DirectoryNotFound", "Directory \"{0}\" does not exist"), FText::FromString(SequencePath));
		UE_LOGF(LogTmvMedia, Error, "FileSequenceFrameProducer: %ls.", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return;
	}

	// Get source files.
	TArray<FString> FoundFiles;
	IFileManager::Get().FindFiles(FoundFiles, *SequencePath, *FileExtension);
	FoundFiles.RemoveAll([](const FString& FileName) { return FileName.StartsWith(TEXT(".")); });
	FoundFiles.Sort();

	if (FoundFiles.Num() == 0)
	{
		const FText Message = FText::Format(LOCTEXT("NoFiles", "No files to import in directory \"{0}\" (Extension: {1})"),
			FText::FromString(SequencePath), FText::FromString(FileExtension));
		UE_LOGF(LogTmvMedia, Error, "FileSequenceFrameProducer: %ls.", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return;
	}

	UE_LOGF(LogTmvMedia, Log, "Found %i image files in %ls to import.", FoundFiles.Num(), *SequencePath);

	// Create image wrapper
	FString Ext = FPaths::GetExtension(FoundFiles[0]);
	EImageFormat ImageFormat = ImageWrapperHelper::GetImageFormat(Ext);

	if (ImageFormat == EImageFormat::Invalid)
	{
		const FText Message = FText::Format(LOCTEXT("InvalidFileFormat", "Invalid file format \"{0}\""), FText::FromString(Ext));
		UE_LOGF(LogTmvMedia, Error, "FileSequenceFrameProducer: %ls.", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return;
	}

	// Update the video track info
	if (InParentJob->Settings.FrameRate.Numerator > 0 && InParentJob->Settings.FrameRate.Denominator > 0)
	{
		FTmvMediaFrameProducerTrackInfo LocalVideoTrackInfo = GetVideoTrackInfo();
		LocalVideoTrackInfo.FrameRate = InParentJob->Settings.FrameRate;
		LocalVideoTrackInfo.Duration = FTimespan::FromSeconds(LocalVideoTrackInfo.FrameRate.AsInterval() * static_cast<double>(FoundFiles.Num()));
		SetVideoTrackInfo(LocalVideoTrackInfo);
	}

	// Get number of threads to use.
	int32 NumThreads = InParentJob->Settings.NumProducerThreads;
	if (NumThreads <= 0)
	{
		NumThreads = 8;
	}

	// Upper bound for this to avoid overloading the system. Todo: Make this configurable?
	NumThreads = FMath::Min(NumThreads, FPlatformMisc::NumberOfCores());

	int32 FirstFrameIndex = 0;
	if (FoundFiles.Num() > 0)
	{
		UE::TmvMedia::GetFrameNumberFromFilename(FoundFiles[0], FirstFrameIndex);
	}
	
	// Loop through all files.
	int NumDone = 0;
	int TotalNum = FoundFiles.Num();
	NumActiveFrames = 0;
	for (const FString& FileName : FoundFiles)
	{
		// Wait for threads to finish if we have too many.
		while (NumActiveFrames >= NumThreads)
		{
			FPlatformProcess::Sleep(0.1f);
		}

		++NumDone;

		UTmvMediaTranscodeJob::SafeUpdateProgress(InParentJob, NumDone, TotalNum);

		++NumActiveFrames;

		const FString FullFileName = FPaths::Combine(SequencePath, FileName);

#if WITH_EDITOR
		EAsyncExecution ExecutionTarget = EAsyncExecution::LargeThreadPool;
#else
		EAsyncExecution ExecutionTarget = EAsyncExecution::ThreadPool;
#endif

		TWeakObjectPtr<UTmvMediaTranscodeJob> ParentJobWeak = InParentJob;
		TWeakObjectPtr<UTmvMediaFileSequenceFrameProducer> SelfWeak(this);

		Async(ExecutionTarget, [SelfWeak, FullFileName, ParentJobWeak, FirstFrameIndex]() mutable
		{
			if (TStrongObjectPtr<UTmvMediaFileSequenceFrameProducer> Self = SelfWeak.Pin())
			{
				if (TStrongObjectPtr<UTmvMediaTranscodeJob> ParentJob = ParentJobWeak.Pin())
				{
					Self->ProcessImage(ParentJob.Get(), FullFileName, FirstFrameIndex);
				}
				--Self->NumActiveFrames;
			}
		});

		// Do we want to cancel?
		if (IsCancelled(InParentJob))
		{
			break;
		}
	}

	// Wait for all our tasks to finish.
	while (NumActiveFrames > 0) // todo: add timeout? (there isn't one in ProcessExr either)
	{
		FPlatformProcess::Sleep(0.2f);
	}
}

bool UTmvMediaFileSequenceFrameProducer::IsCancelled(const UTmvMediaTranscodeJob* InParentJob) const
{
	return !IsValid(InParentJob) || !InParentJob->IsRunning() || GetStageStatus() != ETmvMediaTranscodeStageStatus::Started;
}

bool UTmvMediaFileSequenceFrameProducer::ProcessImage(UTmvMediaTranscodeJob* InParentJob, const FString& InImagePath, int32 InFirstFrameIndex)
{
	TSharedPtr<FImage> Image = MakeShared<FImage>();

	// Load image into buffer.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UTmvMediaFileSequenceFrameProducer::ProcessImageCustom:LoadImage);
		if (!FImageUtils::LoadImage(*InImagePath, *Image))
		{
			UE_LOGF(LogTmvMedia, Error, "Failed to load %ls", *InImagePath);
			return false;
		}
	}

	// Convert to linear float16 if not already.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UTmvMediaFileSequenceFrameProducer::ProcessImageCustom:ChangeFormat);
		Image->ChangeFormat(ERawImageFormat::RGBA16F, EGammaSpace::Linear);
	}

	// Push to next stage
	if (UTmvMediaFrameConverter* FrameConverter = InParentJob->GetStage<UTmvMediaFrameConverter>())
	{
		TUniquePtr<FTmvMediaFrameMips> FrameMips = MakeUnique<FTmvMediaFrameMips>();
		// Extract frame number from the name.
		UE::TmvMedia::GetFrameNumberFromFilename(InImagePath, FrameMips->TimeInfo.FrameIndex);
		FrameMips->TimeInfo.FrameIndexNoOffset = FrameMips->TimeInfo.FrameIndex - InFirstFrameIndex;
		FrameMips->TimeInfo.FrameDuration = FTimespan::FromSeconds(InParentJob->Settings.FrameRate.AsInterval());
		FrameMips->TimeInfo.FrameTime = FrameMips->TimeInfo.FrameDuration * FrameMips->TimeInfo.FrameIndex;
		FrameMips->Init(Image, InParentJob->Settings.bEnableMipMapping);
		FrameConverter->ReceiveMips(InParentJob, MoveTemp(FrameMips));
	}
	return true;
}

#undef LOCTEXT_NAMESPACE