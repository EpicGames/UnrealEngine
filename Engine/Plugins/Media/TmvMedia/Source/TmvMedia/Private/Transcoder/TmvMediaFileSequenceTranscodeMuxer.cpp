// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaFileSequenceTranscodeMuxer.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Utils/TmvMediaPathUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaFileSequenceTranscodeMuxer)

#define LOCTEXT_NAMESPACE "TmvMediaFileSequenceTranscodeMuxer"

int32 UTmvMediaFileSequenceTranscodeMuxer::OpenStream(UTmvMediaTranscodeJob* InParentJob, const FString& InStreamName, const FString& InExtension)
{
	if (!InParentJob)
	{
		return INDEX_NONE;
	}

	if (InStreamName.IsEmpty() || InExtension.IsEmpty())
	{
		const FText Message = FText::Format(LOCTEXT("InvalidStream", "Invalid stream name or extension (Name=\"{0}\", Ext=\"{1}\""),
			FText::FromString(InStreamName), FText::FromString(InExtension));
		UE_LOGF(LogTmvMedia, Error, "FileSequenceTranscodeMuxer::OpenStream: %ls.", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return INDEX_NONE;
	}

	const FString OutPath = InParentJob->Settings.GetAbsoluteOutputPath();

	if (OutPath.IsEmpty())
	{
		const FText Message = LOCTEXT("EmptyOutputPath", "OutputPath is empty");
		UE_LOGF(LogTmvMedia, Error, "FileSequenceTranscodeMuxer::OpenStream: %ls.", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return INDEX_NONE;
	}

	if (!FPaths::DirectoryExists(OutPath))
	{
		if (!FPlatformFileManager::Get().GetPlatformFile().CreateDirectoryTree(*OutPath))
		{
			const FText Message = FText::Format(LOCTEXT("FailedCreateOutputDir", "Failed to create output directory \"{0}\""), FText::FromString(OutPath));
			UE_LOGF(LogTmvMedia, Error, "FileSequenceTranscodeMuxer::OpenStream: %ls.", *Message.ToString());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			return INDEX_NONE;
		}
	}

	FScopeLock Lock(&StreamsCS);
	Streams.Add({InStreamName, InExtension});
	return Streams.Num() - 1;
}

void UTmvMediaFileSequenceTranscodeMuxer::ReceiveAccessUnit(
		UTmvMediaTranscodeJob* InParentJob,
		int32 InStreamId,
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		TSharedPtr<TArray64<uint8>>&& InAccessUnit)
{
	if (!InAccessUnit.IsValid())
	{
		const FText Message = LOCTEXT("InvalidAccessUnit", "Null access unit");
		UE_LOGF(LogTmvMedia, Error, "FileSequenceTranscodeMuxer::ReceiveAccessUnit: %ls.", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return;
	}

	if (InAccessUnit->Num() == 0)
	{
		UE_LOGF(LogTmvMedia, Warning, "ReceiveAccessUnit: Empty access unit (stream %d, frame %d).", InStreamId, InTimeInfo.FrameIndex);
		return;
	}

	FStreamInfo StreamInfo;
	{
		FScopeLock Lock(&StreamsCS);
		if (!Streams.IsValidIndex(InStreamId))
		{
			const FText Message = LOCTEXT("InvalidStreamId", "Invalid stream id");
			UE_LOGF(LogTmvMedia, Error, "FileSequenceTranscodeMuxer::ReceiveAccessUnit: %ls.", *Message.ToString());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			return;
		}
		StreamInfo = Streams[InStreamId];
	}

	if (!InParentJob)
	{
		UE_LOGF(LogTmvMedia, Error, "ReceiveAccessUnit: Invalid Transcode Job.");
		return;
	}

	const FString OutPath = InParentJob->Settings.GetAbsoluteOutputPath();
	const FString FileName = UE::TmvMedia::PathUtils::MakeFrameFilename(*StreamInfo.StreamName, InTimeInfo.FrameIndex, InParentJob->Settings.ZeroPadFrameNumbers, *StreamInfo.Extension);
	const FString FullFileName = FPaths::Combine(OutPath, FileName);

	TUniquePtr<IFileHandle> File(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*FullFileName));
	if (File)
	{
		if (!File->Write(InAccessUnit->GetData(), InAccessUnit->Num()))
		{
			const FText Message = FText::Format(LOCTEXT("WriteFailed", "Failed to write access unit to file \"{0}\""), FText::FromString(FullFileName));
			UE_LOGF(LogTmvMedia, Error, "FileSequenceTranscodeMuxer::ReceiveAccessUnit: %ls.", *Message.ToString());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		}
	}
	else
	{
		const FText Message = FText::Format(LOCTEXT("OpenFailed", "Failed to open output file for writing: \"{0}\""), FText::FromString(FullFileName));
		UE_LOGF(LogTmvMedia, Error, "FileSequenceTranscodeMuxer::ReceiveAccessUnit: %ls.", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
	}
}

#undef LOCTEXT_NAMESPACE
