// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaFrameConverter.h"

#include "ColorManagement/TransferFunctions.h"
#include "ImageCore.h"
#include "ImageParallelFor.h"
#include "MediaShaders.h"
#include "SampleConverter/TmvMediaFrameMipImageBuffer.h"
#include "SampleConverter/TmvMediaFrameMipMemoryBuffer.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaFrameEncoder.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Utils/TmvMediaFrameUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaFrameConverter)

#define LOCTEXT_NAMESPACE "TmvMediaFrameConverter"

namespace UE::TmvMedia::FrameConverter
{
	/** Convert a row from float16 to float32 quadruplets. */
	void ConvertRow(const FFloat16Color* InSrc, const TArrayView<FLinearColor> InOutRow)
	{
		for (int32 Index = 0; Index < InOutRow.Num(); Index++)
		{
			InOutRow[Index] = InSrc[Index].GetFloats();
		}
	}

	/** Add a row of float16 quadruplets to the given float32 quadruplets row. */
	void AddRow(const FFloat16Color* InSrc, const TArrayView<FLinearColor> InOutRow)
	{
		for (int32 Index = 0; Index < InOutRow.Num(); Index++)
		{
			FLinearColor Source = InSrc[Index].GetFloats();
			VectorRegister4Float vResult = VectorAdd(VectorLoad(&InOutRow[Index].R), VectorLoad(&Source.R));
			VectorStore(vResult, &InOutRow[Index].R);
		}
	}

	/** Multiply the row by the given factor. */
	void MultiplyRow(const TArrayView<FLinearColor> InOutRow, float InFactor)
	{
		VectorRegister4Float vFactor = VectorSetFloat1(InFactor);

		for (int32 Index = 0; Index < InOutRow.Num(); Index++)
		{
			VectorRegister4Float vVec = VectorLoad(&InOutRow[Index].R);
			VectorStore(VectorMultiply(vFactor, vVec), &InOutRow[Index].R);
		}
	}

	/** Simple 2 to 1 downsample */
	void DownSampleRow(const TArrayView<FLinearColor> InRow, const TArrayView<FLinearColor> OutRow)
	{
		if (InRow.IsEmpty() || OutRow.IsEmpty())
		{
			return;
		}
		
		const int32 Ratio = InRow.Num() / OutRow.Num();
		const float NormFactor = static_cast<float>(OutRow.Num()) / static_cast<float>(InRow.Num());
		const VectorRegister4Float vNormFactor = VectorSetFloat1(NormFactor);

		for (int32 OutIndex = 0; OutIndex < OutRow.Num(); OutIndex++)
		{
			VectorRegister4Float vColorAccum = VectorLoad(&InRow[OutIndex * Ratio].R);
			for (int32 AccumIndex = 1; AccumIndex < Ratio; ++AccumIndex)
			{
				vColorAccum = VectorAdd(vColorAccum, VectorLoad(&InRow[OutIndex * Ratio + AccumIndex].R));
			}
			VectorStore(VectorMultiply(vColorAccum, vNormFactor), &OutRow[OutIndex].R);
		}
	}

	/** Implements linear transfer function */
	struct FLinearEncoder
	{
		static float Encode(float InValue)
		{
			return InValue;
		}

		static VectorRegister4Float EncodeVector(VectorRegister4Float InVec)
		{
			return InVec;
		}
	};

	/** Implements gamma22 transfer function */
	struct FGamma22Encoder
	{
		static float Encode(float InValue)
		{
			return UE::Color::EncodeGamma22(InValue);
		}

		static VectorRegister4Float EncodeVector(VectorRegister4Float InVec)
		{
			return VectorPow(InVec, VectorSetFloat1(1.0f/2.2f));
		}
	};

	/** Implements SRGB transfer function */
	struct FSRGBEncoder
	{
		static float Encode(float InValue)
		{
			return Color::EncodeSRGB(InValue);
		}

		static VectorRegister4Float EncodeVector(VectorRegister4Float InVec)
		{
			VectorRegister4Float LinearResult = VectorMultiply(InVec, VectorSetFloat1(12.92f));
			VectorRegister4Float PowResult = VectorMultiplyAdd(VectorPow(InVec, VectorSetFloat1(1.0f/2.4f)), VectorSetFloat1(1.055f), VectorSetFloat1(-0.055f));
			return VectorSelect(VectorCompareLE(InVec, VectorSetFloat1(0.04045f / 12.92f)), LinearResult, PowResult);
		}
	};

	/** Spec-literal PQ: input is absolute nits, output is the non-linear PQ signal. */
	struct FSt2084Encoder
	{
		static float Encode(float InValue)
		{
			return Color::EncodeST2084(InValue);
		}

		static VectorRegister4Float EncodeVector(VectorRegister4Float InVec)
		{
			const float Lp = 10000.0f;
			const float m1 = 2610 / 4096.0f * (1.0f / 4.0f);
			const float m2 = 2523 / 4096.0f * 128.0f;
			const float c1 = 3424 / 4096.0f;
			const float c2 = 2413 / 4096.0f * 32.f;
			const float c3 = 2392 / 4096.0f * 32.f;

			VectorRegister4Float V1 = VectorPow(VectorDivide(InVec, VectorSetFloat1(Lp)), VectorSetFloat1(m1));
			VectorRegister4Float Num = VectorMultiplyAdd(V1, VectorSetFloat1(c2), VectorSetFloat1(c1));
			VectorRegister4Float Den = VectorMultiplyAdd(V1, VectorSetFloat1(c3), VectorSetFloat1(1.0f));
			return VectorPow(VectorDivide(Num, Den), VectorSetFloat1(m2));
		}
	};

	/** Spec-literal HLG: input is BT.2100 literal [0, 1], output is the HLG signal. */
	struct FHLGEncoder
	{
		static float Encode(float InValue)
		{
			// HLG signal is spec-bounded to [0, 1]; clamp to peak so per-channel saturation is
			// consistent regardless of which YUV plane the max magnitude lands on.
			return FMath::Min(Color::EncodeHLG(InValue), 1.0f);
		}

		static VectorRegister4Float EncodeVector(VectorRegister4Float InVec)
		{
			// HLG OETF is piecewise (sqrt + log), no simple SIMD form -- fall back to scalar.
			alignas(16) float Values[4];
			VectorStoreAligned(InVec, Values);
			Values[0] = Encode(Values[0]);
			Values[1] = Encode(Values[1]);
			Values[2] = Encode(Values[2]);
			return VectorLoadAligned(Values);
		}
	};

	/** Implements slog3 transfer function */
	struct FSLog3Encoder
	{
		static float Encode(float InValue)
		{
			return Color::EncodeSLog3(InValue);
		}

		static VectorRegister4Float EncodeVector(VectorRegister4Float InVec)
		{
			VectorRegister4Float LinearResult = VectorMultiplyAdd(InVec, VectorSetFloat1(76.2102946929f / (0.01125f * 1023.0f)), VectorSetFloat1(95.0f/1023.0f));

			// Using log2 to evaluate: log10(x) = log2(x)/log2(10)
			const float LogFactor = 1.0f/3.32192809489f; //1/log2(10)

			VectorRegister4Float LogResultTmp = VectorMultiplyAdd(InVec, VectorSetFloat1(1.0f/0.19f), VectorSetFloat1(0.01f/0.19f));
			VectorRegister4Float LogResult = VectorMultiplyAdd(VectorLog2(LogResultTmp), VectorSetFloat1(LogFactor * 261.5f / 1023.0f), VectorSetFloat1(420.0f / 1023.0f));
			return VectorSelect(VectorCompareGE(InVec, VectorSetFloat1(0.01125000f)), LogResult, LinearResult); // Value >= 0.01125000f -> Select LogResult
		}
	};

	// For profiling and validation tests.
#define TMV_MEDIA_USE_VECTOR_PATH 1

	template<typename EncoderType>
	void ConvertYUVInPlaceImpl(FImageView& InImage, const FMatrix44f& RGB2YUVMatrix, float LinearPreScale)
	{
		const bool bApplyPreScale = !FMath::IsNearlyEqual(LinearPreScale, 1.0f);
		const VectorRegister4Float vPreScale = VectorSetFloat1(LinearPreScale);
		for (int32 PixY = 0; PixY < InImage.SizeY; ++PixY)
		{
			FFloat16Color* Row = static_cast<FFloat16Color*>(InImage.GetPixelPointer(0, PixY));
			for (int32 PixX = 0; PixX < InImage.SizeX; ++PixX)
			{
				FLinearColor RGBA = Row[PixX].GetFloats();
				const float A = RGBA.A;

#if TMV_MEDIA_USE_VECTOR_PATH
				VectorRegister4Float vRGBA = VectorLoad(&RGBA.R);
				if (bApplyPreScale)
				{
					// Scale UE scene-linear input into the encoder's native linear space
					// (e.g. nits for PQ, BT.2100 scene-linear for HLG) before the transfer function.
					vRGBA = VectorMultiply(vRGBA, vPreScale);
				}
				VectorRegister4Float vRGBAEnc = VectorMax(EncoderType::EncodeVector(vRGBA), VectorZeroFloat());
				VectorRegister4Float vYUVA = VectorMax(VectorTransformVector(VectorSet_W1(vRGBAEnc), &RGB2YUVMatrix), VectorZeroFloat());
				vYUVA = VectorMergeVecXYZ_VecW(vYUVA, vRGBAEnc);	// Set encoded alpha back in W of the vector.

				FLinearColor YUVA;
				VectorStore(vYUVA, &YUVA.R);
#else
				if (bApplyPreScale)
				{
					RGBA.R *= LinearPreScale;
					RGBA.G *= LinearPreScale;
					RGBA.B *= LinearPreScale;
				}

				// Apply gamma prior to conversion (we need to know the encoder's transfer function curve here)
				RGBA.R = FMath::Max(EncoderType::Encode(RGBA.R), 0.0f);
				RGBA.G = FMath::Max(EncoderType::Encode(RGBA.G), 0.0f);
				RGBA.B = FMath::Max(EncoderType::Encode(RGBA.B), 0.0f);
				RGBA.A = 1.0f;

				FLinearColor YUVA = RGB2YUVMatrix.TransformFVector4(*((FVector4f*)&RGBA));
				YUVA.G = FMath::Max(YUVA.G, 0.0f);
				YUVA.B = FMath::Max(YUVA.B, 0.0f);
#endif

				YUVA.A = A;
				Row[PixX] = FFloat16Color(YUVA);
			}
		}
	}

	template<typename EncoderType>
	void ConvertYUVInPlaceParallelForImpl(FImageView& InImage, const FMatrix44f& InRGB2YUVMatrix, float InLinearPreScale)
	{
		FImageCore::ImageParallelFor(TEXT("TmvMediaFrameConverter::ConvertYUVInPlace"), InImage, [&InRGB2YUVMatrix, InLinearPreScale](FImageView& InImagePart, int64 InRowY)
		{
			ConvertYUVInPlaceImpl<EncoderType>(InImagePart, InRGB2YUVMatrix, InLinearPreScale);
		});
	}

	template<typename EncoderType>
	void ConvertYUVInPlace(FImageView& InImage, const FMatrix44f& InRGB2YUVMatrix, float InLinearPreScale = 1.0f)
	{
		ConvertYUVInPlaceParallelForImpl<EncoderType>(InImage, InRGB2YUVMatrix, InLinearPreScale);
	}

	void ConvertYuvInPlace(FImageView& InImage, const FTmvMediaFrameMipInfo& MipInfo)
	{
		// Compute the yuv conversion matrix.
		FMatrix44f Rgb2YuvMatrix;
		{
			using namespace UE::TmvMedia::FrameUtils;
			const FMatrix YuvMatrix = GetRgbToYuvMatrix(MipInfo.ColorInfo.YuvMatrix, MipInfo.ColorInfo.YuvMatrixRange);
			const int BitDepth = MipInfo.Planes.IsValidIndex(0) ? MipInfo.Planes[0].BitDepth : 10;
			const ETmvMediaFrameComponentType ComponentType = MipInfo.Planes.IsValidIndex(0) ? MipInfo.Planes[0].Type : ETmvMediaFrameComponentType::Int;
			const FVector YuvOffset = GetYuvOffset(ComponentType, MipInfo.ColorInfo.YuvMatrixRange, BitDepth);

			Rgb2YuvMatrix = static_cast<FMatrix44f>(MediaShaders::CombineColorTransformAndOffset(YuvMatrix, YuvOffset));
		}

		// We have to transpose in order to use TransformVector.
		Rgb2YuvMatrix = Rgb2YuvMatrix.GetTransposed();

		// Reference white applied as a pre-scale on the encoder input
		const float LinearPreScale = Color::GetReferenceWhiteLinearScale(MipInfo.ColorInfo.Encoding, MipInfo.ColorInfo.ReferenceWhiteOverride, Color::EReferenceWhiteDirection::Encode);

		// We have implemented the same encodings as the media shader.
		switch (MipInfo.ColorInfo.Encoding)
		{
		case Color::EEncoding::sRGB:
			ConvertYUVInPlace<FSRGBEncoder>(InImage, Rgb2YuvMatrix);
			break;

		case Color::EEncoding::Gamma22:
			ConvertYUVInPlace<FGamma22Encoder>(InImage, Rgb2YuvMatrix);
			break;

		case Color::EEncoding::ST2084:
			ConvertYUVInPlace<FSt2084Encoder>(InImage, Rgb2YuvMatrix, LinearPreScale);
			break;

		case Color::EEncoding::SLog3:
			ConvertYUVInPlace<FSLog3Encoder>(InImage, Rgb2YuvMatrix);
			break;

		case Color::EEncoding::HLG:
			ConvertYUVInPlace<FHLGEncoder>(InImage, Rgb2YuvMatrix, LinearPreScale);
			break;

		default:
			// remain linear.
			ConvertYUVInPlace<FLinearEncoder>(InImage, Rgb2YuvMatrix);
			break;
		}
	}

	// This version is using average subsampling, i.e. it will downsample using mean filtering.
	// This is about 3x slower than "fast swizzle".
	void SubSampleAndSwizzle(FImageView& InImage, FTmvMediaFrameMipBuffer* OutMipBuffer)
	{
		const FTmvMediaFrameMipInfo& MipInfo = OutMipBuffer->GetMipInfoRef();

		// Faster would probably be to swizzle, then downsample.
		for (int32 ComponentIndex = 0; ComponentIndex < MipInfo.NumComponents; ++ComponentIndex)
		{
			int32 ComponentIndexInPlane = 0;
			const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(ComponentIndex, &ComponentIndexInPlane);
			if (!MipInfo.Planes.IsValidIndex(PlaneIndex))
			{
				continue;
			}

			uint8* PlaneBuffer = static_cast<uint8*>(OutMipBuffer->GetPlaneBufferForComponent(ComponentIndex));
			const uint32 PlaneStride = MipInfo.Planes[PlaneIndex].Stride;
			const uint32 PlaneNumComponents = MipInfo.Planes[PlaneIndex].NumComponents;
			const int32 PlaneHeight = MipInfo.Planes[PlaneIndex].Height;
			const int32 PlaneWidth = MipInfo.Planes[PlaneIndex].Width;
			const uint32 BitDepth = MipInfo.Planes[PlaneIndex].BitDepth;
			const float BitScale = static_cast<float>(((int32)1 << BitDepth) - 1);

			if (PlaneWidth <= 0 || PlaneHeight <= 0)
			{
				continue;
			}

			const int32 HeightRatio = InImage.SizeY / PlaneHeight;
			const int32 WidthRatio = InImage.SizeX / PlaneWidth;

			TArray<FLinearColor> SrcRowBuffer;
			SrcRowBuffer.SetNum(InImage.SizeX);
			TArray<FLinearColor> PlaneRowBuffer;
			PlaneRowBuffer.SetNum(PlaneWidth);

			for (int32 DstY = 0; DstY < PlaneHeight; ++DstY)
			{
				// In case height ratio > 1, we would need to down filter the source rows.
				ConvertRow(static_cast<FFloat16Color*>(InImage.GetPixelPointer(0, DstY * HeightRatio)), SrcRowBuffer);

				for (int32 RowIndex = 1; RowIndex < HeightRatio; ++RowIndex)
				{
					AddRow(static_cast<FFloat16Color*>(InImage.GetPixelPointer(0, DstY * HeightRatio + RowIndex)), SrcRowBuffer);
				}
				if (HeightRatio > 1)
				{
					MultiplyRow(SrcRowBuffer, 1.0f/static_cast<float>(HeightRatio));
				}

				if (WidthRatio > 1)
				{
					DownSampleRow(SrcRowBuffer, PlaneRowBuffer);
				}

				const FLinearColor* SrcRow = WidthRatio > 1 ? PlaneRowBuffer.GetData() : SrcRowBuffer.GetData();

				uint16* DstRow = reinterpret_cast<uint16*>(PlaneBuffer + static_cast<size_t>(DstY) * PlaneStride); 
				for (int32 DstX = 0; DstX < PlaneWidth; ++DstX)
				{
					float SrcValue = SrcRow[DstX].Component(ComponentIndex);
					DstRow[DstX * PlaneNumComponents + ComponentIndexInPlane] = static_cast<uint16>(SrcValue * BitScale);
				}
			}
		}
	}
	
	uint8 GetComponent(const FColor& InColor, int32 InComponentIndex)
	{
#if PLATFORM_LITTLE_ENDIAN
		return ((uint8*)&InColor)[InComponentIndex];
#else // PLATFORM_LITTLE_ENDIAN
		return ((uint8*)&InColor)[3 - InComponentIndex];
#endif
	}

	// Int8 -> Int8
	void SwizzleComponent(const FColor& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, uint8& OutValue)
	{
		OutValue = GetComponent(InColor, InComponentIndex);
	}

	// Int8 -> Int16
	void SwizzleComponent(const FColor& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, uint16& OutValue)
	{
		OutValue = static_cast<uint16>(GetComponent(InColor, InComponentIndex)) << InBitShift;
	}
	
	// Int8 -> float16
	void SwizzleComponent(const FColor& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, FFloat16& OutValue)
	{
		OutValue = static_cast<float>(GetComponent(InColor, InComponentIndex)) / InBitScale;
	}

	struct FInt16Color
	{
		uint16 RGBA[4];
	};

	// Int16 -> Int8
	void SwizzleComponent(const FInt16Color& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, uint8& OutValue)
	{
		OutValue = InColor.RGBA[InComponentIndex] >> InBitShift;
	}
	
	// Int16 -> Int16
	void SwizzleComponent(const FInt16Color& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, uint16& OutValue)
	{
		OutValue = InColor.RGBA[InComponentIndex];
	}
	
	// Int16 -> float16
	void SwizzleComponent(const FInt16Color& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, FFloat16& OutValue)
	{
		OutValue = static_cast<float>(InColor.RGBA[InComponentIndex]) / InBitScale;
	}

	// Float16 -> int8
	void SwizzleComponent(const FFloat16Color& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, uint8& OutValue)
	{
		const float FloatValue = InColor.GetFloats().Component(InComponentIndex);
		OutValue = static_cast<uint8>(FloatValue * InBitScale);
	}

	// Float16 -> int16
	void SwizzleComponent(const FFloat16Color& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, uint16& OutValue)
	{
		const float FloatValue = InColor.GetFloats().Component(InComponentIndex);
		OutValue = static_cast<uint16>(FloatValue * InBitScale);
	}

	// Float16 -> Float16
	void SwizzleComponent(const FFloat16Color& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, FFloat16& OutValue)
	{
		//const float FloatValue = InColor.GetFloats().Component(InComponentIndex);
		//OutValue = FloatValue;
		OutValue.Encoded = InColor.GetFourHalves()[InComponentIndex];
	}

	// Float32 -> int8
	void SwizzleComponent(const FLinearColor& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, uint8& OutValue)
	{
		const float FloatValue = InColor.Component(InComponentIndex);
		OutValue = static_cast<uint8>(FloatValue * InBitScale);
	}
	
	// Float32 -> int16
	void SwizzleComponent(const FLinearColor& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, uint16& OutValue)
	{
		const float FloatValue = InColor.Component(InComponentIndex);
		OutValue = static_cast<uint16>(FloatValue * InBitScale);
	}
	
	// Float32 -> Float16
	void SwizzleComponent(const FLinearColor& InColor, int32 InComponentIndex, float InBitScale, uint32 InBitShift, FFloat16& OutValue)
	{
		const float FloatValue = InColor.Component(InComponentIndex);
		OutValue = FloatValue;
	}

	template <typename SrcColorType, typename DstComponentType>
	void FastSwizzleComponentImpl(int32 InComponentIndex, int32 InComponentIndexInPlane, const FTmvMediaFramePlaneInfo& InDstPlaneInfo, const FImageView& InImage, FTmvMediaFrameMipBuffer* OutMipBuffer)
	{
		uint8* PlaneBuffer = static_cast<uint8*>(OutMipBuffer->GetPlaneBufferForComponent(InComponentIndex));
		const uint32 PlaneStride = InDstPlaneInfo.Stride;
		const uint32 PlaneNumComponents = InDstPlaneInfo.NumComponents;
		const int32 PlaneHeight = InDstPlaneInfo.Height;
		const int32 PlaneWidth = InDstPlaneInfo.Width;
		const uint32 BitDepth = InDstPlaneInfo.BitDepth;
		
		// Compute the bit scale for int<->float conversions.
		const float BitScale = static_cast<float>(((int32)1 << BitDepth) - 1);

		// Compute the bit shift for int to int conversions.
		const uint32 CompBitDepth = sizeof(DstComponentType) * 8; 
		const uint32 BitShift = CompBitDepth > BitDepth ? CompBitDepth - BitDepth : BitDepth - CompBitDepth;

		const int32 HeightRatio = InImage.SizeY / PlaneHeight;
		const int32 WidthRatio = InImage.SizeX / PlaneWidth;

		for (int32 DstY = 0; DstY < PlaneHeight; ++DstY)
		{
			const SrcColorType* SrcRow = static_cast<SrcColorType*>(InImage.GetPixelPointer(0, DstY * HeightRatio));
			DstComponentType* DstCompRow = reinterpret_cast<DstComponentType*>(PlaneBuffer + static_cast<size_t>(DstY) * PlaneStride) + InComponentIndexInPlane;

			for (int32 DstX = 0; DstX < PlaneWidth; ++DstX)
			{
				SwizzleComponent(SrcRow[DstX*WidthRatio], InComponentIndex, BitScale, BitShift, *DstCompRow);
				DstCompRow += PlaneNumComponents;
			}
		}
	}
	
	template <typename SrcColorType>
	void FastSwizzleComponent(int32 InComponentIndex, int32 InComponentIndexInPlane, const FTmvMediaFramePlaneInfo& InPlaneInfo, const FImageView& InImage, FTmvMediaFrameMipBuffer* OutMipBuffer)
	{
		const uint32 DstBitDepth = InPlaneInfo.BitDepth;
		
		switch (InPlaneInfo.Type)
		{
		case ETmvMediaFrameComponentType::Float:
			if (DstBitDepth == 16)
			{
				FastSwizzleComponentImpl<SrcColorType, FFloat16>(InComponentIndex, InComponentIndexInPlane, InPlaneInfo, InImage, OutMipBuffer);
			}
			else
			{
				UE_LOGF(LogTmvMedia, Error, "FastSwizzle: Plane bit depth %d floating point is not supported.", DstBitDepth);
			}
			break;
		case ETmvMediaFrameComponentType::Int:
			if (DstBitDepth == 8)
			{
				FastSwizzleComponentImpl<SrcColorType, uint8>(InComponentIndex, InComponentIndexInPlane, InPlaneInfo, InImage, OutMipBuffer);
			}
			else if (DstBitDepth > 8 &&  DstBitDepth <= 16)
			{
				FastSwizzleComponentImpl<SrcColorType, uint16>(InComponentIndex, InComponentIndexInPlane, InPlaneInfo, InImage, OutMipBuffer);
			}
			else
			{
				UE_LOGF(LogTmvMedia, Error, "FastSwizzle: Plane bit depth %d integer is not supported.", DstBitDepth);
			}
			break;
		}
	}

	// This version is only swizzling and will skip chroma samples instead of down sampling.
	void FastSwizzle(const FImageView& InImage, FTmvMediaFrameMipBuffer* OutMipBuffer)
	{
		const FTmvMediaFrameMipInfo& DstMipInfo = OutMipBuffer->GetMipInfoRef();

		for (int32 ComponentIndex = 0; ComponentIndex < DstMipInfo.NumComponents; ++ComponentIndex)
		{
			int32 ComponentIndexInPlane = 0;
			const int32 PlaneIndex = DstMipInfo.GetPlaneIndexForComponent(ComponentIndex, &ComponentIndexInPlane);
			if (!DstMipInfo.Planes.IsValidIndex(PlaneIndex))
			{
				UE_LOGF(LogTmvMedia, Error, "FastSwizzle: Invalid plane index %d for component %d.", PlaneIndex, ComponentIndex);
				continue;
			}

			switch (InImage.Format)
			{
			case ERawImageFormat::RGBA16F:
				FastSwizzleComponent<FFloat16Color>(ComponentIndex, ComponentIndexInPlane, DstMipInfo.Planes[PlaneIndex], InImage, OutMipBuffer);
				break;

				case ERawImageFormat::RGBA32F:
				FastSwizzleComponent<FLinearColor>(ComponentIndex, ComponentIndexInPlane, DstMipInfo.Planes[PlaneIndex], InImage, OutMipBuffer);
				break;

			case ERawImageFormat::BGRA8:
				FastSwizzleComponent<FColor>(ComponentIndex, ComponentIndexInPlane, DstMipInfo.Planes[PlaneIndex], InImage, OutMipBuffer);
				break;

			case ERawImageFormat::RGBA16:
				FastSwizzleComponent<FInt16Color>(ComponentIndex, ComponentIndexInPlane, DstMipInfo.Planes[PlaneIndex], InImage, OutMipBuffer);
				break;

			default:
				break;
			}
		}
	}

	/** Convert a packed (in single plane) format. */
	bool ConvertPacked(FImageView* InSourceImage, const FTmvMediaFrameMipInfo& SourceMipInfo, FTmvMediaFrameMipBuffer* OutMipBuffer)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UTmvMediaFrameConverter::ConvertMip);
		if (!InSourceImage || !OutMipBuffer)
		{
			return false;
		}

		const FTmvMediaFrameMipInfo& MipInfo = OutMipBuffer->GetMipInfoRef();

		if (InSourceImage->SizeX != MipInfo.Width || InSourceImage->SizeY != MipInfo.Height)
		{
			return false;
		}

		// First pass, do the yuv color conversion in place (RGBA -> YUVA) if needed.
		if (SourceMipInfo.ColorModel == ETmvMediaFrameColorModel::RGB && MipInfo.ColorModel == ETmvMediaFrameColorModel::YUV)
		{
			// For now expect a float 16 input image.
			if (InSourceImage->Format != ERawImageFormat::RGBA16F)
			{
				UE_LOGF(LogTmvMedia, Error, "RGB->YUV transform is only supported for rgba 16f for now.");
				return false;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(FrameConverter::ConvertMip::YUV);
			ConvertYuvInPlace(*InSourceImage, MipInfo);
		}

		// Second pass, swizzle and subsample each component
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FrameConverter::ConvertMip::Swizzle);
			//SubSampleAndSwizzle(*InSourceImage, OutMipBuffer);	// too slow, should implement a gpu version of this instead.
			FastSwizzle(*InSourceImage, OutMipBuffer);
		}

		return true;
	}

	/**
	 * Top level mip buffer conversion function.
	 * This still under construction. Not all possible conversions are supported and
	 * this also combines with the gpu conversion the frame producer can do prior to this.
	 * The minimum support has been done for use cases and optimized up to "good enough" (not realtime) speed i.e. up to
	 * a point where the cpu encoder is the main bottleneck.
	 */
	bool Convert(FTmvMediaFrameMipBuffer* InMipBuffer, FTmvMediaFrameMipBuffer* OutMipBuffer)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FrameConverter::ConvertMip);
		if (!InMipBuffer || !OutMipBuffer)
		{
			return false;
		}

		FString ErrorMsg;
		FImageInfo ImageInfo;
		if (FrameUtils::PopulateImageInfo(InMipBuffer->GetMipInfoRef(), ImageInfo, &ErrorMsg))
		{
			FImageView ImageView(ImageInfo, InMipBuffer->GetPlaneBufferForComponent(0));
			return ConvertPacked(&ImageView, InMipBuffer->GetMipInfoRef(), OutMipBuffer);
		}

		// planar conversions are not supported yet.
		UE_LOGF(LogTmvMedia, Error, "Convert Not Supported: %ls", *ErrorMsg);
		return false;
	}

	bool NeedsConvert(const FTmvMediaFrameMipInfo& InMipInfoSrc, const FTmvMediaFrameMipInfo& InMipInfoDst)
	{
		if (InMipInfoSrc.Width != InMipInfoDst.Width
			|| InMipInfoSrc.Height != InMipInfoDst.Height
			|| InMipInfoSrc.NumComponents != InMipInfoDst.NumComponents
			|| InMipInfoSrc.ColorModel != InMipInfoDst.ColorModel
			|| InMipInfoSrc.Layout != InMipInfoDst.Layout	// todo: double check this.
			|| InMipInfoSrc.Planes.Num() != InMipInfoDst.Planes.Num())
		{
			return true;
		}

		// todo: check ColorInfo, when/if color space conversion is supported (and needed). Currently converted by GPU in prior stage.

		// The converter doesn't currently support tiling layouts.
		// We push the tiling down to the encoder itself because it is typically already implemented.
		if (InMipInfoSrc.Layout == ETmvMediaFrameBufferLayout::Tiled)
		{
			UE_LOGF(LogTmvMedia, Warning, "Frame Converter doesn't support input buffer being tiled.");
		}

		for (int32 PlaneIndex = 0; PlaneIndex < InMipInfoSrc.Planes.Num(); ++PlaneIndex)
		{
			const FTmvMediaFramePlaneInfo& SrcPlane = InMipInfoSrc.Planes[PlaneIndex];
			const FTmvMediaFramePlaneInfo& DstPlane = InMipInfoDst.Planes[PlaneIndex];
			if (SrcPlane.NumComponents != DstPlane.NumComponents
				|| SrcPlane.BitDepth != DstPlane.BitDepth
				|| SrcPlane.Type != DstPlane.Type
				|| SrcPlane.Stride != DstPlane.Stride
				|| SrcPlane.Width != DstPlane.Width
				|| SrcPlane.Height != DstPlane.Height
				|| SrcPlane.NumLines != DstPlane.NumLines
				|| SrcPlane.ComponentLayout != DstPlane.ComponentLayout)
			{
				return true;
			}
		}
		return false;
	}
}

void UTmvMediaFrameConverter::ReceiveMips(UTmvMediaTranscodeJob* InParentJob, TUniquePtr<FTmvMediaFrameMips>&& InMips)
{
	if (!InParentJob)
	{
		UE_LOGF(LogTmvMedia, Error, "Invalid Transcoding Job.");
		return;
	}

	if (!InMips || InMips->MipBuffers.IsEmpty())
	{
		UE_LOGF(LogTmvMedia, Error, "Transcoding Job: No mips to convert.");
		return;
	}

	UTmvMediaFrameEncoder* FrameEncoder = InParentJob ? InParentJob->GetStage<UTmvMediaFrameEncoder>() : nullptr;
	if (!FrameEncoder)
	{
		UE_LOGF(LogTmvMedia, Error, "Transcoding Job: Encoding stage not found.");
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FrameConverter::ReceiveMips);

	// Query encoder to get the desired format (FTmvMediaFrameMipInfo) and convert in that format.
	TArray<FTmvMediaFrameMipInfo> EncoderFrameMipInfo;

	{
		FTmvMediaFrameMipInfo MipInfo;
		if (InMips->MipBuffers.IsValidIndex(0) && InMips->MipBuffers[0])
		{
			MipInfo = InMips->MipBuffers[0]->GetMipInfoRef();
		}
		else
		{
			const FText Message = LOCTEXT("InvalidMipsParam", "Invalid Source Mips Parameter: No images or buffers to get info from");			
			UE_LOGF(LogTmvMedia, Error, "FrameConverter: %ls.", *Message.ToString());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FrameConverter::RequestMipInfos);
		if (!FrameEncoder->RequestMipInfos(InParentJob, InMips->TimeInfo, MipInfo, EncoderFrameMipInfo))
		{
			return;
		}
	}

	TArray<FTmvMediaFrameMipBufferHandle> NewMipBuffers;

	for (int32 MipIndex = 0; MipIndex < EncoderFrameMipInfo.Num(); MipIndex++)
	{
		if (InMips->MipBuffers.IsValidIndex(MipIndex) && InMips->MipBuffers[MipIndex].IsValid())
		{
			if (UE::TmvMedia::FrameConverter::NeedsConvert(InMips->MipBuffers[MipIndex]->GetMipInfoRef(), EncoderFrameMipInfo[MipIndex]))
			{
				TSharedPtr<FTmvMediaFrameMipBuffer> NewMip = MakeShared<FTmvMediaFrameMipMemoryBuffer>();
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FrameConverter::RequestAllocation);	// (takes 15 ms, todo: pool this)
					if (!NewMip->RequestAllocation(EncoderFrameMipInfo[MipIndex]))
					{
						const FText Message = FText::Format(LOCTEXT("FailedAllocateMip", "Failed to allocate mip {0}"), FText::AsNumber(MipIndex));			
						UE_LOGF(LogTmvMedia, Error, "FrameConverter: %ls.", *Message.ToString());
						UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
						return;
					}
				}

				if (!UE::TmvMedia::FrameConverter::Convert(InMips->MipBuffers[MipIndex].Get(), NewMip.Get()))
				{
					const FText Message = FText::Format(LOCTEXT("FailedConvertMip", "Failed to convert mip {0}"), FText::AsNumber(MipIndex));			
					UE_LOGF(LogTmvMedia, Error, "FrameConverter: %ls.", *Message.ToString());
					UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
					return;
				}
				
				NewMipBuffers.Add(NewMip);
			}
			else
			{
				NewMipBuffers.Add(InMips->MipBuffers[MipIndex]); // Nothing to do.
			}
		}
	}
	
	InMips->MipBuffers = MoveTemp(NewMipBuffers);

	if (InMips->MipBuffers.Num() > 0 || EncoderFrameMipInfo.Num() == 0)
	{
		FrameEncoder->ReceiveMips(InParentJob, MoveTemp(InMips));
	}
	else
	{
		UE_LOGF(LogTmvMedia, Warning, "Transcoding Job: No mips where converted.");
	}
}

#undef LOCTEXT_NAMESPACE
