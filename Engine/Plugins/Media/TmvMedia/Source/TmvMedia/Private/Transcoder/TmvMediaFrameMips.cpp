// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaFrameMips.h"

#include "ImageCore.h"
#include "SampleConverter/TmvMediaFrameMipImageBuffer.h"

bool FTmvMediaFrameMips::Init(const TSharedPtr<FImage>& InMip0, bool bInGenerateMips)
{
	if (!InMip0)
	{
		return false;
	}
	
	MipBuffers.Reset();
	MipBuffers.Add(MakeShared<FTmvMediaFrameMipImageBuffer>(InMip0, 0));

	// Directly use ImageCore to generate the mips on the cpu. (It's reasonably fast.)
	if (bInGenerateMips && (InMip0->SizeX > 1 || InMip0->SizeY > 1))
	{
		TSharedPtr<FImage> PreviousMip = InMip0;
		// Protecting against infinite loop.
		constexpr int32 MaxMips = 32;

		while (MipBuffers.Num() < MaxMips)
		{
			int32 MipSizeX = FMath::Max(PreviousMip->SizeX/2, 1);
			int32 MipSizeY = FMath::Max(PreviousMip->SizeY/2, 1);
			
			TSharedPtr<FImage> NewMip = MakeShared<FImage>(MipSizeX, MipSizeY, 1, InMip0->Format, InMip0->GammaSpace);
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(TmvMediaTranscode::GenerateMip);
				FImageCore::ResizeImage(*PreviousMip, *NewMip);
			}
			int32 MipLevel = MipBuffers.Num();			
			MipBuffers.Add(MakeShared<FTmvMediaFrameMipImageBuffer>(NewMip, MipLevel));

			PreviousMip = NewMip;

			if (MipSizeX == 1 && MipSizeY == 1)
			{
				break;
			}
		}
	}
	return true;
}
