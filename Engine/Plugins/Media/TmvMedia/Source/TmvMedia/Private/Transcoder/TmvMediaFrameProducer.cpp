// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaFrameProducer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaFrameProducer)

FTmvMediaFrameProducerTrackInfo UTmvMediaFrameProducer::GetVideoTrackInfo()	const
{
	FScopeLock lock(&TrackInfoLock);
	return 	VideoTrackInfo;
}

void UTmvMediaFrameProducer::SetVideoTrackInfo(const FTmvMediaFrameProducerTrackInfo& InVideoTrackInfo)
{
	FScopeLock lock(&TrackInfoLock);
	VideoTrackInfo = InVideoTrackInfo;
}

TOptional<FTimecode> UTmvMediaFrameProducer::GetStartTimecode() const
{
	FScopeLock lock(&TrackInfoLock);
	return StartTimecode;
}

TOptional<FFrameRate> UTmvMediaFrameProducer::GetStartTimecodeRate() const
{
	FScopeLock lock(&TrackInfoLock);
	return StartTimecodeRate;
}

void UTmvMediaFrameProducer::SetStartTimecode(const FTimecode& InStartTimecode)
{
	FScopeLock lock(&TrackInfoLock);
	StartTimecode.Emplace(InStartTimecode);
}

void UTmvMediaFrameProducer::SetStartTimecodeRate(const FFrameRate& InStartTimecodeRate)
{
	FScopeLock lock(&TrackInfoLock);
	StartTimecodeRate.Emplace(InStartTimecodeRate);
}
