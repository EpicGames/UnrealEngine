// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaFrameRender.h"

#include "ColorManagement/TransferFunctions.h"
#include "MediaShaders.h"
#include "PixelShaderUtils.h"
#include "RenderGraphUtils.h"
#include "TextureResource.h"
#include "TmvMediaLog.h"
#include "TmvTextureYuvConvertShader.h"
#include "Utils/TmvMediaFrameUtils.h"
#include "Utils/TmvMediaGpuReadback.h"

namespace UE::TmvMedia
{
	void AddCopyTexturePass(FRDGBuilder& InBuilder, FRDGTexture* InSourceTexture, FRDGTexture* InDestTarget)
	{
		if (!InSourceTexture || !InDestTarget)
		{
			return;
		}

		const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		// Note: this also deals with differing formats.
		FRDGDrawTextureInfo DrawInfo;
		DrawInfo.NumMips = FMath::Min(InSourceTexture->Desc.NumMips, InDestTarget->Desc.NumMips); // Convert all the mips.
		AddDrawTexturePass(InBuilder, GlobalShaderMap, InSourceTexture, InDestTarget, DrawInfo);
	}

	/**
	* Returns the maximum value that can be represented with the given bit depth.
	*/
	uint32 GetMaxEncodeValue(uint8 InBitDepth)
	{
		if (InBitDepth == 0)
		{
			return 0u;
		}
		if (InBitDepth >= 32)
		{
			return 0xFFFFFFFFu;
		}
		return (1u << static_cast<uint32>(InBitDepth)) - 1u;
	}

	/**
	 * Scale factor for normalized _UNORM texture lookup.
	 * This is a scaling factor to store data on InBitDepth from a full byte aligned bit range.
	 */
	float GetEncodeDataScale(ETmvMediaFrameComponentType InType, uint8 InBitDepth)
	{
		if (InType == ETmvMediaFrameComponentType::Int && InBitDepth > 0)
		{
			const uint8 ByteAlignedBits = (InBitDepth + 7u) & ~7u;
			if (ByteAlignedBits > 0)
			{
				return static_cast<float>(GetMaxEncodeValue(InBitDepth))/static_cast<float>(GetMaxEncodeValue(ByteAlignedBits));
			}
		}
		return 1.0f;
	}

	void AddConvertTexturePass(FRDGBuilder& InBuilder, FRDGTexture* InSourceTexture, FRDGTexture* InDestTarget, const FTmvMediaFrameMipInfo& InTargetMipInfo)
	{
		if (!InSourceTexture || !InDestTarget)
		{
			return;
		}

		const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		FRDGDrawTextureInfo DrawInfo;
		DrawInfo.NumMips = FMath::Min(InSourceTexture->Desc.NumMips, InDestTarget->Desc.NumMips); // Convert all the mips.
		const FRDGTextureDesc& OutputDesc = InDestTarget->Desc;

		const FIntPoint DrawSize = DrawInfo.Size == FIntPoint::ZeroValue ? OutputDesc.Extent : DrawInfo.Size;

		// Don't load color data if the whole texture is being overwritten.
		const ERenderTargetLoadAction LoadAction = (DrawInfo.DestPosition == FIntPoint::ZeroValue && DrawSize == OutputDesc.Extent)
			? ERenderTargetLoadAction::ENoAction 
			: ERenderTargetLoadAction::ELoad;

		TShaderMapRef<FTmvTextureYuvConverterPS> PixelShader(GlobalShaderMap);

		for (uint32 MipIndex = 0; MipIndex < DrawInfo.NumMips; ++MipIndex)
		{
			const int32 SourceMipIndex = MipIndex + DrawInfo.SourceMipIndex;
			const int32 DestMipIndex = MipIndex + DrawInfo.DestMipIndex;
			const FIntPoint MipDrawSize(FMath::Max(DrawSize.X >> MipIndex, 1), FMath::Max(DrawSize.Y >> MipIndex, 1));

			for (uint32 SliceIndex = 0; SliceIndex < DrawInfo.NumSlices; ++SliceIndex)
			{
				const int32 SourceSliceIndex = SliceIndex + DrawInfo.SourceSliceIndex;
				const int32 DestSliceIndex = SliceIndex + DrawInfo.DestSliceIndex;

				FRDGTextureSRVDesc SRVDesc = FRDGTextureSRVDesc::CreateForMipLevel(InSourceTexture, SourceMipIndex);
				SRVDesc.FirstArraySlice = SourceSliceIndex;

				FTmvTextureYuvConverterPS::FParameters* PassParameters = InBuilder.AllocParameters<FTmvTextureYuvConverterPS::FParameters>();
				// Set up the parameters
				PassParameters->UVScaleAndOffset = FVector4f(1.0f/MipDrawSize.X, 1.0f/MipDrawSize.Y, 0.0f, 0.0f);
				PassParameters->InTexture = InBuilder.CreateSRV(SRVDesc);
				PassParameters->InTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
				PassParameters->ColorParams.AlphaScale = 1.0f;
				PassParameters->ColorParams.MipTint = FLinearColor(1.0f, 1.0f, 1.0f, 0.0f);

				const Color::FColorSpace& SourceCS = Color::FColorSpace::GetWorking();
				Color::FColorSpace DestCS = Color::FColorSpace(InTargetMipInfo.ColorInfo.ColorSpace);
				const float LinearPreScale = Color::GetReferenceWhiteLinearScale(InTargetMipInfo.ColorInfo.Encoding, InTargetMipInfo.ColorInfo.ReferenceWhiteOverride, Color::EReferenceWhiteDirection::Encode);
				if (SourceCS.Equals(DestCS) && LinearPreScale == 1.0f)
				{
					PassParameters->ColorParams.bApplyColorTransform = 0u;
					PassParameters->ColorParams.ColorSpaceMatrix = FMatrix44f::Identity;
				}
				else
				{
					Color::EChromaticAdaptationMethod Method = Color::DEFAULT_CHROMATIC_ADAPTATION_METHOD;
					FMatrix44f ColorSpaceMatrix = SourceCS.Equals(DestCS)
						? FMatrix44f::Identity
						: Color::Transpose<float>(Color::FColorSpaceTransform(SourceCS, DestCS, Method));
					PassParameters->ColorParams.ColorSpaceMatrix = ColorSpaceMatrix * LinearPreScale;
					PassParameters->ColorParams.bApplyColorTransform = 1u;
				}

				PassParameters->ColorParams.EOTF = static_cast<uint32>(InTargetMipInfo.ColorInfo.Encoding);
				PassParameters->ColorParams.bConvertYUV = InTargetMipInfo.ColorModel == ETmvMediaFrameColorModel::YUV ? 1u : 0u;

				const int BitDepth = InTargetMipInfo.Planes.IsValidIndex(0) ? InTargetMipInfo.Planes[0].BitDepth : 16;
				const ETmvMediaFrameComponentType ComponentType = InTargetMipInfo.Planes.IsValidIndex(0) ? InTargetMipInfo.Planes[0].Type : ETmvMediaFrameComponentType::Int;					

				if (InTargetMipInfo.ColorInfo.YuvMatrix != ETmvMediaFrameColorMatrix::Identity)
				{
					const FMatrix YuvMatrix = FrameUtils::GetRgbToYuvMatrix(InTargetMipInfo.ColorInfo.YuvMatrix, InTargetMipInfo.ColorInfo.YuvMatrixRange);
					const FVector YuvOffset = FrameUtils::GetYuvOffset(ComponentType, InTargetMipInfo.ColorInfo.YuvMatrixRange, BitDepth);
					PassParameters->ColorParams.YUVMatrix = static_cast<FMatrix44f>(
						MediaShaders::CombineColorTransformAndOffset(YuvMatrix, YuvOffset));
				}
				else
				{
					PassParameters->ColorParams.YUVMatrix = FMatrix44f::Identity;
				}

				float DataScale = GetEncodeDataScale(ComponentType, BitDepth);
				PassParameters->DataScale = FVector4f(DataScale, DataScale, DataScale, DataScale);
				
				PassParameters->RenderTargets[0] = FRenderTargetBinding(InDestTarget, LoadAction, DestMipIndex, DestSliceIndex);

				const FIntRect ViewRect(DrawInfo.DestPosition, DrawInfo.DestPosition + DrawSize);

				FPixelShaderUtils::AddFullscreenPass(
					InBuilder,
					GlobalShaderMap,
					RDG_EVENT_NAME("ConvertMip ([%s, Mip: %d, Slice: %d] -> [%s, Mip: %d, Slice: %d])", InSourceTexture->Name, SourceMipIndex,
						SourceSliceIndex, InDestTarget->Name, DestMipIndex, DestSliceIndex),
					PixelShader,
					PassParameters,
					ViewRect
				);
			}
		}
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FEnqueueCopyTexturePass, )
		RDG_TEXTURE_ACCESS(Texture, ERHIAccess::CopySrc)
	END_SHADER_PARAMETER_STRUCT()

	void AddEnqueueCopyPass(FRDGBuilder& InBuilder, FTmvMediaGpuTextureReadback* InReadback, FRDGTextureRef InSourceTexture, uint32 InMipIndex)
	{
		FEnqueueCopyTexturePass* PassParameters = InBuilder.AllocParameters<FEnqueueCopyTexturePass>();
		PassParameters->Texture = InSourceTexture;

		FTmvMediaTextureReadbackParams ReadbackParams;
		ReadbackParams.SourceMip = InMipIndex;

		InBuilder.AddPass(
			RDG_EVENT_NAME("EnqueueCopy(%s)", InSourceTexture->Name),
			PassParameters,
			ERDGPassFlags::Readback,
			[InReadback, InSourceTexture, ReadbackParams](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			InReadback->EnqueueCopy(RHICmdList, InSourceTexture->GetRHI(), ReadbackParams);
		});
	}
}
