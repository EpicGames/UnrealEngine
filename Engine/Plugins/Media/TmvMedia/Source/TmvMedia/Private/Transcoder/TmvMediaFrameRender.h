// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphResources.h"

class FTmvMediaGpuTextureReadback;
struct FTmvMediaFrameMipInfo;

/**
 * Conversion and transcoding helper functions for rendering media frames.
 */
namespace UE::TmvMedia
{
	/** 
	 * Add a texture copy pass from source to destination for all mips.
	 * Can do basic format conversion with a simple "draw texture" pass.
	 */
	void AddCopyTexturePass(FRDGBuilder& InBuilder, FRDGTexture* InSourceTexture, FRDGTexture* InDestTarget);

	/** 
	 * Add a texture conversion pass from source to destination.
	 */
	void AddConvertTexturePass(FRDGBuilder& InBuilder, FRDGTexture* InSourceTexture, FRDGTexture* InDestTarget, const FTmvMediaFrameMipInfo& InTargetMipInfo);

	/**
	 * Add a texture copy pass for readback in a staging cpu buffer.
	 */
	void AddEnqueueCopyPass(FRDGBuilder& InBuilder, FTmvMediaGpuTextureReadback* InReadback, FRDGTextureRef InSourceTexture, uint32 InMipIndex);
}


