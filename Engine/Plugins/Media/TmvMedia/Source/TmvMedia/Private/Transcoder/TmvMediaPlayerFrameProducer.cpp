// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaPlayerFrameProducer.h"

#include "Async/Async.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IMediaEventSink.h"
#include "MediaDelegates.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ScreenPass.h"
#include "TextureResource.h"
#include "TmvMediaLog.h"
#include "TmvMediaPlayerFrameProducerResource.h"
#include "Transcoder/TmvMediaFrameConverter.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Utils/TmvMediaTimeUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaPlayerFrameProducer)

#define LOCTEXT_NAMESPACE "TmvMediaPlayerFrameProducer"

/** Timespan logging utility functions. */
namespace UE::TmvMedia
{
	/**
	 * Utility function to convert a timespan to pretty formatted string for logging. 
	 */
	FString ToString(const FTimespan& InTimespan)
	{
		// The selected format displays the fraction as ticks "t" to be consistent with
		// LogMediaAssets (MediaPlayer), LogMediaUtils (PlayerFacade) and sequence media track.
		// LogElectraPlayer is a mix of seconds (playback controls) or ticks (Presentation time stamps).
		return InTimespan.ToString(TEXT("%h:%m:%s.%t"));
	}

	/** Utility function to convert a timespan bound to formatted string for logging. */
	FString ToString(const TRange<FTimespan>::BoundsType& InBounds)
	{
		// Only call GetValue() if the bound is closed.
		if (InBounds.IsClosed())
		{
			return ToString(InBounds.GetValue());
		}
		return TEXT("<open>");
	}
	
	FString GetTimecodeStringFromMediaSource(const UMediaPlayer* InMediaPlayer)
	{
		if (UMediaPlaylist* Playlist = InMediaPlayer->GetPlaylist())
		{
			if (const UMediaSource* MediaSource = Playlist->Get(InMediaPlayer->GetPlaylistIndex()))
			{
				return MediaSource->GetMediaOption(UMediaPlayer::MediaInfoNameStartTimecodeValue.Resolve(), FString());
			}
		}
		return FString();
	}
}

UTmvMediaPlayerFrameProducer::UTmvMediaPlayerFrameProducer()
{
	bHasAsyncQueue = true;
}

bool UTmvMediaPlayerFrameProducer::Start(UTmvMediaTranscodeJob* InParentJob)
{
	if (!InParentJob)
	{
		UE_LOGF(LogTmvMedia, Error, "UTmvMediaPlayerFrameProducer::Start: InParentJob is null.");
		return false;
	}

	MaxProcessingFrames = InParentJob->Settings.NumProducerThreads;
	if (MaxProcessingFrames <= 0)
	{
		MaxProcessingFrames = 8;	// Default producer threads. Todo: Make this configurable?
	}

	// Upper bound for this to avoid overloading the system. Todo: Make this configurable?
	MaxProcessingFrames = FMath::Min(MaxProcessingFrames, FPlatformMisc::NumberOfCores());

	// Create media source.
	if (InParentJob->Settings.InputSource == ETmvMediaTranscodeInputSource::File)
	{
		MediaSource = UMediaSource::SpawnMediaSourceForString(InParentJob->Settings.GetAbsoluteInputPath(), this);
		if (MediaSource == nullptr)
		{
			const FText Message = FText::Format(LOCTEXT("FailedToCreateMediaSource", 
				"Failed to create a media source from path \"{0}\""),
				FText::FromString(InParentJob->Settings.GetInputPath()));
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			UE_LOGF(LogTmvMedia, Error, "MediaPlayerFrameProducer::Start: %ls.", *Message.ToString());
			return false;
		}
	}
	else
	{
		MediaSource = InParentJob->Settings.InputMediaSource.LoadSynchronous();
		if (MediaSource == nullptr)
		{
			const FText Message = FText::Format(LOCTEXT("InvalidMediaSourceAsset", 
				"Invalid Media Source Asset: \"{0}\""),
				FText::FromString(InParentJob->Settings.InputMediaSource.ToString()));
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			UE_LOGF(LogTmvMedia, Error, "MediaPlayerFrameProducer::Start: %ls.", *Message.ToString());
			return false;
		}
	}

	// Create player.
	MediaPlayer = NewObject<UMediaPlayer>(this, "MediaPlayer", RF_Transient);
	MediaPlayer->PlayOnOpen = false;
	MediaPlayer->SetLooping(false);

	// UMediaPlayer::HandlePlayerMediaEvent
	MediaPlayer->OnMediaClosed.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaClosed);
	MediaPlayer->OnMediaOpened.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaOpened);
	MediaPlayer->OnMediaOpenFailed.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaOpenFailed);
	MediaPlayer->OnEndReached.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaEndReached);
	MediaPlayer->OnPlaybackResumed.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaResumed);
	MediaPlayer->OnPlaybackSuspended.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaSuspended);
	MediaPlayer->OnSeekCompleted.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaSeekCompleted);
	MediaPlayer->OnTracksChanged.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaTrackChanged);
	MediaPlayer->OnBufferingStart.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaBuffering);
	MediaPlayer->OnBufferingCompleted.AddDynamic(this, &UTmvMediaPlayerFrameProducer::OnMediaBufferingCompleted);

	// Low-level event multicast used to seek-on-open early enough that the seek lands before any
	// first-frame sample is auto-emitted. Mirrors FMovieSceneMediaData::HandleMediaPlayerEvent.
	MediaPlayer->OnMediaEvent().AddUObject(this, &UTmvMediaPlayerFrameProducer::HandleMediaPlayerEvent);

	// Create texture.
	MediaTexture = NewObject<UMediaTexture>(this, "MediaTexture", RF_Transient);
	MediaTexture->SetMediaPlayer(MediaPlayer);
	// Have the media texture generate the mipmaps (if possible).
	MediaTexture->EnableGenMips = InParentJob->Settings.bEnableMipMapping;
	MediaTexture->UpdateResource();

	ProducerResource = MakeShared<FTmvMediaPlayerFrameProducerResource>(InParentJob, MediaTexture);

	// Note: using a weak ptr to the resource in case it gets cancelled before the render command runs.
	ENQUEUE_RENDER_COMMAND(InitProducerResource)([ProducerResourceWeak = ProducerResource.ToWeakPtr()](FRHICommandListImmediate& InRHICmdList)
	{
		if (TSharedPtr<FTmvMediaPlayerFrameProducerResource> ProducerResource = ProducerResourceWeak.Pin())
		{
			ProducerResource->InitRHI(InRHICmdList);
		}
	});

	// Start playing.
	MediaStartTime = CurrentTime = MediaFrameDuration = FTimespan::Zero();
	MediaPlaybackRange = TRange<FTimespan>::Empty();
	MediaFrameRate = FFrameRate(0, 0);
	PreviousRenderedSampleTime = FTimespan::MinValue();

	FMediaPlayerOptions Options;
	Options.SetAllAsOptional();

	// Environment_Sequencer vs Environment_Preview:
	// in Environment_Preview, the player will not try to pre-cache as it is not necessary for transcoding.
	Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::Environment(), MediaPlayerOptionValues::Environment_Preview());
	
	// Read the time code from the source media if available.
	Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ParseTimecodeInfo(), FVariant());
	
	Options.PlayOnOpen = EMediaPlayerOptionBooleanOverride::Disabled;
	Options.Loop = EMediaPlayerOptionBooleanOverride::Disabled;
	// Let the media start at whichever time it defaults to.
	Options.SeekTimeType = EMediaPlayerOptionSeekTimeType::Ignored;
	// We don't need audio.
	Options.Tracks.Audio = -1;
	// For image media, we avoid filling the global cache which will needlessly hold onto frame data.
	Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheEnabled(), FVariant(true));
	// Setting a look ahead time to 0 means the minimum of precaching, which should be 2 frames.
	Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheTimeToLookAhead(), FVariant(0.0f));
	Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ViewMediaTexture(), FVariant(FSoftObjectPath(MediaTexture).ToString()));

	CurrentState = EState::Opening;
	WatchdogTimeRemaining = 10.0f;	// We will wait up to 10 seconds for the video to open.
	
	if (!MediaPlayer->OpenSourceWithOptions(MediaSource, Options))
	{
		Close(InParentJob);
		return false;
	}

	return Super::Start(InParentJob);
}

void UTmvMediaPlayerFrameProducer::RequestStop(UTmvMediaTranscodeJob* InParentJob)
{
	Super::RequestStop(InParentJob);
	Close(InParentJob);
}

void UTmvMediaPlayerFrameProducer::Tick(UTmvMediaTranscodeJob* InParentJob, const FTmvMediaTranscodeJobTime& InTime)
{
	using namespace UE::TmvMedia;

	if (!InParentJob)
	{
		UE_LOGF(LogTmvMedia, Error, "UTmvMediaPlayerFrameProducer::Tick: InParentJob is null. Stopping Stage.");
		RequestStop(InParentJob);
		return;
	}
	
	if (!MediaPlayer || !MediaTexture || !MediaSource)
	{
		// Should technically be closed or errored.
		ensure(CurrentState == EState::Closed || CurrentState == EState::Errored);
		ensure(GetStageStatus() == ETmvMediaTranscodeStageStatus::Stopped);
		return;
	}

	// Keep the delta time in check. We typically create a thumbnail after having selected a new source
	// and if that brought up the system file selector box the time spent in the file selector is included
	// in the delta time.
	float DeltaTime = FMath::Min(InTime.DeltaTime, 0.1f);
	
	// Note: using a weak ptr to the resource in case it gets canceled before the render command runs.
	ENQUEUE_RENDER_COMMAND(ProcessPendingReadbackRequests)([ProducerResourceWeak = ProducerResource.ToWeakPtr(), InTime](FRHICommandListImmediate& InRHICmdList)
	{
		if (const TSharedPtr<FTmvMediaPlayerFrameProducerResource> ProducerResource = ProducerResourceWeak.Pin())
		{
			ProducerResource->ProcessPendingReadbackRequests(InRHICmdList, InTime);
		}
	});
	
	switch(CurrentState)
	{
	case EState::Closed:
	case EState::Errored:
		{
			break;
		}
	case EState::Opening:
		{
			// Make sure this doesn't drag on forever.
			if ((WatchdogTimeRemaining -= DeltaTime) < 0.0f)
			{
				UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Timed out waiting for media to open.");
				CurrentState = EState::TimedOut;
			}
			break;
		}
	case EState::Open:
		{
			// Make sure this doesn't drag on forever.
			if ((WatchdogTimeRemaining -= DeltaTime) < 0.0f)
			{
				UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Timed out waiting for media to start rendering.");
				CurrentState = EState::TimedOut;
				return;
			}

			TryUpdateMediaInfo();

			// Wait for the first frame to be rendered (while player is paused).
			// WARNING: This may not happen for all players. Some players don't produce frames until SetRate(1).
			// This method has been tested with Electra and ImgMedia players.
			if (ProducerResource)
			{
				const FTimespan RenderedFrameTime = ProducerResource->GetLastRenderedSampleTime();

				if (RenderedFrameTime != FTimespan::MinValue())
				{
					// Consider the first rendered frame to be the start of the video.
					MediaStartTime = CurrentTime = RenderedFrameTime;
					UE_LOGF(LogTmvMedia, Verbose,
						"Transcode Frame Producer: Received First Frame %ls",
						*ToString(MediaStartTime));

					const FTimespan RenderedFrameDuration = ProducerResource->GetLastRenderedSampleDuration();

					// Validation logging. Doesn't mean things won't work, but might need investigating.
					if (MediaFrameDuration != FTimespan::Zero() && MediaFrameDuration != RenderedFrameDuration)
					{
						UE_LOGF(LogTmvMedia, Warning,
							"Transcode Frame Producer: Frame duration mismatch: %ls (player) vs %ls (sample).",
							*ToString(MediaFrameDuration), *ToString(RenderedFrameDuration));
					}
					
					// We need to guard in case the provided frame duration is zero, getting the job stuck.
					const FTimespan FrameDuration = (RenderedFrameDuration > FTimespan::Zero()) ? RenderedFrameDuration : MediaFrameDuration;

					if (FrameDuration <= FTimespan::Zero())
					{
						UE_LOGF(LogTmvMedia, Error,
							"Transcode Frame Producer: Can't safely block/advance player without a frame duration: %ls (player) vs %ls (sample).",
							*ToString(MediaFrameDuration), *ToString(RenderedFrameDuration));
						CurrentState = EState::Failed;
						return;
					}
					
					// Since we already rendered the first frame, we want to block on the next one.
					CurrentTime += FrameDuration;
					UE_LOGF(LogTmvMedia, Verbose,
						"Transcode Frame Producer: Initial SetBlockOnTimeRange [%ls, %ls) (target frame after first).",
						*ToString(CurrentTime), *ToString(CurrentTime + FrameDuration));
					MediaPlayer->SetBlockOnTimeRange(TRange<FTimespan>(CurrentTime, CurrentTime + FrameDuration));
				
					MediaPlayer->Play();
					if (MediaPlayer->IsPreparing())
					{
						CurrentState = EState::Preparing;
					}
					else
					{
						CurrentState = EState::Playing;
					}
					WatchdogTimeRemaining = 5.0f;
				}
			}
			else
			{
				UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Render resource not available.");
				CurrentState = EState::Failed;
			}
			break;
		}
	case EState::Preparing:
		{
			if (MediaPlayer->IsPreparing())
			{
				if ((WatchdogTimeRemaining -= DeltaTime) < 0.0f)
				{
					UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Timed out waiting for media to prepare.");
					CurrentState = EState::TimedOut;
					return;
				}
			}
			else
			{
				CurrentState = EState::Playing;
				WatchdogTimeRemaining = 5.0f;	// Reset watchdog.
				HandlePlaying(InParentJob, DeltaTime);
			}
			break;
		}
	case EState::Playing:
		{
			HandlePlaying(InParentJob, DeltaTime);
			break;
		}
	case EState::MediaCompleted:

		if ((WatchdogTimeRemaining -= DeltaTime) < 0.0f)
		{
			UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Timed out waiting for media to finish encoding.");
			CurrentState = EState::TimedOut;
		}

		if (ProducerResource)
		{
			const int32 CurrentSubmittedCount = ProducerResource->GetSubmittedCount();
			
			int32 RenderedFrameIndex = 0;
			int32 TotalFrames = 0;
			const FTimespan RenderedSampleTime = ProducerResource->GetLastRenderedSampleTime();
			if (RenderedSampleTime != FTimespan::MinValue())
			{
				const FTimespan RelativeTime = RenderedSampleTime - MediaStartTime;
				const FFrameRate FrameRate = TimeUtils::GetOrComputeFrameRate(MediaFrameRate, ProducerResource->GetLastRenderedSampleDuration());
				if (FrameRate.IsValid() && !MediaPlaybackRange.IsEmpty())
				{
					RenderedFrameIndex = TimeUtils::RoundToFrameIndex(FrameRate, RelativeTime.GetTotalSeconds());
					const FTimespan SequenceDuration = MediaPlaybackRange.Size<FTimespan>();
					TotalFrames = TimeUtils::RoundToFrameIndex(FrameRate, SequenceDuration.GetTotalSeconds());
				}
			}

			if (CurrentSubmittedCount != LastSubmittedCount)
			{
				LastSubmittedCount = CurrentSubmittedCount;

				// Keep updating the stats
				UTmvMediaTranscodeJob::SafeUpdateProgress(InParentJob, FMath::Min(RenderedFrameIndex + 1, TotalFrames), TotalFrames);

				// Reset watchdog. Waiting for encoding (slow).
				WatchdogTimeRemaining = 10.0f;
			}

			// Wait for final captures to process.
			ensure(CurrentSubmittedCount >= 0);
			if (CurrentSubmittedCount == 0)
			{
				int32 FramesProduced = RenderedFrameIndex + 1;
				
				if ((TotalFrames > 0) && (FramesProduced < TotalFrames))
				{
					UE_LOGF(LogTmvMedia, Warning,
						"Transcode Frame Producer: Completed SHORT. FramesProduced=%d TotalFrames=%d LastRendered=%ls.",
						FramesProduced, TotalFrames, *ToString(RenderedSampleTime));
				}
				else
				{
					UE_LOGF(LogTmvMedia, Verbose,
						"Transcode Frame Producer: Completed. FramesProduced=%d TotalFrames=%d LastRendered=%ls.",
						FramesProduced, TotalFrames, *ToString(RenderedSampleTime));
				}

				RequestStop(InParentJob);
			}
		}
		else
		{
			UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Render resource not available.");
			CurrentState = EState::Failed;
		}
		break;
	case EState::TimedOut:
	case EState::NotSupported:
	case EState::Failed:
		{
			// Opening failed. No point in doing anything, so close everything.
			CurrentState = EState::Errored;
			RequestStop(InParentJob);
			return;
		}
	}
}

void UTmvMediaPlayerFrameProducer::HandlePlaying(UTmvMediaTranscodeJob* InParentJob, float InDeltaTime)
{
	using namespace UE::TmvMedia;
	
	if (!ProducerResource)
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Render resource not available.");
		CurrentState = EState::Failed;
		return; // error, not started.
	}

	// Get which frame the player is on.
	const FTimespan RenderedSampleTime = ProducerResource->GetLastRenderedSampleTime();

	if ((WatchdogTimeRemaining -= InDeltaTime) < 0.0f)
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Timed out waiting for media to render the next frame.");
		CurrentState = EState::TimedOut;
		return;
	}

	if (RenderedSampleTime == FTimespan::MinValue())
	{
		return;	// not rendered yet.
	}

	// Detect when new frames are rendered.
	if (RenderedSampleTime != PreviousRenderedSampleTime)
	{
		WatchdogTimeRemaining = 5.0f;	// Reset watchdog for next frame.
	}

	PreviousRenderedSampleTime = RenderedSampleTime;

	if (MediaFrameDuration == FTimespan::Zero())
	{
		// Try to update one last time.
		TryUpdateFrameDuration();

		// We need a valid frame duration for transcoding to work properly.
		if (MediaFrameDuration == FTimespan::Zero())
		{
			UE_LOGF(LogTmvMedia, Error, "Unable to get a valid frame duration for \"%ls\". Transcoding will fail.", *MediaSource->GetUrl());
			UE_LOGF(LogTmvMedia, Error, "This is likely a variable frame rate media and is not supported.");
			CurrentState = EState::NotSupported;
			return;
		}
	}

	// We need to guard in case the provided frame duration is zero, getting the job stuck.
	const FTimespan RenderedSampleDuration = ProducerResource->GetLastRenderedSampleDuration();
	const FTimespan FrameDuration = (RenderedSampleDuration > FTimespan::Zero()) ? RenderedSampleDuration : MediaFrameDuration;

	if (FrameDuration <= FTimespan::Zero())
	{
		UE_LOGF(LogTmvMedia, Error,
			"Transcode Frame Producer: Can't safely block/advance player without a frame duration: %ls (player) vs %ls (sample).",
			*ToString(MediaFrameDuration), *ToString(RenderedSampleDuration));
		CurrentState = EState::Failed;
		return;
	}
	
	const FTimespan EndRenderedSampleTime = RenderedSampleTime + FrameDuration;

	// Render time caught up with play time?
	if (EndRenderedSampleTime >= CurrentTime)
	{
		const FFrameRate FrameRate = TimeUtils::GetOrComputeFrameRate(MediaFrameRate, FrameDuration);
		int32 RenderedFrameIndex = 0;
		int32 TotalFrames = 0;

		if (FrameRate.IsValid())
		{
			const FTimespan RelativeFrameTime = RenderedSampleTime - MediaStartTime;
			RenderedFrameIndex = TimeUtils::RoundToFrameIndex(FrameRate, RelativeFrameTime.GetTotalSeconds());

			if (!MediaPlaybackRange.IsEmpty())
			{
				const FTimespan SequenceDuration = MediaPlaybackRange.Size<FTimespan>();
				TotalFrames = TimeUtils::RoundToFrameIndex(FrameRate, SequenceDuration.GetTotalSeconds());
			}
		}

		// Throttle the number of frames we send to avoid flooding encoding.
		if (ProducerResource->GetSubmittedCount() < MaxProcessingFrames)
		{
			// Update notification.
			UTmvMediaTranscodeJob::SafeUpdateProgress(InParentJob, FMath::Min(RenderedFrameIndex+1, TotalFrames), TotalFrames);

			// Move to the next frame in the sequence.
			CurrentTime = RenderedSampleTime + FrameDuration;

			UE_LOGF(LogTmvMedia, Verbose, "Transcode Frame Producer: Moving to next frame: Rendered Time:%ls (Index: %d) Rendered Duration:%ls",
				*ToString(RenderedSampleTime), RenderedFrameIndex, *ToString(FrameDuration));

			if (!MediaPlaybackRange.IsEmpty())
			{
				// For detecting the last frame, it is possible there are small imprecision on the upper bound
				// of the range or the frame's duration. Those values may be rounded down or up by 1 tick depending on players or
				// containers. We will give a tolerance of a small fraction of a frame to the detection of the last frame to compensate.
				const FTimespan UpperBoundTolerance = FMath::Max(FrameDuration * 0.1, FTimespan(2));
				if (CurrentTime < MediaPlaybackRange.GetUpperBoundValue() - UpperBoundTolerance)
				{
					const TRange<FTimespan> NextBlockOnRange = TRange<FTimespan>(CurrentTime + FTimespan(1), CurrentTime + FrameDuration);
					MediaPlayer->SetBlockOnTimeRange(NextBlockOnRange);
				}
				else
				{
					// Warn if the last expected frame may not have been rendered yet.
					// CurrentTime here is the *next* frame's expected start time; the player should already
					// have produced frame (RenderedFrameIndex) but the final one may still be in flight.
					const int32 LastSubmittedSoFar = ProducerResource->GetSubmittedCount();
					
					if ((RenderedFrameIndex + 1) < TotalFrames)
					{
						UE_LOGF(LogTmvMedia, Warning,
							"Transcode Frame Producer: Media completed but only %d/%d frames rendered. NextTime=%ls Range=[%ls, %ls] Tolerance=%ls LastRendered=%ls Submitted=%d.",
							RenderedFrameIndex + 1, TotalFrames,
							*ToString(CurrentTime),
							*ToString(MediaPlaybackRange.GetLowerBound()), *ToString(MediaPlaybackRange.GetUpperBound()),
							*ToString(UpperBoundTolerance),
							*ToString(RenderedSampleTime),
							LastSubmittedSoFar);
					}
					else
					{
						UE_LOGF(LogTmvMedia, Verbose,
							"Transcode Frame Producer: Media completed: NextTime=%ls Range=[%ls, %ls] Tolerance=%ls LastRendered=%ls RenderedFrame=%d/%d Submitted=%d.",
							*ToString(CurrentTime),
							*ToString(MediaPlaybackRange.GetLowerBound()), *ToString(MediaPlaybackRange.GetUpperBound()),
							*ToString(UpperBoundTolerance),
							*ToString(RenderedSampleTime),
							RenderedFrameIndex + 1, TotalFrames,
							LastSubmittedSoFar);
					}

					// Frame production is completed on the player side, need to wait for the rest of the stages to finish.
					CurrentState = EState::MediaCompleted;
					WatchdogTimeRemaining = 10.0f;
					LastSubmittedCount = LastSubmittedSoFar;
					// removing the block on range ensures the player can complete without blocking.
					MediaPlayer->SetBlockOnTimeRange(TRange<FTimespan>::Empty());
				}
			}
		}
		else
		{
			// Since the encoder can be very slow, we want to reset the producer's watchdog while
			// we wait, otherwise this could lead to timeouts because of the encoder.
			WatchdogTimeRemaining = 5.0f;

			UE_LOGF(LogTmvMedia, Verbose, "Transcode Frame Producer: WAITING  Player Time: %ls Rendered Time: %ls Playback Range: [%ls, %ls] Frame: %d",
				*ToString(CurrentTime),
				*ToString(RenderedSampleTime),
				*ToString(MediaPlaybackRange.GetLowerBound()),
				*ToString(MediaPlaybackRange.GetUpperBound()),
				RenderedFrameIndex);
		}
	}
}

void UTmvMediaPlayerFrameProducer::Close(const UTmvMediaTranscodeJob* InParentJob)
{
	if (ProducerResource)
	{
		// Note: using a shared ptr to keep it alive until the command is executed.
		ENQUEUE_RENDER_COMMAND(ReleaseProducerResource)([ProducerResource = MoveTemp(ProducerResource)](FRHICommandListImmediate& InRHICmdList)
		{
			ProducerResource->ReleaseRHI();
		});
	}

	if (MediaTexture)
	{
		MediaTexture->SetMediaPlayer(nullptr);
		MediaTexture = nullptr;
	}
	
	if (MediaPlayer)
	{
		MediaPlayer->OnMediaClosed.RemoveAll(this);
		MediaPlayer->OnMediaOpened.RemoveAll(this);
		MediaPlayer->OnMediaOpenFailed.RemoveAll(this);
		MediaPlayer->OnEndReached.RemoveAll(this);
		MediaPlayer->OnPlaybackResumed.RemoveAll(this);
		MediaPlayer->OnPlaybackSuspended.RemoveAll(this);
		MediaPlayer->OnSeekCompleted.RemoveAll(this);
		MediaPlayer->OnTracksChanged.RemoveAll(this);
		MediaPlayer->OnBufferingStart.RemoveAll(this);
		MediaPlayer->OnBufferingCompleted.RemoveAll(this);
		MediaPlayer->OnMediaEvent().RemoveAll(this);

		MediaPlayer->Close();
		MediaPlayer = nullptr;
	}

	CurrentState = EState::Closed;
	WatchdogTimeRemaining = 0.0f;
}

void UTmvMediaPlayerFrameProducer::TryUpdateMediaInfo()
{
	if (MediaPlaybackRange.IsEmpty())
	{
		TryUpdatePlaybackRange();
	}

	if (MediaFrameDuration == FTimespan::Zero())
	{
		TryUpdateFrameDuration();
	}
}

void UTmvMediaPlayerFrameProducer::TryUpdatePlaybackRange()
{
	using namespace UE::TmvMedia;
	// Check that we have a media duration. When we have that we can assume
	// that we also know which tracks are present.
	const FTimespan MediaDuration = MediaPlayer->GetDuration();
	if (MediaDuration > FTimespan::Zero() && MediaDuration < FTimespan::MaxValue())
	{
		const int32 SelectedTrack = MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Video);
		const TOptional<FTimespan> TrackDuration = MediaPlayer->GetTrackDuration(EMediaPlayerTrack::Video, SelectedTrack);

		// Priority to track duration, more precise in case other tracks in present.
		if (TrackDuration.IsSet())
		{
			MediaPlaybackRange = TRange<FTimespan>(FTimespan::Zero(), TrackDuration.GetValue());
		}
		else
		{
			// Fallback to GetPlaybackTimeRange api.
			MediaPlaybackRange = MediaPlayer->GetPlaybackTimeRange(EMediaTimeRangeType::Absolute);

			// Note: also fallback if bounds are open. IsEmpty() returns fall in that case, but it is still an invalid case.
			if (MediaPlaybackRange.IsEmpty() || !MediaPlaybackRange.HasLowerBound() || !MediaPlaybackRange.HasUpperBound())
			{
				// Fallback to GetDuration api (older api).
				MediaPlaybackRange = TRange<FTimespan>(FTimespan::Zero(), MediaDuration);
			}
		}

		UE_LOGF(LogTmvMedia, Verbose, "Transcode Frame Producer: Media Playback Range [%ls, %ls].",
			*ToString(MediaPlaybackRange.GetLowerBound()), *ToString(MediaPlaybackRange.GetUpperBound()));

		// Make sure the media has a video track.
		if (MediaPlayer->GetNumTracks(EMediaPlayerTrack::Video) <= 0 || MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, 0) < 0)
		{
			UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Media has no video track.");
			CurrentState = EState::NotSupported;
		}
		
        // Update the track info
        FTmvMediaFrameProducerTrackInfo LocalVideoTrackInfo = GetVideoTrackInfo();
		LocalVideoTrackInfo.Duration =  MediaPlaybackRange.Size<FTimespan>();
        SetVideoTrackInfo(LocalVideoTrackInfo);
		
		// Query the start timecode
		const FVariant Value = MediaPlayer->GetMediaInfo(UMediaPlayer::MediaInfoNameStartTimecodeValue.Resolve());
		const FString TimecodeString = (Value.IsEmpty() || Value.GetType() != EVariantTypes::String)
			? UE::TmvMedia::GetTimecodeStringFromMediaSource(MediaPlayer.Get())
			: Value.GetValue<FString>();
		if (!TimecodeString.IsEmpty())
		{
			const TOptional<FTimecode> Timecode = FTimecode::ParseTimecode(*TimecodeString);
			if (Timecode.IsSet())
			{
				SetStartTimecode(Timecode.GetValue());
				
				// Read the optional start timecode frame rate.
				const FVariant TimecodeRate = MediaPlayer->GetMediaInfo(UMediaPlayer::MediaInfoNameStartTimecodeFrameRate.Resolve());
				if (!TimecodeRate.IsEmpty() && TimecodeRate.GetType() == EVariantTypes::String)
				{
					FFrameRate Result;
					const FString TimecodeRateString = TimecodeRate.GetValue<FString>();
					if (TryParseString(Result, *TimecodeRateString))
					{
						SetStartTimecodeRate(Result);
					}
					else
					{
						UE_LOGF(LogTmvMedia, Error, "Frame Producer: failed to parse time code rate \"%ls\" from player.", *TimecodeRateString );
					}
				}
			}
			else
			{
				UE_LOGF(LogTmvMedia, Error, "Frame Producer: failed to parse time code \"%ls\" from player.", *TimecodeString );
			}
		}
	}
}

void UTmvMediaPlayerFrameProducer::TryUpdateFrameDuration()
{
	// We need to wait after the first "tick input" to query frame duration because that is where
	// the default track selection logic is done.
	const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);
	if (FrameRate > 0.0f)
	{
		UE_LOGF(LogTmvMedia, Verbose, "Transcode Frame Producer: Media Frame Rate is %f.", FrameRate);
		MediaFrameDuration = FTimespan::FromSeconds(1.0f / FrameRate);
        MediaFrameRate = UE::TmvMedia::TimeUtils::GetFrameRate(FrameRate);

		// Propagate to the render thread resource to compute frame indices accurately.
		ENQUEUE_RENDER_COMMAND(SetMediaFrameRate)([ProducerResourceWeak = ProducerResource.ToWeakPtr(), FrameRate = MediaFrameRate](FRHICommandListImmediate& InRHICmdList)
		{
			if (const TSharedPtr<FTmvMediaPlayerFrameProducerResource> ProducerResource = ProducerResourceWeak.Pin())
			{
				ProducerResource->SetMediaFrameRate_RenderThread(FrameRate);
			}
		});
		
		// Update the track info
		{
			FTmvMediaFrameProducerTrackInfo LocalVideoTrackInfo = GetVideoTrackInfo();
			LocalVideoTrackInfo.FrameRate = MediaFrameRate;
			SetVideoTrackInfo(LocalVideoTrackInfo);
		}
	}
}

void UTmvMediaPlayerFrameProducer::OnMediaClosed()
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media Closed");
}

void UTmvMediaPlayerFrameProducer::OnMediaOpened(FString InUrl)
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media \"%ls\" Opened", *InUrl);

	// Give it a moment in case the metadata is not immediately available.
	WatchdogTimeRemaining = 5.0f;	// Todo: tune/expose this for slower computers.
	CurrentState = EState::Open;
	MediaPlayer->SetRate(0.0f);

	TryUpdateMediaInfo();
}

void UTmvMediaPlayerFrameProducer::OnMediaOpenFailed(FString InUrl)
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media \"%ls\" Opened Failed", *InUrl);
	CurrentState = EState::Failed;
}

void UTmvMediaPlayerFrameProducer::OnMediaEndReached()
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media End Reached");

	// This event is not precise enough, we will use it only if we don't have a playback range.
	if (MediaPlaybackRange.IsEmpty())
	{
		CurrentState = EState::MediaCompleted;
		WatchdogTimeRemaining = 10.0f;
	
		// Unblock the player.
		MediaPlayer->SetBlockOnTimeRange(TRange<FTimespan>::Empty());
	}
}

void UTmvMediaPlayerFrameProducer::OnMediaResumed()
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media Resumed");
}

void UTmvMediaPlayerFrameProducer::OnMediaSuspended()
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media Suspended");
}

void UTmvMediaPlayerFrameProducer::OnMediaSeekCompleted()
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media Seek Completed");
}

void UTmvMediaPlayerFrameProducer::OnMediaTrackChanged()
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media Track Changed");
	TryUpdateMediaInfo();
}

void UTmvMediaPlayerFrameProducer::OnMediaBuffering()
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media Buffering");
}

void UTmvMediaPlayerFrameProducer::OnMediaBufferingCompleted()
{
	UE_LOGF(LogTmvMedia, Verbose, "MediaPlayerFrameProducer Media Buffering Completed");
}

void UTmvMediaPlayerFrameProducer::HandleMediaPlayerEvent(EMediaEvent Event)
{
	if (Event != EMediaEvent::MediaOpened || MediaPlayer == nullptr)
	{
		return;
	}

	if (!MediaPlayer->SupportsSeeking())
	{
		UE_LOGF(LogTmvMedia, Verbose, "Transcode Frame Producer: Media does not support seeking; skipping seek-on-open.");
		return;
	}

	// Ensure FrameDuration and playback range are queried before we use them below.
	TryUpdateMediaInfo();

	// Anchor the seek/block-on at the start of the playback range.
	const FTimespan StartTime = MediaPlaybackRange.IsEmpty()
		? FTimespan::Zero()
		: MediaPlaybackRange.GetLowerBoundValue();

	// MediaFrameDuration may still be zero here: some players (notably Electra on certain mp4s)
	// don't populate GetVideoTrackFrameRate until after the UFUNCTION OnMediaOpened, which runs
	// after this lower-level multicast. An empty block-on range lets the player auto-advance
	// past the first sample before we can lock onto it (frame 0 is silently dropped, shifting
	// every subsequent index down by one). Fall back to 1 ms, which is shorter than one frame
	// at any plausible rate (<1000 fps) so it still selects only frame 0.
	const FTimespan BlockOnDuration = (MediaFrameDuration > FTimespan::Zero())
		? MediaFrameDuration
		: FTimespan::FromMilliseconds(1);

	UE_LOGF(LogTmvMedia, Verbose,
		"Transcode Frame Producer: HandleMediaPlayerEvent(MediaOpened) -> SetRate(0) + SetBlockOnTimeRange [%ls, %ls) + Seek(%ls).",
		*UE::TmvMedia::ToString(StartTime),
		*UE::TmvMedia::ToString(StartTime + BlockOnDuration),
		*UE::TmvMedia::ToString(StartTime));

	// Seek-on-open + block-on-first-frame pattern.
	// Specifying BlockOnTimeRange prior to seek ensures the correct first frame is presented.
	MediaPlayer->SetRate(0.0f);
	MediaPlayer->SetBlockOnTimeRange(TRange<FTimespan>(StartTime, StartTime + BlockOnDuration));
	MediaPlayer->Seek(StartTime);
}

#undef LOCTEXT_NAMESPACE
