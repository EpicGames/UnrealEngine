// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaPlayerFrameProducerResource.h"

#include "Async/Async.h"
#include "MediaDelegates.h"
#include "MediaTexture.h"
#include "PixelShaderUtils.h"
#include "RenderGraphUtils.h"
#include "SampleConverter/TmvMediaFrameMipMemoryBuffer.h"
#include "TextureResource.h"
#include "TmvMediaFrameRender.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaFrameConverter.h"
#include "Transcoder/TmvMediaFrameEncoder.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Utils/TmvMediaFrameUtils.h"
#include "Utils/TmvMediaGpuReadback.h"
#include "Utils/TmvMediaTimeUtils.h"

DECLARE_GPU_STAT_NAMED(TmvMediaFrameProducer_ConvertFrame, TEXT("TmvMediaFrameProducer.ConvertFrame"));
DECLARE_GPU_STAT_NAMED(TmvMediaFrameProducer_ReadbackFrame, TEXT("TmvMediaFrameProducer.ReadbackFrame"));

namespace UE::TmvMedia
{
	/**
	 * Only works for channel that have the same number of bits per channel for all channels.
	 * Some exceptions: PF_R5G6B5_UNORM, PF_FloatR11G11B10.
	 */
	uint8 CalcNumBitsPerChannel(const FPixelFormatInfo& InPixelFormatInfo)
	{
		if (InPixelFormatInfo.NumComponents == 0)
		{
			return 0;
		}
		
		const int32 PixelsPerBlock = InPixelFormatInfo.BlockSizeX * InPixelFormatInfo.BlockSizeY * InPixelFormatInfo.BlockSizeZ;
		if (PixelsPerBlock == 0)
		{
			return 0;
		}

		// todo: warn about exceptions?
		return 8 * InPixelFormatInfo.BlockBytes / (PixelsPerBlock * InPixelFormatInfo.NumComponents);
	}
	
	void PopulateMipInfoFromTexture(int32 InWidth, int32 InHeight, EPixelFormat InPixelFormat, int32 InMipLevel, FTmvMediaFrameMipInfo& OutMipInfo)
	{
		OutMipInfo.MipLevel = InMipLevel;
		OutMipInfo.Width = InWidth;
		OutMipInfo.Height = InHeight;
		const FPixelFormatInfo& PixelFormatInfo = GPixelFormats[InPixelFormat];
		OutMipInfo.NumComponents = PixelFormatInfo.NumComponents;
		OutMipInfo.ColorModel = ETmvMediaFrameColorModel::RGB;
		// By default, textures are considered in the "working" color space, which we represent as "None" in this case.
		OutMipInfo.ColorInfo.ColorSpace = Color::EColorSpace::None;
		OutMipInfo.ColorInfo.Encoding = IsFloatFormat(InPixelFormat) ?	Color::EEncoding::Linear : Color::EEncoding::sRGB;
		OutMipInfo.ColorInfo.YuvMatrix = ETmvMediaFrameColorMatrix::Identity;
		OutMipInfo.ColorInfo.YuvMatrixRange = ETmvMediaFrameColorMatrixRange::Full;
		OutMipInfo.TileWidth = OutMipInfo.Width;
		OutMipInfo.TileHeight = OutMipInfo.Height;
		OutMipInfo.NumTiles = FIntPoint(1,1);

		OutMipInfo.Planes.SetNumUninitialized(1);

		FTmvMediaFramePlaneInfo& PlaneInfo = OutMipInfo.Planes[0];

		PlaneInfo.NumComponents = PixelFormatInfo.NumComponents;
		PlaneInfo.BitDepth = CalcNumBitsPerChannel(PixelFormatInfo);
		PlaneInfo.Type = IsFloatFormat(InPixelFormat) ? ETmvMediaFrameComponentType::Float : ETmvMediaFrameComponentType::Int;
		PlaneInfo.Width = InWidth;
		PlaneInfo.Height = InHeight;
		PlaneInfo.Stride = PixelFormatInfo.Get2DImageSizeInBytes(InWidth, 1);
		PlaneInfo.NumLines = InHeight;
		PlaneInfo.ComponentLayout = ETmvMediaFrameComponentLayout::Packed;
		PlaneInfo.WidthRatio = PlaneInfo.HeightRatio = 1;
	}
	
	void PopulateMipInfoFromTexture(const FRHITexture* InTexture, int32 InMipLevel, FTmvMediaFrameMipInfo& OutMipInfo)
	{
		PopulateMipInfoFromTexture(InTexture->GetSizeX(), InTexture->GetSizeY(), InTexture->GetFormat(), InMipLevel, OutMipInfo);
	}

	void PopulateMipInfoFromTexture(const FRDGTexture* InTexture, int32 InMipLevel, FTmvMediaFrameMipInfo& OutMipInfo)
	{
		PopulateMipInfoFromTexture(InTexture->Desc.GetSize().X, InTexture->Desc.GetSize().Y, InTexture->Desc.Format, InMipLevel, OutMipInfo);
	}

	/**
	 * Keeps information and resources (staging buffer) for reading back a gpu texture to cpu buffer.
	 * 
	 * For reading the media texture render target, we need to keep the readback requests around
	 * and poll it to know when it has been executed and is ready to be read.
	 */
	struct FFrameReadbackRequest
	{
		void Init(const FRDGTexture* InReadbackRdg, const FTmvMediaFrameTimeInfo& InTimeInfo)
		{
			// Setup all frame info.
			Format = InReadbackRdg->Desc.Format;
			TimeInfo = InTimeInfo;

			// When receiving the pixels from the media texture, we assume it is in the default working colorspace and linear encoding.
			PopulateMipInfoFromTexture(InReadbackRdg, 0, MipInfo);

			// Setup readbacks if necessary. In case of reuse, we just make sure the number of mips is the same (it should).
			const int32 NumMips = InReadbackRdg->Desc.NumMips;
			if (Readbacks.Num() != NumMips)
			{
				Readbacks.Reset();
				Readbacks.Reserve(NumMips);
				for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
				{
					Readbacks.Add(MakeUnique<FTmvMediaGpuTextureReadback>(TEXT("MediaTextureReadback")));
				}
			}
		}

		bool IsReady() const
		{
			// Note that, we would probably only need a fence on the last request.
			for (const TUniquePtr<FTmvMediaGpuTextureReadback>& Readback : Readbacks)
			{
				if (!Readback->IsReady())
				{
					return false;
				}
			}
			return true;
		}

		virtual ~FFrameReadbackRequest() = default;

		/** "Readback" helpers (manage the staging buffer) for each mips. */
		TArray<TUniquePtr<FTmvMediaGpuTextureReadback>> Readbacks;

		/** Keep track of the original pixel format. */
		EPixelFormat Format = EPixelFormat::PF_Unknown;

		/** MipInfo populated from the readback texture */ 
		FTmvMediaFrameMipInfo MipInfo;

		/** Frame time for this readback request */
		FTmvMediaFrameTimeInfo TimeInfo;
	};

	void SubmitReadbackRequest(
		FRDGBuilder& InBuilder, 
		FRDGTexture* InReadbackRdg,
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		const TSharedPtr<FFrameReadbackRequest>& InReadbackRequest)
	{
		const int32 NumMips = InReadbackRdg->Desc.NumMips;

		// Init/recycle setup
		InReadbackRequest->Init(InReadbackRdg, InTimeInfo);

		for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
		{
			TmvMedia::AddEnqueueCopyPass(InBuilder, InReadbackRequest->Readbacks[MipIndex].Get(), InReadbackRdg, MipIndex);
		}
	}

	TUniquePtr<FTmvMediaFrameMips> ProcessReadbackRequest(const TSharedPtr<FFrameReadbackRequest>& InReadbackRequest)
	{
		TUniquePtr<FTmvMediaFrameMips> FrameMips = MakeUnique<FTmvMediaFrameMips>();
		FrameMips->TimeInfo = InReadbackRequest->TimeInfo;

		// Second pass to read the readbacks.
		int32 MipWidth = InReadbackRequest->MipInfo.Width;
		int32 MipHeight = InReadbackRequest->MipInfo.Height;

		TRACE_CPUPROFILER_EVENT_SCOPE(ReadGpuTextureMipsV2::CopyMips);
		for (int32 MipIndex = 0; MipIndex < InReadbackRequest->Readbacks.Num(); ++MipIndex)
		{
			FTmvMediaGpuTextureReadback* Readback = InReadbackRequest->Readbacks[MipIndex].Get();
			if (!Readback)
			{
				return nullptr;
			}

			int32 RowPitchInPixels;
			int32 BufferHeight;
			const uint8* LockedData = (const uint8*)Readback->Lock(RowPitchInPixels, &BufferHeight);

			if (!LockedData)
			{
				UE_LOGF(LogTmvMedia, Error, "Readback failed and returned null locked data");
				return nullptr;
			}

			if (RowPitchInPixels == 0 || BufferHeight == 0)
			{
				UE_LOGF(LogTmvMedia, Error, "Readback failed and returned data with zero pitch/buffer height");
				return nullptr;
			}

			// todo: here buffers could (and should) be pooled and recycled.
			TSharedPtr<FTmvMediaFrameMipMemoryBuffer> MipBuffer = MakeShared<FTmvMediaFrameMipMemoryBuffer>();
			FTmvMediaFrameMipInfo MipBufferInfo = InReadbackRequest->MipInfo;

			// Adjust sizes for the mip level
			MipBufferInfo.MipLevel = MipIndex;
			MipBufferInfo.Width = MipBufferInfo.TileWidth = MipWidth;
			MipBufferInfo.Height = MipBufferInfo.TileHeight = MipHeight;
			MipBufferInfo.Planes[0].Width = MipWidth;
			MipBufferInfo.Planes[0].Stride = GPixelFormats[InReadbackRequest->Format].Get2DImageSizeInBytes(MipWidth, 1);
			MipBufferInfo.Planes[0].Height = MipBufferInfo.Planes[0].NumLines = MipHeight;

			if (MipBuffer->RequestAllocation(MipBufferInfo))
			{
				const SIZE_T ImageSizeInBytes = GPixelFormats[InReadbackRequest->Format].Get2DImageSizeInBytes(MipWidth, MipHeight);

				if (RowPitchInPixels == MipWidth && BufferHeight >= MipHeight)
				{
					FMemory::Memcpy(MipBuffer->GetPlaneBufferForComponent(0), LockedData, ImageSizeInBytes);
				}
				else
				{
					const SIZE_T SrcPitch = GPixelFormats[InReadbackRequest->Format].Get2DImageSizeInBytes(RowPitchInPixels, 1);
					const SIZE_T DstPitch = MipBufferInfo.Planes[0].Stride;
					const int32 HeightToCopy = FMath::Min(BufferHeight, MipHeight);
					const int32 RowSizeToCopy = FMath::Min(SrcPitch, DstPitch);
					for (int32 RowIndex = 0; RowIndex < HeightToCopy; ++RowIndex)
					{
						const uint8* SrcRow = LockedData + RowIndex * SrcPitch;
						uint8* DstRow = static_cast<uint8*>(MipBuffer->GetPlaneBufferForComponent(0)) + RowIndex * DstPitch;
						FMemory::Memcpy(DstRow, SrcRow, RowSizeToCopy);
					}
				}
				FrameMips->MipBuffers.Add(MipBuffer);
			}
			else
			{
				UE_LOGF(LogTmvMedia, Error, "Failed to allocate mip buffer %d (%dx%d)", MipIndex, MipWidth, MipHeight);
			}

			Readback->Unlock();

			MipWidth = FMath::Max(MipWidth/2, 1);
			MipHeight = FMath::Max(MipHeight/2, 1);
		}

		return FrameMips;
	}
}

FTmvMediaPlayerFrameProducerResource::FTmvMediaPlayerFrameProducerResource(UTmvMediaTranscodeJob* InParentJob, UMediaTexture* InMediaTexture)
	: ParentJob_RenderThread(InParentJob)
	, MediaTexture_RenderThread(InMediaTexture)
	, MediaFrameRate_RenderThread(0, 0) // invalid
{
	LastRenderedSampleTime = FirstRenderedSampleTime = FTimespan::MinValue();	// Indicates nothing is rendered yet.
}

FTmvMediaPlayerFrameProducerResource::~FTmvMediaPlayerFrameProducerResource() = default;

void FTmvMediaPlayerFrameProducerResource::InitRHI(FRHICommandListImmediate& InRHICmdList)
{
	FMediaDelegates::OnSampleRendered_RenderThread.AddSP(this, &FTmvMediaPlayerFrameProducerResource::OnMediaTextureRendered_RenderThread);
}

void FTmvMediaPlayerFrameProducerResource::ReleaseRHI()
{
	FMediaDelegates::OnSampleRendered_RenderThread.RemoveAll(this);
	RenderTargetRhi.SafeRelease();
}

void FTmvMediaPlayerFrameProducerResource::ReleaseReadbackRequest(TSharedPtr<UE::TmvMedia::FFrameReadbackRequest>&& InReadbackRequest)
{
	FScopeLock lock(&AvailableRequestsLock);

	// Recycle the request for next frame.
	if (AvailableRequests.Num() < 32) // Runaway Protection
	{
		AvailableRequests.Add(MoveTemp(InReadbackRequest));
	}
	else
	{
		UE_LOGF(LogTmvMedia, Warning, "More than 32 requests where created, discarding excess. (Request throttling has failed)");
	}
}

TSharedPtr<UE::TmvMedia::FFrameReadbackRequest> FTmvMediaPlayerFrameProducerResource::AcquireReadbackRequest()
{
	FScopeLock lock(&AvailableRequestsLock);
	return AvailableRequests.IsEmpty() ? MakeShared<UE::TmvMedia::FFrameReadbackRequest>() : AvailableRequests.Pop(EAllowShrinking::No);
}

void FTmvMediaPlayerFrameProducerResource::ProcessPendingReadbackRequests(FRHICommandListImmediate& InRHICmdList, const FTmvMediaTranscodeJobTime& InTime)
{
	using namespace UE::TmvMedia;
	for (TSharedPtr<FFrameReadbackRequest>& PendingRequest : PendingRequests)
	{
		if (PendingRequest->IsReady())
		{
			ProcessAndDispatch(MoveTemp(PendingRequest));
		}
	}

	PendingRequests.RemoveAll([](const TSharedPtr<FFrameReadbackRequest>& InRequest) { return !InRequest.IsValid(); });
}

const FTextureResource* FTmvMediaPlayerFrameProducerResource::GetMediaTextureResource() const
{
	if (UMediaTexture* MediaTexture = MediaTexture_RenderThread.Get())
	{
		return MediaTexture->GetResource();
	}
	return nullptr;
}


void FTmvMediaPlayerFrameProducerResource::CreateRenderTarget_RenderThread(FRHICommandListImmediate& InRHICmdList, const FRHITextureDesc& InRenderTargetDesc)
{
	const static FLazyName ClassName(TEXT("UTmvMediaPlayerFrameProducer"));

	const FRHITextureCreateDesc Desc =
	FRHITextureCreateDesc::Create2D(TEXT("MediaPlayerFrameProducerOutput"),
		InRenderTargetDesc.GetSize().X,
		InRenderTargetDesc.GetSize().Y,
		InRenderTargetDesc.Format)
	.SetNumMips(InRenderTargetDesc.NumMips)
	.SetClearValue(InRenderTargetDesc.ClearValue)
	.SetClassName(ClassName)
	.SetFlags(InRenderTargetDesc.Flags);

	RenderTargetRhi = InRHICmdList.CreateTexture(Desc);
}

void FTmvMediaPlayerFrameProducerResource::OnMediaTextureRendered_RenderThread(const FTextureResource* InMediaTextureResource, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> InMediaTextureSample)
{
	if (InMediaTextureResource == nullptr || !InMediaTextureSample.IsValid() || InMediaTextureResource != GetMediaTextureResource())
	{
		return;
	}

	FRHITexture* MediaTextureRhi = InMediaTextureResource->GetTexture2DRHI();
	if (!MediaTextureRhi)
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Frame Producer: Media Texture Resource has invalid texture rhi.");
		return;
	}

	UE_LOGF(LogTmvMedia, Verbose,
		"Transcode Frame Producer: Render callback Time=%ls Duration=%ls (PrevLast=%ls).",
		*InMediaTextureSample->GetTime().Time.ToString(),
		*InMediaTextureSample->GetDuration().ToString(),
		*LastRenderedSampleTime.load().ToString());

	// Avoid re-encoding already rendered samples.
	// Because of this, we only support going forward. Could use a Set instead.
	if (InMediaTextureSample->GetTime().Time < LastRenderedSampleTime)
	{
		UE_LOGF(LogTmvMedia, Error,
			"Transcode Frame Producer: Sample Time %ls earlier than last rendered time %ls.",
			*InMediaTextureSample->GetTime().Time.ToString(),
			*LastRenderedSampleTime.load().ToString());
		return;
	}

	// Keep track of the first frame to be able to calculate the frame index.
	if (FirstRenderedSampleTime == FTimespan::MinValue())
	{
		FirstRenderedSampleTime = InMediaTextureSample->GetTime().Time;
	}
	
	FTmvMediaFrameTimeInfo TimeInfo;
	TimeInfo.FrameIndex = INDEX_NONE;	// That can be calculated from frame time and duration for fixed rate videos.
	LastRenderedSampleTime = TimeInfo.FrameTime = InMediaTextureSample->GetTime().Time;
	LastRenderedSampleDuration = TimeInfo.FrameDuration = InMediaTextureSample->GetDuration();

	const FFrameRate FrameRate = UE::TmvMedia::TimeUtils::GetOrComputeFrameRate(MediaFrameRate_RenderThread, TimeInfo.FrameDuration);
	
	if (FrameRate.IsValid())
	{
		const FTimespan RelativeFrameTime = TimeInfo.FrameTime - FirstRenderedSampleTime;
		TimeInfo.FrameIndex = UE::TmvMedia::TimeUtils::RoundToFrameIndex(FrameRate, RelativeFrameTime.GetTotalSeconds());
		TimeInfo.FrameIndexNoOffset = TimeInfo.FrameIndex;	// This should already be zero based.
	}

	FRHICommandListImmediate& RhiCmdList = FRHICommandListImmediate::Get();

	if (!RenderTargetRhi)
	{
		// Fetch the desired render target format from the encoder requirements.
		if (UTmvMediaTranscodeJob* ParentJob = ParentJob_RenderThread.Get())
		{
			if (UTmvMediaFrameEncoder* FrameEncoder = ParentJob->GetStage<UTmvMediaFrameEncoder>())
			{
				FTmvMediaFrameMipInfo MediaTextureMipInfo;
				UE::TmvMedia::PopulateMipInfoFromTexture(MediaTextureRhi, 0, MediaTextureMipInfo);

				TargetMipInfo.Reset();

				if (!FrameEncoder->RequestMipInfos(ParentJob, TimeInfo, MediaTextureMipInfo, TargetMipInfo))
				{
					return;
				}
			}
		}

		FRHITextureDesc RenderTargetDesc = MediaTextureRhi->GetDesc();
		
		UE_LOGF(LogTmvMedia, Verbose, "Transcode Frame Producer: Media Texture Format: %d x %d, %d mips, %ls.",
			RenderTargetDesc.Extent.X, RenderTargetDesc.Extent.Y, RenderTargetDesc.NumMips, GPixelFormats[RenderTargetDesc.Format].Name);
		
		if (TargetMipInfo.IsValidIndex(0))
		{
			// Note: The destination render target number of mips matches the media texture, not TargetMipInfo.

			// Make it packed, all components in the same plane, and align with supported formats.
			TargetMipInfo[0].Planes[0].NumComponents = (TargetMipInfo[0].NumComponents == 1) ? 1 : 4;	// simplify to only support 1 or 4 components.
			RenderTargetDesc.Format = UE::TmvMedia::FrameUtils::GetPlanePixelFormat(TargetMipInfo[0].Planes[0], /*bNormalized*/ true);
			RenderTargetDesc.Flags &= ~ETextureCreateFlags::SRGB;	// Remove the SRGB because we will be encoding manually to the desired TF.
			RenderTargetDesc.Flags |= ETextureCreateFlags::RenderTargetable;	// Make sure the texture has a render target view.

			UE_LOGF(LogTmvMedia, Verbose, "Transcode Frame Producer: Conversion Format: %d x %d, %d mips, %ls.",
				RenderTargetDesc.Extent.X, RenderTargetDesc.Extent.Y, RenderTargetDesc.NumMips, GPixelFormats[RenderTargetDesc.Format].Name);
		}

		if (RenderTargetDesc.Format != PF_Unknown)
		{
			CreateRenderTarget_RenderThread(RhiCmdList, RenderTargetDesc);
		}
	}

	bool bGpuConverted = false;
	FRDGBuilder GraphBuilder(RhiCmdList);
	FRDGTextureRef MediaTextureRdg = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(MediaTextureRhi, TEXT("MediaTexture")));

	// Prepare for readback.
	FRDGTextureRef ReadbackRdg = MediaTextureRdg;

	// Convert if needed.
	if (RenderTargetRhi)
	{
		RHI_BREADCRUMB_EVENT_STAT_F(RhiCmdList, TmvMediaFrameProducer_ConvertFrame, "TmvMediaFrameProducer.ConvertFrame", "TmvMediaFrameProducer.ConvertFrame %d", TimeInfo.FrameIndex);
		FRDGTextureRef RenderTargetRdg = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(RenderTargetRhi, TEXT("RenderTarget")));
		if (TargetMipInfo.IsValidIndex(0))
		{
			UE::TmvMedia::AddConvertTexturePass(GraphBuilder, MediaTextureRdg, RenderTargetRdg, TargetMipInfo[0]);
			bGpuConverted = true;
		}
		else
		{
			UE::TmvMedia::AddCopyTexturePass(GraphBuilder, MediaTextureRdg, RenderTargetRdg);
		}
		ReadbackRdg = RenderTargetRdg;
	}

	if (PendingRequests.Num() < 32)	// Protect against too many pushes.
	{
		RHI_BREADCRUMB_EVENT_STAT_F(RhiCmdList, TmvMediaFrameProducer_ReadbackFrame, "TmvMediaFrameProducer.ReadbackFrame", "TmvMediaFrameProducer.ReadbackFrame %d", TimeInfo.FrameIndex);
		using namespace UE::TmvMedia;
		if (TSharedPtr<FFrameReadbackRequest> ReadbackRequest = AcquireReadbackRequest())
		{
			SubmitReadbackRequest(GraphBuilder, ReadbackRdg, TimeInfo, ReadbackRequest);

			// If the texture was converted by the GPU, it should have converted the color space, color model and encoded.
			if (bGpuConverted && TargetMipInfo.IsValidIndex(0))
			{
				ReadbackRequest->MipInfo.ColorInfo = TargetMipInfo[0].ColorInfo;
				ReadbackRequest->MipInfo.ColorModel = TargetMipInfo[0].ColorModel;
			}
			PendingRequests.Add(ReadbackRequest);

			// Increment the number of "submitted" frames to the async processing task queue so we can throttle the "pushes" accordingly.
			++SubmittedCount;
		}
		else
		{
			UE_LOGF(LogTmvMedia, Error,
				"Transcode Frame Producer: Failed to make a readback requests for Sample Time %ls.", *InMediaTextureSample->GetTime().Time.ToString());
		}
	}
	else
	{
		UE_LOGF(LogTmvMedia, Error,
			"Transcode Frame Producer: Submit pipeline is overflowing. Skipping sample Time %ls.", *InMediaTextureSample->GetTime().Time.ToString());
	}

	GraphBuilder.Execute();
}

void FTmvMediaPlayerFrameProducerResource::ProcessAndDispatch(TSharedPtr<UE::TmvMedia::FFrameReadbackRequest>&& InReadbackRequest)
{
#if WITH_EDITOR
	EAsyncExecution ExecutionTarget = EAsyncExecution::LargeThreadPool;
#else
	EAsyncExecution ExecutionTarget = EAsyncExecution::ThreadPool;
#endif

	if (FTmvMediaGpuTextureReadback::SupportsAnyThreadReadback())
	{
		TWeakPtr<FTmvMediaPlayerFrameProducerResource> LocalWeakThis = SharedThis(this);

		// Everything from that point will be done in a worker thread.
		Async(ExecutionTarget, [LocalWeakThis, ReadbackRequest = MoveTemp(InReadbackRequest)]()
		{
			if (TSharedPtr<FTmvMediaPlayerFrameProducerResource> LocalThis = LocalWeakThis.Pin())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UTmvMediaPlayerFrameProducer::ProcessFrame);
				
				TUniquePtr<FTmvMediaFrameMips> FrameMips = UE::TmvMedia::ProcessReadbackRequest(ReadbackRequest);

				// Return the request to the pool now that we are done with it.
				TSharedPtr<UE::TmvMedia::FFrameReadbackRequest> ReadbackRequestMovable = ReadbackRequest;
				LocalThis->ReleaseReadbackRequest(MoveTemp(ReadbackRequestMovable));

				if (FrameMips)
				{
					UTmvMediaTranscodeJob* ParentJob = LocalThis->ParentJob_RenderThread.Get();
					if (UTmvMediaFrameConverter* FrameConverter = ParentJob ? ParentJob->GetStage<UTmvMediaFrameConverter>() : nullptr)
					{
						FrameConverter->ReceiveMips(ParentJob, MoveTemp(FrameMips));
					}
				}
				--LocalThis->SubmittedCount;
			}
		});
	}
	else
	{
		/**
		 * Todo (eventually...):
		 * If we want to move the readback lock-copy-unlock out of the render thread, and lock/unlock 
		 * is only supported on render thread, we will need to split that into 3 stages, which will have 2 
		 * (render thread) pending states:
		 * - PendingLock: First waiting on the gpu to complete and the readback to be "ready" to be locked.
		 *   When it is locked, it goes in an async task to be processed (copy). Once copy is done, it 
		 *   is submitted back to the pending unlock list.
		 * - PendingUnlock: After being processed (while still locked), it has to go in pending unlock so that
		 *   the render thread unlocks it on the next render tick and finally returns it to the available list.
		 */

		// Perform lock-copy-unlock on the render thread for now. Simplest way.
		TUniquePtr<FTmvMediaFrameMips> FrameMips = UE::TmvMedia::ProcessReadbackRequest(InReadbackRequest);

		// Return the request to the pool now that we are done with it.
		ReleaseReadbackRequest(MoveTemp(InReadbackRequest));

		if (!FrameMips)
		{
			return;
		}

		TWeakPtr<FTmvMediaPlayerFrameProducerResource> LocalWeakThis = SharedThis(this);

		// When the converted render target is ready, we fire the rest of the cpu work on another thread.
		Async(ExecutionTarget, [LocalWeakThis, FrameMipsMovable = MoveTemp(FrameMips)]() mutable
		{
			if (TSharedPtr<FTmvMediaPlayerFrameProducerResource> LocalThis = LocalWeakThis.Pin())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UTmvMediaPlayerFrameProducer::ProcessFrame);
				UTmvMediaTranscodeJob* ParentJob = LocalThis->ParentJob_RenderThread.Get();
				if (UTmvMediaFrameConverter* FrameConverter = ParentJob ? ParentJob->GetStage<UTmvMediaFrameConverter>() : nullptr)
				{
					FrameConverter->ReceiveMips(ParentJob, MoveTemp(FrameMipsMovable));
				}
				--LocalThis->SubmittedCount;
			}
		});
	}
}