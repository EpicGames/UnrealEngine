// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timespan.h"
#include "RHIFwd.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "TmvMediaFrameInfo.h"
#include "Misc/FrameRate.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FTextureResource;
class IMediaTextureSample;
class UMediaTexture;
class UTmvMediaTranscodeJob;
struct FRHITextureDesc;
struct FTmvMediaFrameTimeInfo;
struct FTmvMediaTranscodeJobTime;

namespace UE::TmvMedia
{
	struct FFrameReadbackRequest;
}

/**
 * Render thread object for the media player frame producer.
 * Handles the render thread callbacks and holds the rhi resources.
 */
class FTmvMediaPlayerFrameProducerResource : public TSharedFromThis<FTmvMediaPlayerFrameProducerResource>
{
public:
	FTmvMediaPlayerFrameProducerResource(UTmvMediaTranscodeJob* InParentJob, UMediaTexture* InMediaTexture);

	~FTmvMediaPlayerFrameProducerResource();

	/**
	 * Initializes the dynamic RHI resource on the render thread.
	 */
	void InitRHI(FRHICommandListImmediate& InRHICmdList);

	/**
	 * Releases the dynamic RHI resource on the render thread.
	 */
	void ReleaseRHI();

	/**
	 * Poll all pending readback requests and process the ones that are ready.
	 */
	void ProcessPendingReadbackRequests(FRHICommandListImmediate& InRHICmdList, const FTmvMediaTranscodeJobTime& InTime);

	/**
	 * Returns the last sample time that was rendered by the media texture.
	 */
	FTimespan GetLastRenderedSampleTime() const
	{
		return LastRenderedSampleTime;
	}

	/**
	 * Returns the last sample duration that was rendered by the media texture.
	 */
	FTimespan GetLastRenderedSampleDuration() const
	{
		return LastRenderedSampleDuration;
	}

	/**
	 * Returns the number of concurrent samples that are being processed (submitted to transcoding pipeline).
	 */
	int32 GetSubmittedCount() const
	{
		return SubmittedCount.load(std::memory_order_relaxed);
	}

	/**
	 * Set the media frame rate to use to compute the frame indices.  
	 */
	void SetMediaFrameRate_RenderThread(const FFrameRate& InFrameRate)
	{
		MediaFrameRate_RenderThread = InFrameRate;
	}

private:
	/** Returns the texture resource from the associated media texture. */
	const FTextureResource* GetMediaTextureResource() const;

	/** Create the intermediate conversion render target. */
	void CreateRenderTarget_RenderThread(FRHICommandListImmediate& InRHICmdList, const FRHITextureDesc& InRenderTargetDesc);

	/** Callback for the media texture rendering delegate. */
	void OnMediaTextureRendered_RenderThread(const FTextureResource* InMediaTextureResource, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> InMediaTextureSample);

	/** Process the given readback request and dispatch the mips to the next transcoding stage. */
	void ProcessAndDispatch(TSharedPtr<UE::TmvMedia::FFrameReadbackRequest>&& InReadbackRequest);

	/** Acquire a readback request from the pool if possible, or create a new one otherwise. */
	TSharedPtr<UE::TmvMedia::FFrameReadbackRequest> AcquireReadbackRequest();

	/** Returns the request to the pool for recycling. */
	void ReleaseReadbackRequest(TSharedPtr<UE::TmvMedia::FFrameReadbackRequest>&& InReadbackRequest);

private:
	/** Pointer to parent job used in render thread. */
	TWeakObjectPtr<UTmvMediaTranscodeJob> ParentJob_RenderThread;

	/** Pointer to the parent media texture used in render thread. */
	TWeakObjectPtr<UMediaTexture> MediaTexture_RenderThread;

	/** Intermediate render target to convert the media texture to desired format for conversion. */
	FTextureRHIRef RenderTargetRhi;
	
	/** Media Frame rate from the player. Used to compute frame index. */
	FFrameRate MediaFrameRate_RenderThread;

	/** Keep track of the first rendered sample time to compute the frame index. */
	FTimespan FirstRenderedSampleTime;

	/** This is updated on the render thread to keep track of the rendered "play head". */
	std::atomic<FTimespan> LastRenderedSampleTime;

	/** This is updated on the render thread as additional information for controller the player. */
	std::atomic<FTimespan> LastRenderedSampleDuration;

	/** Number of frames submitted to the converter/decoder (in flight after the producer). */
	std::atomic_int32_t SubmittedCount = 0;

	/** 
	 * Array of pending readback requests. Those are requests that have been submitted to the
	 * gpu and are being polled (on the render thread) until they become "ready", i.e. the gpu
	 * has done the work and it is ready to be processed, i.e. locked-copied-unlocked without 
	 * causing a cpu-gpu stall. 
	 */
	TArray<TSharedPtr<UE::TmvMedia::FFrameReadbackRequest>> PendingRequests;

	/** Critical Section for accessing the AvailableRequests. */
	FCriticalSection AvailableRequestsLock;

	/** Array of free available requests (for reuse). */
	TArray<TSharedPtr<UE::TmvMedia::FFrameReadbackRequest>> AvailableRequests;

	/** Information about the desired mip formats for the encoder. */
	TArray<FTmvMediaFrameMipInfo> TargetMipInfo;
};
