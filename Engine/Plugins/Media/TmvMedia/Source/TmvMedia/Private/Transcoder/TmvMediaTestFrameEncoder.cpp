// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaTestFrameEncoder.h"

#include "HAL/PlatformFileManager.h"
#include "ImageCore.h"
#include "ImageUtils.h"
#include "Misc/Paths.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Utils/TmvMediaFrameUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaTestFrameEncoder)

bool UTmvMediaTestFrameEncoder::Start(UTmvMediaTranscodeJob* InParentJob)
{
	const FString OutPath = InParentJob->Settings.GetAbsoluteOutputPath();
	if (OutPath.IsEmpty())
	{
		UE_LOGF(LogTmvMedia, Error, "TestFrameEncoder: OutputPath is empty; cannot write test frames.");
		return false;
	}
	
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.CreateDirectoryTree(*OutPath))
	{
		UE_LOGF(LogTmvMedia, Error, "TestFrameEncoder: Failed to create output directory: %ls", *OutPath);
		return false;
	}
	
	return Super::Start(InParentJob);
}

void UTmvMediaTestFrameEncoder::ReceiveMips(UTmvMediaTranscodeJob* InParentJob, TUniquePtr<FTmvMediaFrameMips>&& InMips)
{
	FString OutPath = InParentJob->Settings.GetAbsoluteOutputPath();
	int32 MipIndex = 0;
	for (const FTmvMediaFrameMipBufferHandle& MipBuffer : InMips->MipBuffers)
	{
		if (!MipBuffer)
		{
			UE_LOGF(LogTmvMedia, Warning,
				"TestFrameEncoder: Null mip buffer for frame %d (mip %d). Skipping.",
				InMips->TimeInfo.FrameIndex, MipIndex);
			++MipIndex;
			continue;
		}
		
		using namespace UE::TmvMedia::FrameUtils;
		FImageInfo ImageInfo;
		FString FailureReason;
		if (PopulateImageInfo(MipBuffer->GetMipInfoRef(), ImageInfo, &FailureReason))
		{
			FImageView ImageView(ImageInfo, MipBuffer->GetPlaneBufferForComponent(0));
			FString FileName = FString::Printf(TEXT("image%05d_mip%02d"), InMips->TimeInfo.FrameIndex, MipIndex);
			FString Name = FPaths::Combine(OutPath, FileName);
			if (!FImageUtils::SaveImageAutoFormat(*Name, ImageView))
			{
				UE_LOGF(LogTmvMedia, Error, "TestFrameEncoder: Failed to save image %ls (%d x %d) format %ls.", 
					*Name, ImageView.SizeX, ImageView.SizeY, ERawImageFormat::GetName(ImageView.Format));
			}
		}
		else // todo: for yuv planar formats, we can use Y4M format.
		{
			UE_LOGF(LogTmvMedia, Error, "TestFrameEncoder: Failed to save mip buffer for frame %d: %ls",
				InMips->TimeInfo.FrameIndex, *FailureReason);
		}
		++MipIndex;
	}
}