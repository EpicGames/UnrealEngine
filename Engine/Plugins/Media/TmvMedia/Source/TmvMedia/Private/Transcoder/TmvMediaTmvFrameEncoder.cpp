// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaTmvFrameEncoder.h"

#include "Encoder/ITmvMediaEncoder.h"
#include "Encoder/ITmvMediaEncoderFactory.h"
#include "Encoder/TmvMediaEncoderOptions.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "ITmvMediaModule.h"
#include "ImageCore.h"
#include "Misc/Paths.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaContainerTranscodeMuxer.h"
#include "Transcoder/TmvMediaFrameProducer.h"
#include "Transcoder/TmvMediaTranscodeMuxer.h"
#include "Utils/TmvMediaMessageContext.h"
#include "Utils/TTmvMediaSharedObjectPool.h"
#include "Utils/TmvMediaFrameUtils.h"
#include "Utils/TmvMediaPathUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaTmvFrameEncoder)

#define LOCTEXT_NAMESPACE "TmvMediaTmvFrameEncoder"

/** Helper to trouble shoot frame conversion. */
static TAutoConsoleVariable<bool> CVarTmvEncoderEnableDumpRawFrame(
	TEXT("TmvEncoder.EnableDumpRawFrame"),
	false,
	TEXT("Helper to trouble shoot frame transcoding. The TMV encoder will save raw frames, prior to encoding, to temporary files for inspection.\n"),
	ECVF_Default);

// Helper functions to debug frame conversion.
namespace UE::TmvMedia::FrameUtils
{
	// Debug helper to dump the buffer to a suitable file for inspection.
	bool WriteFrameToFile(const UTmvMediaTranscodeJob* InParentJob, const FString& InStreamName, const FTmvMediaFrameMips& InMips)
	{
		if (!InParentJob || !InMips.MipBuffers.IsValidIndex(0) || !InMips.MipBuffers[0])
		{
			return false;
		}

		// save only mip 0 for now.
		constexpr int32 MipIndex = 0;
		const FString OutPath = InParentJob->Settings.GetAbsoluteOutputPath();
		if (OutPath.IsEmpty())
		{
			UE_LOGF(LogTmvMedia, Error, "WriteFrameToFile Failed: Empty output path.");
			return false;
		}
		FString TmpFileName = FString::Printf(TEXT("%s%05d_mip%02d_tmp"), *InStreamName, InMips.TimeInfo.FrameIndex, MipIndex);
		return WriteMipBufferToFile(FPaths::Combine(OutPath, FPaths::MakeValidFileName(TmpFileName)), InMips.MipBuffers[MipIndex]);
	}
}

/**
 * Encoder pool allows to acquire and recycle the encoder contexts between the worker threads.
 */
class FTmvMediaTmvEncoderPool : public TTmvMediaSharedObjectPool<ITmvMediaEncoder>
{
public:
	explicit FTmvMediaTmvEncoderPool(const TSharedPtr<ITmvMediaEncoderFactory>& InEncoderFactory)
		: EncoderFactory(InEncoderFactory)
	{
	}

	FHandle AcquireEncoder(const FString& InCodecFormat, const TInstancedStruct<FTmvMediaEncoderOptions>& InOptions)
	{
		return Acquire([this, &InCodecFormat, &InOptions]()
			{
				return EncoderFactory->CreateEncoder(InCodecFormat, InOptions);
			});
	}

	/** Encoder factory to use to create more encoders if needed. */
	TSharedPtr<ITmvMediaEncoderFactory> EncoderFactory;
};

/** Implementation of a memory writer. */
class FTmvMediaMemoryEncoderAccessUnit : public ITmvMediaEncoderAccessUnit
{
public:
	mutable FMemoryWriter64 MemoryWriter;
	int32 FrameId;
	FString Filename;

	FTmvMediaMemoryEncoderAccessUnit(TArray64<uint8>& InBytes, int32 InFrameId, const FString& InFilename)
		: MemoryWriter(InBytes, true)
		, FrameId(InFrameId)
		, Filename(InFilename)
	{
	}

	//~ Begin ITmvMediaEncoderAccessUnit
	virtual int64 Tell() const override
	{
		return MemoryWriter.Tell();
	}

	virtual int64 GetTotalSize() const override
	{
		return MemoryWriter.TotalSize();
	}

	virtual bool Seek(int64 InOffset) override
	{
		MemoryWriter.Seek(InOffset);
		return !MemoryWriter.IsError();
	}

	virtual int64 Write(const void *InBuffer, int64 InSize) override
	{
		MemoryWriter.Serialize((void *)InBuffer, InSize);
		return MemoryWriter.IsError() ? 0 : InSize;
	}

	virtual int32 GetFrameId() const override
	{
		return FrameId;
	}

	virtual const FString& GetFilename() const override
	{
		return Filename;
	}

	virtual FArchive* GetUnderlyingArchive() const override
	{
		return &MemoryWriter;
	}
	//~ End ITmvMediaEncoderAccessUnit
};

bool UTmvMediaTmvFrameEncoder::Start(UTmvMediaTranscodeJob* InParentJob)
{
	StreamId = INDEX_NONE;

	if (!InParentJob)
	{
		UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: Start called with invalid Parent Job.");
		return false;
	}

	if (!EncoderOptions.IsValid())
	{
		UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: Encoder options haven't been set.");
		return false;
	}

	TSharedPtr<ITmvMediaEncoderFactory> EncoderFactory;
	if (ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get())
	{
		EncoderFactory = TmvMediaModule->FindEncoderFactory(EncoderOptions.Get().GetEncoderName());
	}

	if (!EncoderFactory)
	{
		UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: Failed to find encoder factory \"%ls\".", *EncoderOptions.Get().GetEncoderName().ToString());
		return false;
	}

	EncoderPool = MakeShared<FTmvMediaTmvEncoderPool>(EncoderFactory);

	StreamName = !InParentJob->Settings.OutputBaseName.IsEmpty() ? InParentJob->Settings.OutputBaseName : TEXT("Frame");
	
	// If the encoder supports a memory "writer" access unit interface, then we can use the muxer stage.
	// Otherwise, it is assumed the encoder has some fixed muxer implementation that can't be externalized.
	if (EncoderFactory->SupportsMemoryAccessUnit())
	{
		if (UTmvMediaTranscodeMuxer* Muxer = InParentJob->GetStage<UTmvMediaTranscodeMuxer>())
		{
			StreamId = Muxer->OpenStream(InParentJob, StreamName, EncoderOptions.Get().GetFileSequenceExtension());

			// Check if this is a container muxer for deferred track config.
			ContainerMuxer = Cast<UTmvMediaContainerTranscodeMuxer>(Muxer);
			bContainerTrackConfigSet = false;
		}

		if (StreamId == INDEX_NONE)
		{
			UE_LOGF(LogTmvMedia, Error, "Failed to open stream %ls with muxer.", *StreamName);
			return false;
		}
	}
	else
	{
		const FString OutPath = InParentJob->Settings.GetAbsoluteOutputPath();
		if (!FPaths::DirectoryExists(OutPath))
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (!PlatformFile.CreateDirectoryTree(*OutPath))
			{
				UE_LOGF(LogTmvMedia, Error, "Failed to create output directory %ls", *OutPath);
				return false;
			}
		}
	}

	return Super::Start(InParentJob);
}

void UTmvMediaTmvFrameEncoder::RequestStop(UTmvMediaTranscodeJob* InParentJob)
{
	EncoderPool.Reset();
	Super::RequestStop(InParentJob);
}

void UTmvMediaTmvFrameEncoder::SetEncoderOptions(const TInstancedStruct<FTmvMediaEncoderOptions>& InOptions)
{
	if (GetStageStatus() == ETmvMediaTranscodeStageStatus::Started)
	{
		UE_LOGF(LogTmvMedia, Error, "Encoder options can't be set once the stage is started.");
		return;
	}
	EncoderOptions = InOptions;
}

const FTmvMediaEncoderOptions* UTmvMediaTmvFrameEncoder::GetEncoderOptions() const
{
	return EncoderOptions.GetPtr();
}

bool UTmvMediaTmvFrameEncoder::RequestMipInfos(UTmvMediaTranscodeJob* InParentJob, const FTmvMediaFrameTimeInfo& InSourceTimeInfo, const FTmvMediaFrameMipInfo& InSourceMipInfo, TArray<FTmvMediaFrameMipInfo>& OutFrameMipInfo)
{
	if (!InParentJob)
	{
		UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: RequestMipInfos called with invalid Parent Job.");
		return false;
	}

	FTmvMediaEncoderMipInfo EncoderMipInfo;
	EncoderMipInfo.bEnableMips = InParentJob->Settings.bEnableMipMapping;

	if (EncoderOptions.IsValid() && EncoderOptions.Get().bEnableColorManagement)
	{
		EncoderMipInfo.ColorInfo.Encoding = static_cast<UE::Color::EEncoding>(EncoderOptions.Get().DestinationEncoding);
		EncoderMipInfo.ColorInfo.ReferenceWhiteOverride = static_cast<UE::Color::EReferenceWhite>(EncoderOptions.Get().ReferenceWhite);

		if (EncoderOptions.Get().DestinationColorSpace != ETextureColorSpace::TCS_Custom)
		{
			EncoderMipInfo.ColorInfo.ColorSpace = static_cast<UE::Color::EColorSpace>(EncoderOptions.Get().DestinationColorSpace);
		}

		EncoderMipInfo.ColorInfo.YuvMatrix = static_cast<ETmvMediaFrameColorMatrix>(EncoderOptions.Get().YuvMatrix);
		EncoderMipInfo.ColorInfo.YuvMatrixRange = static_cast<ETmvMediaFrameColorMatrixRange>(EncoderOptions.Get().YuvMatrixRange);
	}
	else
	{
		EncoderMipInfo.ColorInfo = FTmvMediaFrameColorInfo::MakeNeutral();
	}

	EncoderMipInfo.Width = InSourceMipInfo.Width;
	EncoderMipInfo.Height = InSourceMipInfo.Height;

	TSharedPtr<FTmvMediaTmvEncoderPool> EncoderPoolRef = EncoderPool;

	if (!EncoderPoolRef.IsValid())
	{
		UE_LOGF(LogTmvMedia, Log, "Frame Encoder Stage: Encoder Pool not available.");
		return false;
	}

	FTmvMediaTmvEncoderPool::FHandle EncoderHandle = EncoderPoolRef->AcquireEncoder(TEXT(""), EncoderOptions);
	if (!EncoderHandle.IsValid())
	{
		UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: Failed to acquire encoder \"%ls\".", *EncoderOptions.Get().GetEncoderName().ToString());
		return false;
	}

	FTmvMediaMessageContext MessageContext;
	const ETmvMediaEncoderResult Result = EncoderHandle->RequestMipInfos(InSourceTimeInfo, EncoderMipInfo, OutFrameMipInfo, &MessageContext);
	if (Result != ETmvMediaEncoderResult::Success)
	{
		const FText Message = MessageContext.ToText();
		UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage Error: %ls.", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return false;
	}

	// Set the container muxer track config on first call now that we know dimensions.
	if (ContainerMuxer && !bContainerTrackConfigSet && StreamId != INDEX_NONE)
	{
		FTmvMediaMuxerTrackConfig TrackConfig;
		TrackConfig.TrackType = ETmvMediaTrackType::Video;
		TrackConfig.SampleEntryFormat = EncoderOptions.Get().GetCodecFourCC();
		TrackConfig.DisplayWidth = static_cast<uint32>(InSourceMipInfo.Width);
		TrackConfig.DisplayHeight = static_cast<uint32>(InSourceMipInfo.Height);
		TrackConfig.bIsAllKeyframes = true;

		// Set timescale and constant duration from frame rate.
		FFrameRate FrameRate = InParentJob->Settings.GetInputFramerate();
		// Fetch the frame rate from the frame producer, which is going to be what the media file was.
		if (UTmvMediaFrameProducer* FrameProducer = InParentJob->GetStage<UTmvMediaFrameProducer>())
		{
			const FTmvMediaFrameProducerTrackInfo VideoTrackInfo = FrameProducer->GetVideoTrackInfo();
			FrameRate = VideoTrackInfo.FrameRate;
		}
		TrackConfig.Timescale = static_cast<uint32>(FrameRate.Numerator);
		TrackConfig.ConstantSampleDuration = static_cast<uint32>(FrameRate.Denominator);

		ContainerMuxer->SetStreamTrackConfig(StreamId, TrackConfig);
		bContainerTrackConfigSet = true;
	}

	return true;
}

void UTmvMediaTmvFrameEncoder::ReceiveMips(UTmvMediaTranscodeJob* InParentJob, TUniquePtr<FTmvMediaFrameMips>&& InMips)
{
	if (!InParentJob)
	{
		UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: ReceiveMips called with invalid Parent Job.");
		return;
	}

	if (!InMips)
	{
		UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: ReceiveMips called with invalid Mips.");
		return;
	}

	// Troubleshooting - Dump received frame buffer for inspection. 
	if (CVarTmvEncoderEnableDumpRawFrame.GetValueOnAnyThread())
	{
		if (!UE::TmvMedia::FrameUtils::WriteFrameToFile(InParentJob, StreamName, *InMips))
		{
			UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: Failed to Dump Raw Frame %d.", InMips->TimeInfo.FrameIndex);
		}
	}

	// Take a ref on the encoder pool to avoid loosing it while we are processing.
	TSharedPtr<FTmvMediaTmvEncoderPool> EncoderPoolRef = EncoderPool;

	if (!EncoderPoolRef.IsValid())
	{
		UE_LOGF(LogTmvMedia, Log, "Frame Encoder Stage: Encoder Pool not available. Skipping Frame %d", InMips->TimeInfo.FrameIndex);
		return;
	}

	const FTmvMediaTmvEncoderPool::FHandle EncoderHandle = EncoderPoolRef->AcquireEncoder(TEXT(""), EncoderOptions);
	if (!EncoderHandle.IsValid())
	{
		const FText Message = FText::Format(LOCTEXT("FailedAcquireEncoder", "Failed to acquire encoder \"{0}\""), FText::FromName(EncoderOptions.Get().GetEncoderName()));
		UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: %ls.", *Message.ToString());
		UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
		return;
	}

	// TODO: pool this?
	TSharedPtr<TArray64<uint8>> AccessUnitBuffer = MakeShared<TArray64<uint8>>();

	{
		FString OutputPath = InParentJob->Settings.GetAbsoluteOutputPath();

		if (OutputPath.IsEmpty())
		{
			const FText Message = LOCTEXT("EmptyOutputPath", "Empty Output Path");
			UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: %ls.", *Message.ToString());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			return;
		}

		const FString FileName = UE::TmvMedia::PathUtils::MakeFrameFilename(*StreamName, InMips->TimeInfo.FrameIndex, InParentJob->Settings.ZeroPadFrameNumbers, *EncoderOptions.Get().GetFileSequenceExtension());
		FString FullFilename = FPaths::Combine(OutputPath, FileName);

		FTmvMediaMemoryEncoderAccessUnit AccessUnit(*AccessUnitBuffer, InMips->TimeInfo.FrameIndex, FullFilename);

		TArray<FTmvMediaEncoderMipRequest> MipRequests;

		if (!InMips->MipBuffers.IsEmpty())
		{
			// Converted buffers
			MipRequests.Reserve(InMips->MipBuffers.Num());
			for (FTmvMediaFrameMipBufferHandle& MipBuffer : InMips->MipBuffers)
			{
				FTmvMediaEncoderMipRequest MipRequest;
				MipRequest.MipBuffer = MipBuffer;
				MipRequests.Add(MoveTemp(MipRequest));
			}
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(UTmvMediaTmvFrameEncoder::Encode);
		FTmvMediaMessageContext MessageContext;
		const ETmvMediaEncoderResult Result = EncoderHandle->Encode(InMips->TimeInfo, AccessUnit, MipRequests, &MessageContext);
		if (Result != ETmvMediaEncoderResult::Success)
		{
			const FText Message = FText::Format(
				LOCTEXT("FailedEncodeFrame", "Failed to encode frame {0}: {1}"),
				FText::AsNumber(InMips->TimeInfo.FrameIndex), MessageContext.ToText());
			UE_LOGF(LogTmvMedia, Error, "Frame Encoder Stage: %ls.", *Message.ToString());
			UTmvMediaTranscodeJob::SafeReportError(InParentJob, Message);
			return;
		}
	}
	
	if (UTmvMediaTranscodeMuxer* Muxer = InParentJob->GetStage<UTmvMediaTranscodeMuxer>())
	{
		if (StreamId != INDEX_NONE)
		{
			Muxer->ReceiveAccessUnit(InParentJob, StreamId, InMips->TimeInfo, MoveTemp(AccessUnitBuffer));
		}
	}
}

#undef LOCTEXT_NAMESPACE