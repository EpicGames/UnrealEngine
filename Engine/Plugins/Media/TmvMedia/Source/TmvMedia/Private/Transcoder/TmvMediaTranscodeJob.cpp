// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaTranscodeJob.h"

#include "MediaSource.h"
#include "Misc/Paths.h"
#include "TmvMediaLog.h"
#include "Transcoder/ITmvMediaTranscodeJobManager.h"
#include "Transcoder/TmvMediaFrameProducer.h"
#include "Utils/TmvMediaPathUtils.h"
#include "Utils/TmvMediaUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaTranscodeJob)

#define LOCTEXT_NAMESPACE "TmvMediaTranscodeJob"

FString FTmvMediaTranscodeJobSettings::GetInputPath() const
{
	if (InputSource == ETmvMediaTranscodeInputSource::File)
	{
		return InputPath.FilePath;
	}

	// Returns the input path.
	return InputMediaSource.ToString();
}

FString FTmvMediaTranscodeJobSettings::GetAbsoluteInputPath() const
{
	if (InputSource == ETmvMediaTranscodeInputSource::File)
	{
		return UE::TmvMedia::PathUtils::ConvertSanitizedPathToFull(InputPath.FilePath);
	}

	// Get the media file from the media source.
	if (UMediaSource* MediaSource = InputMediaSource.LoadSynchronous())
	{
		const FString MediaPath = UE::TmvMedia::PathUtils::GetMediaSourceMediaFullPath(MediaSource);

		if (MediaPath.IsEmpty())
		{
			UE_LOGF(LogTmvMedia, Error, "Media Source \"%ls\" is not a supported codec for transcoding.", *MediaSource->GetUrl());
		}

		return MediaPath;
	}

	UE_LOGF(LogTmvMedia, Error, "Unable to load asset \"%ls\".", *InputMediaSource.ToString());
	return FString();
}


bool FTmvMediaTranscodeJobSettings::IsInputPathSet() const
{
	if (InputSource == ETmvMediaTranscodeInputSource::File)
	{
		return InputPath.FilePath.IsEmpty() == false;
	}

	return InputMediaSource.IsNull() == false;
}

FFrameRate FTmvMediaTranscodeJobSettings::GetInputFramerate() const
{
	if (InputSource == ETmvMediaTranscodeInputSource::MediaSource)
	{
		// Get the frame rate from the media source if possible.
		if (UMediaSource* MediaSource = InputMediaSource.LoadSynchronous())
		{
			if (FStructProperty* const FrameRateProperty = FindFProperty<FStructProperty>(MediaSource->GetClass(), TEXT("FrameRateOverride")))
			{
				if (const void* SrcPtr = FrameRateProperty->ContainerPtrToValuePtr<FFrameRate>(MediaSource, 0))
				{
					FFrameRate LocalFrameRate = FrameRate;
					FrameRateProperty->CopyCompleteValue(&LocalFrameRate, SrcPtr);
					return LocalFrameRate;
				}
			}
		}
		else
		{
			UE_LOGF(LogTmvMedia, Error, "Unable to load asset \"%ls\".", *InputMediaSource.ToString());
		}
	}

	return FrameRate;
}

bool FTmvMediaTranscodeJobSettings::IsOutputPathSet() const
{
	return OutputPath.Path.IsEmpty() == false;
}

FString FTmvMediaTranscodeJobSettings::GetAbsoluteOutputPath() const
{
	return UE::TmvMedia::PathUtils::ConvertSanitizedPathToFull(OutputPath.Path);
}

UTmvMediaTranscodeJob::UTmvMediaTranscodeJob()
{
	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		ITmvMediaTranscodeJobManager::SafeRegisterTranscodeJob(this);
	}
}

void UTmvMediaTranscodeJob::SetId(const FGuid& InId)
{
	Id = InId;
	ITmvMediaTranscodeJobManager::SafeRegisterTranscodeJob(this);
}

bool UTmvMediaTranscodeJob::Start(double InCurrentTime)
{
	UE_LOGF(LogTmvMedia, Verbose, "Transcode Job for \"%ls\" Started.", *Settings.GetInputPath());
	
	JobStatus = ETmvMediaTranscodeJobStatus::Running;
	StopReason = ETmvMediaTranscodeJobStopReason::None;
	bIsError = false;
	JobStats = FTmvMediaTranscodingJobStats();
	JobStats.StartTime = InCurrentTime;

	UTmvMediaFrameProducer* FrameProducer = GetStage<UTmvMediaFrameProducer>();
	if (!FrameProducer)
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Job for \"%ls\" failed to start: missing the Frame Producer stage.", *Settings.GetInputPath());
		SafeReportError(this, LOCTEXT("ErrorMissingFrameProducer", "Missing Frame Producer Stage"));
		JobStatus = ETmvMediaTranscodeJobStatus::Stopped;
		OnStopped(InCurrentTime);
		return false;
	}

	// Start all consumer stages first.
	for (UTmvMediaTranscodeStage* Stage : Stages)
	{
		if (Stage != FrameProducer)
		{
			if (!Stage->Start(this))
			{
				bIsError = true;	// Must be set before calling OnStopped.
				JobStatus = ETmvMediaTranscodeJobStatus::Stopped;
				OnStopped(InCurrentTime);
				return false;
			}
		}
	}

	// Start the producer last.
	if (!FrameProducer->Start(this))
	{
		bIsError = true;	// Must be set before calling OnStopped.
		JobStatus = ETmvMediaTranscodeJobStatus::Stopped;
		OnStopped(InCurrentTime);
	}

	return IsRunning();
}

void UTmvMediaTranscodeJob::OnStopped(double InCurrentTime)
{
	JobStats.StopTime = InCurrentTime;

	SafeBroadcastJobFinished(this);
	
	if (Notification.IsValid())
	{
		bool bIsCompleted;
		{
			FScopeLock lock(&JobStatsLock);
			bIsCompleted = JobStats.ProcessedFrame == JobStats.TotalFramesToProcess;
		}
		Notification->Close(bIsCompleted && !bIsError);
	}
}

void UTmvMediaTranscodeJob::RequestStop(double InCurrentTime, ETmvMediaTranscodeJobStopReason InReason)
{
	UE_LOGF(LogTmvMedia, Verbose, "Transcode Job for \"%ls\" Stopped. Reason: %ls",
		*Settings.GetInputPath(), *UE::TmvMedia::Utils::StaticEnumToString(InReason)); 
	
	if (IsRunning())
	{
		JobStatus = ETmvMediaTranscodeJobStatus::Stopping;
		StopReason = InReason;
	}
	else
	{
		UE_LOGF(LogTmvMedia, Verbose, "Transcode Job for \"%ls\" was already stopping", *Settings.GetInputPath()); 
	}

	// Stop the stages anyway.
	for (UTmvMediaTranscodeStage* Stage : Stages)
	{
		Stage->RequestStop(this);	// Requests stop.
	}

	// Check if the stages are actually stopped.
	if (AreStagesStopped())
	{
		ETmvMediaTranscodeJobStatus PreviousJobStatus = JobStatus;
		JobStatus = ETmvMediaTranscodeJobStatus::Stopped;	// truly stopped.
		
		// Only call OnStopped if the job actually stopped.
		if (PreviousJobStatus != JobStatus)
		{
			OnStopped(InCurrentTime);
		}
	}
}

void UTmvMediaTranscodeJob::Discard(double InCurrentTime)
{
	if (IsRunning())
	{
		UE_LOGF(LogTmvMedia, Warning, "Transcode Job for \"%ls\" is being discarded while still running. Stopping... ", *Settings.GetInputPath());
		RequestStop(InCurrentTime, ETmvMediaTranscodeJobStopReason::Discarded);
	}

	// Close explicitly: OnStopped only fires when stages settle inline, so async stops leave
	// the toast dangling until GC. No-op once NotificationItem has already been cleared.
	if (Notification.IsValid())
	{
		if (StopReason == ETmvMediaTranscodeJobStopReason::Cancelled)
		{
			Notification->SetText(LOCTEXT("TranscodeCancelled", "Transcode Cancelled"));
		}
		Notification->Close(/*bInSuccess*/ false);
		Notification.Reset();
	}

	UE_LOGF(LogTmvMedia, Verbose, "Discarding Transcode Job for \"%ls\".", *Settings.GetInputPath());
	ITmvMediaTranscodeJobManager::SafeUnregisterTranscodeJob(this);
}

void UTmvMediaTranscodeJob::Tick(const FTmvMediaTranscodeJobTime& InTime)
{
	if (JobStatus != ETmvMediaTranscodeJobStatus::Stopped)
	{
		// Cancel the job in case of error.
		if (bIsError)
		{
			RequestStop(InTime.CurrentTime, ETmvMediaTranscodeJobStopReason::Cancelled);
			return;
		}
		
		for (UTmvMediaTranscodeStage* Stage : Stages)
		{
			Stage->Tick(this, InTime);
		}

		if (AreStagesStopped())
		{
			if (StopReason == ETmvMediaTranscodeJobStopReason::None)
			{
				StopReason = ETmvMediaTranscodeJobStopReason::Completed;
			}
			JobStatus = ETmvMediaTranscodeJobStatus::Stopped;
			OnStopped(InTime.CurrentTime);
		}
	}
}

void UTmvMediaTranscodeJob::UpdateProgress(int32 InCurrentFrame, int32 InTotalFrames)
{
	if (Notification.IsValid())
	{
		Notification->SetText(FText::Format(LOCTEXT("TranscodeProgress", "Transcode Completed {0}/{1}"),
			FText::AsNumber(InCurrentFrame), FText::AsNumber(InTotalFrames)));
	}

	{
		FScopeLock lock(&JobStatsLock);
		JobStats.ProcessedFrame = InCurrentFrame;
		JobStats.TotalFramesToProcess = InTotalFrames;
	}
}

FTmvMediaTranscodingJobStats UTmvMediaTranscodeJob::GetJobStats() const
{
	FScopeLock lock(&JobStatsLock);
	return JobStats;
}

bool UTmvMediaTranscodeJob::IsCompleted() const
{
	return JobStatus == ETmvMediaTranscodeJobStatus::Stopped;
}

bool UTmvMediaTranscodeJob::AreStagesStopped() const 
{
	for (const UTmvMediaTranscodeStage* Stage : Stages)
	{
		if (Stage->GetStageStatus() != ETmvMediaTranscodeStageStatus::Stopped)
		{
			return false;
		}
	}
	return true;
}

void UTmvMediaTranscodeJob::BeginDestroy()
{
	Super::BeginDestroy();
	ITmvMediaTranscodeJobManager::SafeUnregisterTranscodeJob(this);
}

#undef LOCTEXT_NAMESPACE