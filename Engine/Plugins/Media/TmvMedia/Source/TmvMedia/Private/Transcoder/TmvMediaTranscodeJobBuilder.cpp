// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaTranscodeJobBuilder.h"

#include "Encoder/TmvMediaEncoderOptions.h"
#include "Internationalization/Internationalization.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaContainerTranscodeMuxer.h"
#include "Transcoder/TmvMediaFileSequenceFrameProducer.h"
#include "Transcoder/TmvMediaFileSequenceTranscodeMuxer.h"
#include "Transcoder/TmvMediaFrameConverter.h"
#include "Transcoder/TmvMediaFrameProducer.h"
#include "Transcoder/TmvMediaPlayerFrameProducer.h"
#include "Transcoder/TmvMediaTmvFrameEncoder.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeList.h"
#include "Transcoder/TmvMediaTranscodeMuxer.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Utils/TmvMediaMessageContext.h"

FTmvMediaTranscodeJobBuilder::FTmvMediaTranscodeJobBuilder(const FTmvMediaTranscodeListItem& InJobItem)
	: JobItem(InJobItem)
{
}

FTmvMediaTranscodeJobBuilder& FTmvMediaTranscodeJobBuilder::WithOuter(UObject* InOuter)
{
	Outer = InOuter;
	return *this;
}

FTmvMediaTranscodeJobBuilder& FTmvMediaTranscodeJobBuilder::WithFrameProducerClass(TSubclassOf<UTmvMediaFrameProducer> InClass)
{
	FrameProducerClass = InClass;
	return *this;
}

FTmvMediaTranscodeJobBuilder& FTmvMediaTranscodeJobBuilder::WithFrameConverterClass(TSubclassOf<UTmvMediaFrameConverter> InClass)
{
	FrameConverterClass = InClass;
	return *this;
}

FTmvMediaTranscodeJobBuilder& FTmvMediaTranscodeJobBuilder::WithFrameEncoderClass(TSubclassOf<UTmvMediaFrameEncoder> InClass)
{
	FrameEncoderClass = InClass;
	return *this;
}

FTmvMediaTranscodeJobBuilder& FTmvMediaTranscodeJobBuilder::WithMuxerClass(TSubclassOf<UTmvMediaTranscodeMuxer> InClass)
{
	MuxerClass = InClass;
	return *this;
}

FTmvMediaTranscodeJobBuilder& FTmvMediaTranscodeJobBuilder::OnPostBuild(FOnTranscodeJobBuilt InDelegate)
{
	PostBuildDelegate = MoveTemp(InDelegate);
	return *this;
}

UTmvMediaTranscodeJob* FTmvMediaTranscodeJobBuilder::Build(FTmvMediaMessageContext* OutMessageContext) const
{
	// Container output requires an encoder with a non-zero FourCC. Refuse to build when the user
	// configured an encoder that can't be muxed into a container (e.g. EXR) so the mismatch is
	// surfaced immediately rather than silently picking a different output format.
	if (JobItem.Settings.OutputFormat == ETmvMediaTranscodeOutputFormat::Container)
	{
		const FTmvMediaEncoderOptions* EncoderOptionsPtr = JobItem.EncoderOptions.GetPtr<FTmvMediaEncoderOptions>();
		if (EncoderOptionsPtr && EncoderOptionsPtr->GetCodecFourCC() == 0)
		{
			FText Message = FText::Format(
				NSLOCTEXT("TmvMediaTranscodeJobBuilder", "ContainerNotSupportedByEncoder",
					"Job \"{0}\": encoder \"{1}\" cannot be muxed into a container. Pick a container-capable encoder or change the Output Format to Image Sequence."),
				FText::FromString(JobItem.Name),
				FText::FromName(EncoderOptionsPtr->GetEncoderName()));

			UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogTmvMedia, Error, TEXT("TmvMediaTranscodeJobBuilder"), Message);
			return nullptr;
		}
	}

	UObject* EffectiveOuter = Outer ? Outer.Get() : GetTransientPackage();

	UTmvMediaTranscodeJob* TranscodeJob = NewObject<UTmvMediaTranscodeJob>(EffectiveOuter, NAME_None);
	if (!TranscodeJob)
	{
		UE_LOGF(LogTmvMedia, Error, "FTmvMediaTranscodeJobBuilder: Failed to create UTmvMediaTranscodeJob.");
		return nullptr;
	}

	TranscodeJob->SetId(JobItem.Id);
	TranscodeJob->JobName = JobItem.Name;
	TranscodeJob->Settings = JobItem.Settings;

	// Frame producer: caller override, or default from Settings.bUseMediaPlayer.
	UClass* EffectiveFrameProducerClass = FrameProducerClass.Get();
	if (!EffectiveFrameProducerClass)
	{
		EffectiveFrameProducerClass = TranscodeJob->Settings.bUseMediaPlayer
			? static_cast<UClass*>(UTmvMediaPlayerFrameProducer::StaticClass())
			: static_cast<UClass*>(UTmvMediaFileSequenceFrameProducer::StaticClass());
	}
	TranscodeJob->AddStage(NewObject<UTmvMediaFrameProducer>(TranscodeJob, EffectiveFrameProducerClass, NAME_None));

	// Frame converter.
	UClass* EffectiveFrameConverterClass = FrameConverterClass.Get();
	if (!EffectiveFrameConverterClass)
	{
		EffectiveFrameConverterClass = UTmvMediaFrameConverter::StaticClass();
	}
	TranscodeJob->AddStage(NewObject<UTmvMediaFrameConverter>(TranscodeJob, EffectiveFrameConverterClass, NAME_None));

	// Frame encoder.
	UClass* EffectiveFrameEncoderClass = FrameEncoderClass.Get();
	if (!EffectiveFrameEncoderClass)
	{
		EffectiveFrameEncoderClass = UTmvMediaTmvFrameEncoder::StaticClass();
	}
	TranscodeJob->AddStage(NewObject<UTmvMediaFrameEncoder>(TranscodeJob, EffectiveFrameEncoderClass, NAME_None));

	// Muxer: caller override, or default from Settings.OutputFormat.
	UClass* EffectiveMuxerClass = MuxerClass.Get();
	if (!EffectiveMuxerClass)
	{
		EffectiveMuxerClass = (TranscodeJob->Settings.OutputFormat == ETmvMediaTranscodeOutputFormat::Container)
			? static_cast<UClass*>(UTmvMediaContainerTranscodeMuxer::StaticClass())
			: static_cast<UClass*>(UTmvMediaFileSequenceTranscodeMuxer::StaticClass());
	}
	TranscodeJob->AddStage(NewObject<UTmvMediaTranscodeMuxer>(TranscodeJob, EffectiveMuxerClass, NAME_None));

	if (UTmvMediaFrameEncoder* EncoderStage = TranscodeJob->GetStage<UTmvMediaFrameEncoder>())
	{
		EncoderStage->SetEncoderOptions(JobItem.EncoderOptions);
	}

	PostBuildDelegate.ExecuteIfBound(TranscodeJob);

	return TranscodeJob;
}
