// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaTranscodeJobManager.h"
#include "ITmvMediaModule.h"
#include "Transcoder/TmvMediaTranscodeJob.h"

ITmvMediaTranscodeJobManager* ITmvMediaTranscodeJobManager::Get()
{
	if (const ITmvMediaModule* TmvModule = ITmvMediaModule::Get())
	{
		return TmvModule->GetTranscodeJobManager();
	}
	return nullptr;
}

void FTmvMediaTranscodeJobManager::GetTranscodeJobs(TArray<TWeakObjectPtr<UTmvMediaTranscodeJob>>& OutTranscodeJobs) const
{
	FScopeLock lock(&TranscodeJobsLock);
	OutTranscodeJobs.Append(TranscodeJobs);
}

UTmvMediaTranscodeJob* FTmvMediaTranscodeJobManager::GetTranscodeJob(const FGuid& InJobId) const
{
	FScopeLock lock(&TranscodeJobsLock);
	const TWeakObjectPtr<UTmvMediaTranscodeJob>* FoundTranscodeJob = TranscodeJobsById.Find(InJobId);
	return FoundTranscodeJob ? FoundTranscodeJob->Get() : nullptr;
}

void FTmvMediaTranscodeJobManager::RegisterTranscodeJob(UTmvMediaTranscodeJob* InTranscodeJob)
{
	if(!InTranscodeJob)
	{
		return;
	}

	bool bJobExisted;
	
	{
		FScopeLock lock(&TranscodeJobsLock);
		PruneStaleTranscodeJobs();
		
		bJobExisted = TranscodeJobs.Contains(InTranscodeJob);
		if (!bJobExisted)
		{
			TranscodeJobs.Add(InTranscodeJob);
		}
		
		// Update the transcode by id lookup even if the job was already registered, it might have changed id.
		if (InTranscodeJob->GetId().IsValid())
		{
			if (bJobExisted)
			{
				// Remove any previous id(s) that still map to this job.
				for (TMap<FGuid, TWeakObjectPtr<UTmvMediaTranscodeJob>>::TIterator It = TranscodeJobsById.CreateIterator(); It; ++It)
				{
					if (It.Value() == InTranscodeJob && It.Key() != InTranscodeJob->GetId())
					{
						It.RemoveCurrent();
					}
				}
			} 
			
			TranscodeJobsById.Add(InTranscodeJob->GetId(), InTranscodeJob);
		}
	}

	// Only signal if the job was added. (broadcast outside the lock.)
	if (!bJobExisted)
	{
		OnTranscodeJobAdded.Broadcast(InTranscodeJob);
	}
}

void FTmvMediaTranscodeJobManager::UnregisterTranscodeJob(UTmvMediaTranscodeJob* InTranscodeJob)
{
	if (InTranscodeJob)
	{
		{
			FScopeLock lock(&TranscodeJobsLock);
			TranscodeJobs.Remove(InTranscodeJob);

			// Ensures no Id aliases remain for an unregistered job, even if the job’s Id changed over time or is currently invalid.
			for (TMap<FGuid, TWeakObjectPtr<UTmvMediaTranscodeJob>>::TIterator It = TranscodeJobsById.CreateIterator(); It; ++It)
			{
				if (It.Value() == InTranscodeJob)
				{
					It.RemoveCurrent();
				}
			}

			PruneStaleTranscodeJobs();
		}

		// Broadcast outside the lock.
		OnTranscodeJobRemoved.Broadcast(InTranscodeJob);
	}
}

void FTmvMediaTranscodeJobManager::PruneStaleTranscodeJobs()
{
	for (TArray<TWeakObjectPtr<UTmvMediaTranscodeJob>>::TIterator Iter(TranscodeJobs); Iter; ++Iter)
	{
		if (Iter->IsStale())
		{
			Iter.RemoveCurrent();
		}
	}

	for (TMap<FGuid, TWeakObjectPtr<UTmvMediaTranscodeJob>>::TIterator Iter(TranscodeJobsById); Iter; ++Iter)
	{
		if (Iter.Value().IsStale())
		{
			Iter.RemoveCurrent();
		}
	}
}
