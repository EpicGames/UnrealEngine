// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Transcoder/ITmvMediaTranscodeJobManager.h"
#include "UObject/WeakObjectPtr.h"

/**
 * Singleton manager for all transcode jobs.
 */
class FTmvMediaTranscodeJobManager : public TSharedFromThis<FTmvMediaTranscodeJobManager>, public ITmvMediaTranscodeJobManager
{
public:
	virtual ~FTmvMediaTranscodeJobManager() override = default;
	
	//~ Begin ITmvMediaTranscodeJobManager
	virtual void GetTranscodeJobs(TArray<TWeakObjectPtr<UTmvMediaTranscodeJob>>& OutTranscodeJobs) const override;
	
	virtual UTmvMediaTranscodeJob* GetTranscodeJob(const FGuid& InJobId) const override;
	
	virtual void RegisterTranscodeJob(UTmvMediaTranscodeJob* InTranscodeJob) override;

	virtual void UnregisterTranscodeJob(UTmvMediaTranscodeJob* InTranscodeJob) override;
	
	virtual FOnTranscodeJobEvent& GetOnTranscodeJobAdded() override
	{
		return OnTranscodeJobAdded;
	}

	virtual FOnTranscodeJobEvent& GetOnTranscodeJobRemoved() override
	{
		return OnTranscodeJobRemoved;
	}
	//~ End ITmvMediaTranscodeJobManager

private:
	/** Helper function to remove stale jobs from the internal containers. */
	void PruneStaleTranscodeJobs();

	/** Delegate for OnTranscodeJobAdded. */ 
	FOnTranscodeJobEvent OnTranscodeJobAdded;

	/** Delegate for OnTranscodeJobRemoved. */ 
	FOnTranscodeJobEvent OnTranscodeJobRemoved;

	/** Critical section for TranscodeJobs containers. */
	mutable FCriticalSection TranscodeJobsLock;

	/** Container for all registered jobs, regardless of ids. */
	TArray<TWeakObjectPtr<UTmvMediaTranscodeJob>> TranscodeJobs;
	
	/** Container for registered jobs with an Id. */
	TMap<FGuid, TWeakObjectPtr<UTmvMediaTranscodeJob>> TranscodeJobsById;
};