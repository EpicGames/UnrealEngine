// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaTranscodeJobRunner.h"

#include "ITmvMediaModule.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Stats/Stats.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeStage.h"

ITmvMediaTranscodeJobRunner* ITmvMediaTranscodeJobRunner::Get()
{
	if (const ITmvMediaModule* TmvModule = ITmvMediaModule::Get())
	{
		return TmvModule->GetTranscodeJobRunner();
	}
	return nullptr;
}

FTmvMediaTranscodeJobRunner::FTmvMediaTranscodeJobRunner()
{
	// Subscribed at runner construction so cleanup runs before module unload begins. The
	// dtor itself is too late: UnloadModulesAtShutdown unloads plugin modules
	// (encoder/muxer/demuxer/MediaPlayer) below TmvMedia first, so by the time we'd run
	// CancelAll from there the stage UObjects' vtables already point at freed code.
	EnginePreExitHandle = FCoreDelegates::OnEnginePreExit.AddRaw(this, &FTmvMediaTranscodeJobRunner::OnEnginePreExit);
}

FTmvMediaTranscodeJobRunner::~FTmvMediaTranscodeJobRunner()
{
	FCoreDelegates::OnEnginePreExit.Remove(EnginePreExitHandle);
}

void FTmvMediaTranscodeJobRunner::OnEnginePreExit()
{
	CancelAll();
}

void FTmvMediaTranscodeJobRunner::EnqueueJob(UTmvMediaTranscodeJob* InJob, const FTmvMediaTranscodeJobRunOptions& InOptions)
{
	if (!InJob)
	{
		return;
	}

	PendingJobs.Add(FQueuedJob{ TStrongObjectPtr<UTmvMediaTranscodeJob>(InJob), InOptions });

	if (!CurrentJob.IsValid())
	{
		StartNextJob();
	}
}

void FTmvMediaTranscodeJobRunner::CancelAll()
{
	const double Now = FApp::GetCurrentTime();

	if (CurrentJob.IsValid())
	{
		// Signal stages to stop, then finalize symmetrically with pending jobs below. This
		// fires OnJobFinished for the active job synchronously instead of deferring to Tick,
		// so callers that won't tick again (e.g., the OnEnginePreExit shutdown hook) still
		// notify listeners. Stages may still be wrapping up async work after this returns;
		// that's an unavoidable race that exists whether we finalize here or wait for Tick.
		CurrentJob->RequestStop(Now, ETmvMediaTranscodeJobStopReason::Cancelled);
		DiscardCurrentJob(Now);
	}

	// Take ownership before broadcasting so listeners see the runner already drained.
	TArray<FQueuedJob> Drained = MoveTemp(PendingJobs);
	for (const FQueuedJob& Pending : Drained)
	{
		if (Pending.Job.IsValid())
		{
			Pending.Job->Discard(Now);
			OnJobFinishedDelegate.Broadcast(Pending.Job.Get());
		}
	}
}

void FTmvMediaTranscodeJobRunner::CancelJob(const FGuid& InJobId)
{
	if (!InJobId.IsValid())
	{
		return;
	}

	const double Now = FApp::GetCurrentTime();

	if (CurrentJob.IsValid() && CurrentJob->GetId() == InJobId)
	{
		// Symmetric with CancelAll's active path and the pending branch below: finalize the
		// job synchronously so OnJobFinished fires immediately and IsJobActiveOrPending
		// returns false on return. Advance to the next pending job (or signal idle) the same
		// way Tick's completion path does, otherwise a pending job would stall behind the
		// now-null CurrentJob (IsTickable gates on CurrentJob.IsValid()).
		CurrentJob->RequestStop(Now, ETmvMediaTranscodeJobStopReason::Cancelled);
		DiscardCurrentJob(Now);
		if (!StartNextJob() && !HasActiveOrPendingJobs())
		{
			OnAllJobsFinishedDelegate.Broadcast();
		}
		return;
	}

	for (int32 Index = 0; Index < PendingJobs.Num(); ++Index)
	{
		if (PendingJobs[Index].Job.IsValid() && PendingJobs[Index].Job->GetId() == InJobId)
		{
			FQueuedJob Removed = MoveTemp(PendingJobs[Index]);
			PendingJobs.RemoveAt(Index);
			Removed.Job->Discard(Now);
			OnJobFinishedDelegate.Broadcast(Removed.Job.Get());
			return;
		}
	}
}

bool FTmvMediaTranscodeJobRunner::IsJobActiveOrPending(const FGuid& InJobId) const
{
	if (CurrentJob.IsValid() && CurrentJob->GetId() == InJobId)
	{
		return true;
	}

	for (const FQueuedJob& Pending : PendingJobs)
	{
		if (Pending.Job.IsValid() && Pending.Job->GetId() == InJobId)
		{
			return true;
		}
	}
	return false;
}

bool FTmvMediaTranscodeJobRunner::HasActiveOrPendingJobs() const
{
	return CurrentJob.IsValid() || !PendingJobs.IsEmpty();
}

void FTmvMediaTranscodeJobRunner::Tick(float InDeltaTime)
{
	if (!CurrentJob.IsValid())
	{
		return;
	}

	const double Now = FApp::GetCurrentTime();

	FTmvMediaTranscodeJobTime JobTime;
	JobTime.CurrentTime = Now;
	JobTime.DeltaTime = InDeltaTime;
	CurrentJob->Tick(JobTime);

	if (!bCurrentJobTimedOut
		&& CurrentJobOptions.TimeoutSeconds > 0.0
		&& (Now - CurrentJobStartTime) > CurrentJobOptions.TimeoutSeconds)
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Job \"%ls\" exceeded timeout of %.1fs - requesting cancel.",
			*CurrentJob->JobName, CurrentJobOptions.TimeoutSeconds);
		CurrentJob->RequestStop(Now, ETmvMediaTranscodeJobStopReason::Cancelled);
		bCurrentJobTimedOut = true;
	}

	if (CurrentJob->IsCompleted())
	{
		DiscardCurrentJob(Now);

		if (!StartNextJob() && !HasActiveOrPendingJobs())
		{
			OnAllJobsFinishedDelegate.Broadcast();
		}
	}
}

bool FTmvMediaTranscodeJobRunner::IsTickable() const
{
	return CurrentJob.IsValid();
}

TStatId FTmvMediaTranscodeJobRunner::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FTmvMediaTranscodeJobRunner, STATGROUP_Tickables);
}

bool FTmvMediaTranscodeJobRunner::StartNextJob()
{
	while (!PendingJobs.IsEmpty())
	{
		FQueuedJob Next = MoveTemp(PendingJobs[0]);
		PendingJobs.RemoveAt(0);

		if (!Next.Job.IsValid())
		{
			continue;
		}

		CurrentJob = MoveTemp(Next.Job);
		CurrentJobOptions = MoveTemp(Next.Options);
		CurrentJobStartTime = FApp::GetCurrentTime();
		bCurrentJobTimedOut = false;

		CurrentJobOptions.OnPreStart.ExecuteIfBound(CurrentJob.Get());

		if (CurrentJob->Start(CurrentJobStartTime))
		{
			UE_LOGF(LogTmvMedia, Display, "Transcode Job \"%ls\" started.", *CurrentJob->JobName);
			OnJobStartedDelegate.Broadcast(CurrentJob.Get());
			return true;
		}

		// Start failed synchronously: discard the job (unregisters from the manager so the
		// Monitor drops it) and notify listeners so their bookkeeping is pruned.
		UE_LOGF(LogTmvMedia, Error, "Transcode Job \"%ls\" failed to start. Skipping.", *CurrentJob->JobName);
		CurrentJob->Discard(FApp::GetCurrentTime());
		OnJobFinishedDelegate.Broadcast(CurrentJob.Get());
		CurrentJob.Reset();
		CurrentJobOptions = FTmvMediaTranscodeJobRunOptions{};
		CurrentJobStartTime = 0.0;
	}
	return false;
}

void FTmvMediaTranscodeJobRunner::DiscardCurrentJob(double InCurrentTime)
{
	if (!CurrentJob.IsValid())
	{
		return;
	}

	if (bCurrentJobTimedOut)
	{
		UE_LOGF(LogTmvMedia, Error, "Transcode Job \"%ls\" aborted after timeout.", *CurrentJob->JobName);
	}
	else if (CurrentJob->StopReason == ETmvMediaTranscodeJobStopReason::Cancelled)
	{
		UE_LOGF(LogTmvMedia, Display, "Transcode Job \"%ls\" cancelled.", *CurrentJob->JobName);
	}
	else
	{
		UE_LOGF(LogTmvMedia, Display, "Transcode Job \"%ls\" completed.", *CurrentJob->JobName);
	}

	// Discard first (stops + unregisters), then broadcast while the strong ref is still
	// held so listeners observe a valid pointer regardless of GC timing.
	CurrentJob->Discard(InCurrentTime);
	OnJobFinishedDelegate.Broadcast(CurrentJob.Get());

	CurrentJob.Reset();
	CurrentJobOptions = FTmvMediaTranscodeJobRunOptions{};
	CurrentJobStartTime = 0.0;
	bCurrentJobTimedOut = false;
}
