// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "Tickable.h"
#include "Transcoder/ITmvMediaTranscodeJobRunner.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "UObject/StrongObjectPtr.h"

/**
 * Game-thread implementation of the transcode job runner.
 *
 * Ticked by FTickableGameObject so the active job advances independently of Slate
 * window state (minimize / hide / close).
 */
class FTmvMediaTranscodeJobRunner final : public ITmvMediaTranscodeJobRunner, public FTickableGameObject
{
public:
	FTmvMediaTranscodeJobRunner();
	virtual ~FTmvMediaTranscodeJobRunner() override;

	//~ Begin ITmvMediaTranscodeJobRunner
	virtual void EnqueueJob(UTmvMediaTranscodeJob* InJob, const FTmvMediaTranscodeJobRunOptions& InOptions = {}) override;
	virtual void CancelAll() override;
	virtual void CancelJob(const FGuid& InJobId) override;
	virtual bool IsJobActiveOrPending(const FGuid& InJobId) const override;
	virtual bool HasActiveOrPendingJobs() const override;

	virtual FOnRunnerJobEvent& GetOnJobStarted() override { return OnJobStartedDelegate; }
	virtual FOnRunnerJobEvent& GetOnJobFinished() override { return OnJobFinishedDelegate; }
	virtual FOnAllJobsFinished& GetOnAllJobsFinished() override { return OnAllJobsFinishedDelegate; }
	//~ End ITmvMediaTranscodeJobRunner

	//~ Begin FTickableGameObject
	virtual void Tick(float InDeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	virtual bool IsTickable() const override;
	virtual bool IsTickableInEditor() const override { return true; }
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject

private:
	/** Pop and start jobs from the head of the queue until one starts or the queue is drained. */
	bool StartNextJob();

	/** Discard the active job and clear active-job state. Broadcasts OnJobFinished. */
	void DiscardCurrentJob(double InCurrentTime);

	/** Cancel any in-flight work while plugin modules are still loaded (pre-shutdown hook). */
	void OnEnginePreExit();

	/** Handle returned by FCoreDelegates::OnEnginePreExit.Add, removed in the destructor. */
	FDelegateHandle EnginePreExitHandle;

	struct FQueuedJob
	{
		TStrongObjectPtr<UTmvMediaTranscodeJob> Job;
		FTmvMediaTranscodeJobRunOptions Options;
	};

	TStrongObjectPtr<UTmvMediaTranscodeJob> CurrentJob;
	FTmvMediaTranscodeJobRunOptions CurrentJobOptions;
	double CurrentJobStartTime = 0.0;
	bool bCurrentJobTimedOut = false;

	TArray<FQueuedJob> PendingJobs;

	FOnRunnerJobEvent OnJobStartedDelegate;
	FOnRunnerJobEvent OnJobFinishedDelegate;
	FOnAllJobsFinished OnAllJobsFinishedDelegate;
};
