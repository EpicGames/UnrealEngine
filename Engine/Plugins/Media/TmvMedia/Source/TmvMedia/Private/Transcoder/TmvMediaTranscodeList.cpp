// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaTranscodeList.h"

#include "ITmvMediaModule.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Encoder/ITmvMediaMuxerFactory.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaTranscodeList)

FTmvMediaTranscodeListItem UTmvMediaTranscodeList::InvalidItem;

namespace UE::TmvMedia::Transcode
{
	const UScriptStruct* GetDefaultEncoderOptionsStruct()
	{
		// Preferred default encoder options is APV.
		const UScriptStruct* BaseStruct = FTmvMediaEncoderOptions::StaticStruct();

		const UScriptStruct* FoundStruct = nullptr;
		
		for (const UScriptStruct* CurrentStruct : TObjectRange<UScriptStruct>())
		{
			if (CurrentStruct != BaseStruct && CurrentStruct->IsChildOf(BaseStruct))
			{
				if (CurrentStruct->GetName().Contains(TEXT("apv")))
				{
					FoundStruct = CurrentStruct;
					break;
				}
				if (FoundStruct == nullptr)
				{
					FoundStruct = CurrentStruct;
				}
			}
		}

		return FoundStruct;
	}
	
	/** Returns the name of the first registered muxer factory. */ 
	FName GetDefaultMuxerName()
	{
		const ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get();
		if (!TmvMediaModule)
		{
			return NAME_None;
		}

		TArray<TWeakPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>> MuxerFactories;
		TmvMediaModule->GetMuxerFactories(MuxerFactories);
		for (const TWeakPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>& FactoryWeak : MuxerFactories)
		{
			if (const TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> MuxerFactory = FactoryWeak.Pin())
			{
				return MuxerFactory->GetName();
			}
		}
		return NAME_None;
	}
}

void UTmvMediaTranscodeList::InsertItemAt(int32 InItemIndex, const FTmvMediaTranscodeListItem* InItem)
{
	if (InItemIndex < 0)
	{
		return;
	}

	FTmvMediaTranscodeListItem NewItem;
	NewItem.Id = FGuid::NewGuid();

	if (InItem)
	{
		NewItem.Name = GenerateDuplicateName(InItem->Name);
		NewItem.Settings = InItem->Settings;
		NewItem.EncoderOptions = InItem->EncoderOptions;
	}
	else
	{
		NewItem.Name = GenerateNewItemName();
		NewItem.Settings.Muxer.Name = UE::TmvMedia::Transcode::GetDefaultMuxerName();
		if (const UScriptStruct* SelectedStruct = UE::TmvMedia::Transcode::GetDefaultEncoderOptionsStruct())
		{
			NewItem.EncoderOptions.InitializeAsScriptStruct(SelectedStruct);
		}
	}

	if (InItemIndex >= Items.Num())
	{
		Items.Add(MoveTemp(NewItem));
		OnItemEvent.Broadcast(this, FTmvMediaTranscodeListItemEventArgs{ETmvMediaTranscodeListItemEventType::ItemsAdded, {Items.Num()-1}});
	}
	else
	{
		Items.Insert(MoveTemp(NewItem), InItemIndex);
		OnItemEvent.Broadcast(this, FTmvMediaTranscodeListItemEventArgs{ETmvMediaTranscodeListItemEventType::ItemsAdded, {InItemIndex}});
	}
}
	
bool UTmvMediaTranscodeList::RemoveItemAt(int32 InItemIndex)
{
	if (Items.IsValidIndex(InItemIndex))
	{
		Items.RemoveAt(InItemIndex);
		OnItemEvent.Broadcast(this, FTmvMediaTranscodeListItemEventArgs{ETmvMediaTranscodeListItemEventType::ItemsRemoved, {InItemIndex}});
		return true;
	}
	return false;
}

void UTmvMediaTranscodeList::RemoveItems(TConstArrayView<int32> InItemIndices)
{
	// We need to remove the items starting from the highest index for the
	// removal to not change indices that are not yet removed.

	TArray<int32> SortedIndices;
	SortedIndices.Append(InItemIndices);
	Algo::Sort(SortedIndices);

	// Extra safety: Remove duplicate indices to prevent accidental extra deletions.
	SortedIndices.SetNum(Algo::Unique(SortedIndices));

	TArray<int32> RemovedIndices;
	RemovedIndices.Reserve(SortedIndices.Num());

	for (int32 Index : ReverseIterate(SortedIndices))
	{
		if (Items.IsValidIndex(Index))
		{
			Items.RemoveAt(Index);
			RemovedIndices.Add(Index);
		}
	}
	
	if (!RemovedIndices.IsEmpty())
	{
		FTmvMediaTranscodeListItemEventArgs ItemEventArgs;
		ItemEventArgs.Type = ETmvMediaTranscodeListItemEventType::ItemsRemoved;
		ItemEventArgs.ItemIndices = MoveTemp(RemovedIndices);
		OnItemEvent.Broadcast(this, ItemEventArgs);
	}
}

bool UTmvMediaTranscodeList::ValidateJobItem(const FTmvMediaTranscodeListItem& InJobItem, FString* OutError)
{
	auto AppendReason = [OutError](const TCHAR* InReason)
	{
		if (OutError)
		{
			if (!OutError->IsEmpty())
			{
				OutError->AppendChar(TEXT('\n'));
			}
			OutError->Append(InReason);
		}
	};

	bool bValid = true;
	if (!InJobItem.Settings.IsInputPathSet())
	{
		AppendReason(TEXT("Input path is not set."));
		bValid = false;
	}
	if (!InJobItem.Settings.IsOutputPathSet())
	{
		AppendReason(TEXT("Output path is not set."));
		bValid = false;
	}
	if (InJobItem.Settings.bMakeOutputAsset && InJobItem.Settings.OutputAssetDirectory.Path.IsEmpty())
	{
		AppendReason(TEXT("bMakeOutputAsset is set but OutputAssetDirectory is empty."));
		bValid = false;
	}
	return bValid;
}

bool UTmvMediaTranscodeList::ReorderItems(int32 InFromIndex, int32 InToIndex)
{
	if (!Items.IsValidIndex(InFromIndex) || InToIndex < 0)
	{
		return false;
	}

	FTmvMediaTranscodeListItem ItemToMove = Items[InFromIndex];

	int32 MovedIndex;
	
	if (InToIndex >= Items.Num())
	{
		Items.Add(MoveTemp(ItemToMove));
		MovedIndex = Items.Num() - 1;
	}
	else
	{
		Items.Insert(MoveTemp(ItemToMove), InToIndex);
		MovedIndex = InToIndex;
	}
	
	if (MovedIndex > InFromIndex)
	{
		--MovedIndex;
		Items.RemoveAt(InFromIndex);
	}
	else
	{
		Items.RemoveAt(InFromIndex+1);
	}

	FTmvMediaTranscodeListItemEventArgs ItemEventArgs;
	ItemEventArgs.Type = ETmvMediaTranscodeListItemEventType::ItemsReordered;
	ItemEventArgs.ItemIndices.Add(MovedIndex);
	OnItemEvent.Broadcast(this, ItemEventArgs);
	return true;
}

void UTmvMediaTranscodeList::PostLoad()
{
	Super::PostLoad();
	
	for (FTmvMediaTranscodeListItem& Item : Items)
	{
		if (!Item.Id.IsValid())
		{
			Item.Id = FGuid::NewGuid();
		}
	}
}

#if WITH_EDITOR
void UTmvMediaTranscodeList::PostEditUndo()
{
	Super::PostEditUndo();

	// Trigger a ui refresh
	GetOnItemEvent().Broadcast(this, {ETmvMediaTranscodeListItemEventType::ItemsModified, {}});
}
#endif

FString UTmvMediaTranscodeList::GenerateNewItemName() const
{
	// Find the highest existing JobItem_N number to avoid collisions when items have been deleted.
	static const FString Prefix = TEXT("JobItem_");
	int32 MaxNumber = -1;
	for (const FTmvMediaTranscodeListItem& Item : Items)
	{
		if (Item.Name.StartsWith(Prefix))
		{
			int32 Number;
			if (LexTryParseString(Number, *Item.Name.RightChop(Prefix.Len())))
			{
				MaxNumber = FMath::Max(MaxNumber, Number);
			}
		}
	}
	return FString::Printf(TEXT("JobItem_%d"), MaxNumber + 1);
}

FString UTmvMediaTranscodeList::GenerateDuplicateName(const FString& InSourceName) const
{
	static const FString DupSuffix = TEXT("_Dup");

	// Strip any existing _Dup or _DupN suffix to get the base name.
	FString BaseName = InSourceName;
	if (BaseName.EndsWith(DupSuffix))
	{
		BaseName.LeftChopInline(DupSuffix.Len());
	}
	else
	{
		const int32 LastDupIdx = BaseName.Find(DupSuffix, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (LastDupIdx != INDEX_NONE)
		{
			const FString Tail = BaseName.RightChop(LastDupIdx + DupSuffix.Len());
			int32 Num;
			if (LexTryParseString(Num, *Tail))
			{
				BaseName.LeftInline(LastDupIdx);
			}
		}
	}

	// Find the highest existing dup index for this base name.
	// _Dup counts as index 1, _Dup2 as 2, etc.
	const FString DupPrefix = BaseName + DupSuffix;
	int32 MaxDupIndex = 0;
	for (const FTmvMediaTranscodeListItem& ExistingItem : Items)
	{
		if (ExistingItem.Name == DupPrefix)
		{
			MaxDupIndex = FMath::Max(MaxDupIndex, 1);
		}
		else if (ExistingItem.Name.StartsWith(DupPrefix))
		{
			const FString Tail = ExistingItem.Name.RightChop(DupPrefix.Len());
			int32 Num;
			if (LexTryParseString(Num, *Tail) && Num >= 2)
			{
				MaxDupIndex = FMath::Max(MaxDupIndex, Num);
			}
		}
	}

	return MaxDupIndex == 0
		? DupPrefix
		: FString::Printf(TEXT("%s_Dup%d"), *BaseName, MaxDupIndex + 1);
}


