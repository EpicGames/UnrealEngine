// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaTranscodeMuxer.h"

#include "ITmvMediaModule.h"
#include "TmvMediaLog.h"
#include "Encoder/ITmvMediaMuxerFactory.h"
#include "Misc/Paths.h"
#include "Transcoder/TmvMediaTranscodeJob.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaTranscodeMuxer)

FString UTmvMediaTranscodeMuxer::GetContainerOutputFilePath(const UTmvMediaTranscodeJob* InParentJob, const TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>& InMuxerFactory)
{
	if (!IsValid(InParentJob))
	{
		return FString();
	}

	TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> MuxerFactory = InMuxerFactory;

	if (!MuxerFactory)
	{
		const FName MuxerName(InParentJob->Settings.Muxer.Name);
		MuxerFactory = FindMuxerFactoryByName(MuxerName);
		if (!MuxerFactory)
		{
			UE_LOGF(LogTmvMedia, Warning, "No Muxer Factory for \"%ls\".", *InParentJob->Settings.Muxer.Name.ToString());
		}
	}

	const FString OutPath = InParentJob->Settings.GetAbsoluteOutputPath();
	const FString BaseName = !InParentJob->Settings.OutputBaseName.IsEmpty() ? InParentJob->Settings.OutputBaseName : InParentJob->JobName;
	const FString FileExtension = MuxerFactory ? MuxerFactory->GetFileExtension() : TEXT("tmv");
	return FPaths::Combine(OutPath, BaseName + TEXT(".") + FileExtension);
}

TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> UTmvMediaTranscodeMuxer::FindMuxerFactoryByName(const FName InFactoryName)
{
	const ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get();
	if (!TmvMediaModule)
	{
		return nullptr;
	}
	
	TArray<TWeakPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>> MuxerFactories;
	TmvMediaModule->GetMuxerFactories(MuxerFactories);
	for (const TWeakPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>& FactoryWeak : MuxerFactories)
	{
		if (TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> MuxerFactory = FactoryWeak.Pin())
		{
			if (MuxerFactory->GetName() == InFactoryName)
			{
				return MuxerFactory;
			}
		}
	}
	return nullptr;
}
