// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaTranscodeSerialization.h"

#include "Encoder/TmvMediaEncoderOptions.h"
#include "JsonObjectConverter.h"
#include "StructUtils/InstancedStruct.h"
#include "TmvMediaLog.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeList.h"
#include "Utils/TmvMediaSerializationUtils.h"

namespace UE::TmvMedia::TranscodeSerialization
{
	/**
	 * Try to get the field named FieldName as an object or return null if it's another type.
	 * (no assert/ensure/crash version)
	 */
	TSharedPtr<FJsonObject> SafeGetObjectField(const FJsonObject& InJsonRoot, FStringView InFieldName)
	{
		const TSharedPtr<FJsonObject>* OutObjectPtr;
		if(InJsonRoot.TryGetObjectField(InFieldName, OutObjectPtr))
		{
			return *OutObjectPtr;
		}
		return nullptr;
	}

	/**
	 * Try to get the field named FieldName as a string. Returns empty string if it doesn't exist or cannot be converted.
	  * (no assert/ensure/crash version)
	 */
	FString SafeGetStringField(const FJsonObject& InJsonRoot, FStringView InFieldName)
	{
		FString OutString;
		if(InJsonRoot.TryGetStringField(InFieldName, OutString))
		{
			return OutString;
		}
		return TEXT("");
	}
	
	bool SerializeTranscodeJobSettingsToJson(
		FArchive& InArchive,
		const FTmvMediaTranscodeJobSettings* InJobSettings,
		const TInstancedStruct<FTmvMediaEncoderOptions>& InEncoderOptions)
	{
		if (!InJobSettings)
		{
			UE_LOGF(LogTmvMedia, Error, "Serialize Transcode Job Settings: InJobSettings is null");
			return false;
		}

		if (!InEncoderOptions.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Serialize Transcode Job Settings: Encoder options are not initialized");
			return false;
		}
		
		using namespace SerializationUtils;
		const TSharedPtr<FJsonObject> JsonJobSettings = SerializeToJson(FTmvMediaTranscodeJobSettings::StaticStruct(), InJobSettings);
		
		if (!JsonJobSettings.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Serialize Transcode Job Settings: Failed to serialize Job Settings");
			return false;
		}

		const FSoftObjectPath EncoderOptionsStructPath(InEncoderOptions.GetScriptStruct());
		
		const TSharedPtr<FJsonObject> JsonEncoderOptions = SerializeToJson(InEncoderOptions.GetScriptStruct(), InEncoderOptions.GetPtr());

		if (!JsonEncoderOptions.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Serialize Transcode Job Settings: Failed to serialize Encoder Options");
			return false;
		}

		const TSharedPtr<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
		// Keep a file version for legacy support.
		JsonRoot->SetStringField(TEXT("TmvMediaTranscodeSettingsFileVersion"), TEXT("1.0"));
		JsonRoot->SetObjectField(TEXT("JobSettings"), JsonJobSettings);
		JsonRoot->SetStringField(TEXT("EncoderOptionsStruct"), EncoderOptionsStructPath.ToString());
		JsonRoot->SetObjectField(TEXT("EncoderOptions"), JsonEncoderOptions);

		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&InArchive);
		if (!FJsonSerializer::Serialize(JsonRoot.ToSharedRef(), Writer))
		{
			UE_LOGF(LogTmvMedia, Error, "Serialize Transcode Job Settings: Failed to write json string");
			return false;
		}
		return true;
	}
	
	bool DeserializeTranscodeJobSettingsFromJson(
		FArchive& InArchive,
		FTmvMediaTranscodeJobSettings* OutJobSettings,
		TInstancedStruct<FTmvMediaEncoderOptions>& OutEncoderOptions)
	{
		if (!OutJobSettings)
		{
			UE_LOGF(LogTmvMedia, Error, "Deserialize Transcode Job Settings: OutJobSettings is null");
			return false;
		}

		using namespace SerializationUtils;

		TSharedPtr<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
		const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(&InArchive);
		if (!FJsonSerializer::Deserialize(JsonReader, JsonRoot) || !JsonRoot.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't parse json data.");
			return false;
		}
		
		const TSharedPtr<FJsonObject> JobSettings = SafeGetObjectField(*JsonRoot, TEXT("JobSettings"));
		if (!JobSettings.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't find \"JobSettings\" in json.");
			return false;
		}

		FText ErrorMessage;

		if (!DeserializeFromJson(JobSettings.ToSharedRef(), FTmvMediaTranscodeJobSettings::StaticStruct(), OutJobSettings, ErrorMessage))
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't load job settings from json: %ls.", *ErrorMessage.ToString());
			return false;
		}

		FSoftObjectPath EncoderOptionsStructPath(SafeGetStringField(*JsonRoot, TEXT("EncoderOptionsStruct")));
		
		UObject* EncoderOptionsStructObj = EncoderOptionsStructPath.ResolveObject();
		if (!EncoderOptionsStructObj)
		{
			EncoderOptionsStructObj = EncoderOptionsStructPath.TryLoad();
		}
		
		const UScriptStruct* EncoderOptionsStruct = Cast<UScriptStruct>(EncoderOptionsStructObj);
		if (!EncoderOptionsStruct)
		{
			UE_LOGF(LogTmvMedia, Error, "Json Parsing FInstancedStruct Property: Unknown Script Type: \"%ls\"", *EncoderOptionsStructPath.ToString());
			return false;
		}

		if (!EncoderOptionsStruct->IsChildOf(FTmvMediaEncoderOptions::StaticStruct()))
		{
			UE_LOGF(LogTmvMedia, Error, "EncoderOptionsStruct is not derived from FTmvMediaEncoderOptions: \"%ls\"", *EncoderOptionsStruct->GetPathName());
			return false;
		}

		OutEncoderOptions.InitializeAsScriptStruct(EncoderOptionsStruct);
		if (!OutEncoderOptions.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "FInstancedStruct: Failed to create script type: \"%ls\"", *EncoderOptionsStructPath.ToString());
			return false;
		}
		
		const TSharedPtr<FJsonObject> EncoderOptions = SafeGetObjectField(*JsonRoot, TEXT("EncoderOptions"));
		if (!EncoderOptions.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't find \"EncoderOptions\" in json.");
			return false;
		}

		if (!DeserializeFromJson(EncoderOptions.ToSharedRef(), EncoderOptionsStruct, OutEncoderOptions.GetMutablePtr(), ErrorMessage))
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't load \"EncoderOptions\" from json: %ls.", *ErrorMessage.ToString());
			return false;
		}

		UE_LOGF(LogTmvMedia, Verbose, "EncoderOptions and JobSettings successfully loaded from json.");
		return true;
	}
	
	bool SerializeTranscodeListToJson(
	FArchive& InArchive,
	const UTmvMediaTranscodeList& InTranscodeList)
	{
		using namespace SerializationUtils;
		const TSharedPtr<FJsonObject> TranscodeListJson = SerializeToJson(UTmvMediaTranscodeList::StaticClass(), &InTranscodeList);
		
		if (!TranscodeListJson.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Failed to serialize Transcode List to json");
			return false;
		}

		const TSharedPtr<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
		// Keep a file version for legacy support.
		JsonRoot->SetStringField(TEXT("TmvMediaTranscodeListFileVersion"), TEXT("1.0"));
		JsonRoot->SetObjectField(TEXT("TranscodeList"), TranscodeListJson);

		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&InArchive);
		if (!FJsonSerializer::Serialize(JsonRoot.ToSharedRef(), Writer))
		{
			UE_LOGF(LogTmvMedia, Error, "Serialize Transcode List: Failed to write json string");
			return false;
		}
		return true;
	}

	bool DeserializeTranscodeListFromJson(
		FArchive& InArchive,
		UTmvMediaTranscodeList& OutTranscodeList)
	{
		using namespace SerializationUtils;

		TSharedPtr<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
		const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(&InArchive);
		if (!FJsonSerializer::Deserialize(JsonReader, JsonRoot) || !JsonRoot.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't parse json data.");
			return false;
		}
		
		const TSharedPtr<FJsonObject> TranscodeListJson = SafeGetObjectField(*JsonRoot, TEXT("TranscodeList"));
		if (!TranscodeListJson.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't find \"TranscodeList\" field in json.");
			return false;
		}

		FText ErrorMessage;

		if (!DeserializeFromJson(TranscodeListJson.ToSharedRef(), UTmvMediaTranscodeList::StaticClass(), &OutTranscodeList, ErrorMessage))
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't load transcode list from json: %ls.", *ErrorMessage.ToString());
			return false;
		}

		OutTranscodeList.PostLoad();
		
		UE_LOGF(LogTmvMedia, Verbose, "Transcode list successfully loaded from json.");
		return true;
	}

	bool SerializeTranscodeListItemsToJson(
		FArchive& InArchive,
		const TArray<FTmvMediaTranscodeListItem>& InTranscodeListItems)
	{
		using namespace SerializationUtils;
		bool bSerializeError = false;

		TArray<TSharedPtr<FJsonValue>> ItemEntries;
		ItemEntries.Reserve(InTranscodeListItems.Num());

		for (const FTmvMediaTranscodeListItem& Item : InTranscodeListItems)
		{
			if (TSharedPtr<FJsonObject> ItemJson = SerializeToJson(FTmvMediaTranscodeListItem::StaticStruct(), &Item))
			{
				ItemEntries.Add(MakeShared<FJsonValueObject>(ItemJson));	
			}
			else
			{
				UE_LOGF(LogTmvMedia, Error, "Failed to serialize Transcode Item to json");
				bSerializeError = true;
			}
		}

		const TSharedRef<FJsonObject> TranscodeListItemsJson = MakeShared<FJsonObject>();
		TranscodeListItemsJson->SetArrayField(TEXT("Items"), ItemEntries);

		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&InArchive);
		if (!FJsonSerializer::Serialize(TranscodeListItemsJson, Writer))
		{
			UE_LOGF(LogTmvMedia, Error, "Serialize Transcode Items: Failed to write json string");
			bSerializeError = true;
		}
		return !bSerializeError;
	}

	bool DeserializeTranscodeListItemsFromJson(
		FArchive& InArchive,
		TArray<FTmvMediaTranscodeListItem>& OutTranscodeListItems)
	{
		using namespace SerializationUtils;

		TSharedPtr<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
		const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(&InArchive);
		if (!FJsonSerializer::Deserialize(JsonReader, JsonRoot) || !JsonRoot.IsValid())
		{
			UE_LOGF(LogTmvMedia, Error, "Couldn't parse json data.");
			return false;
		}

		const TArray<TSharedPtr<FJsonValue>>* ItemEntries;
		if (!JsonRoot->TryGetArrayField(TEXT("Items"), ItemEntries))
		{
			UE_LOGF(LogTmvMedia, Error, "Missing \"Items\" entry field in pasted text");
			return false;
		}

		OutTranscodeListItems.Reset();
		OutTranscodeListItems.Reserve(ItemEntries->Num());
	
		for (const TSharedPtr<FJsonValue>& ItemEntry : *ItemEntries)
		{
			if (!ItemEntry.IsValid() || ItemEntry->Type != EJson::Object)
			{
				UE_LOGF(LogTmvMedia, Warning, "Invalid item entry. Not an object");
				continue;
			}

			if (const TSharedPtr<FJsonObject>& ItemObject = ItemEntry->AsObject())
			{
				FTmvMediaTranscodeListItem Item;
				FText ErrorMessage;

				if (DeserializeFromJson(ItemObject.ToSharedRef(), FTmvMediaTranscodeListItem::StaticStruct(), &Item, ErrorMessage))
				{
					OutTranscodeListItems.Emplace(MoveTemp(Item));
				}
				else
				{
					UE_LOGF(LogTmvMedia, Warning, "Unable to convert Item Entry Json Object to Transcode List Item Struct.");
				}
			}
			else
			{
				UE_LOGF(LogTmvMedia, Warning, "Item Entry is not a Json Object.");
			}
		}

		UE_LOGF(LogTmvMedia, Verbose, "Transcode list successfully loaded from json.");
		return true;
	}
}
