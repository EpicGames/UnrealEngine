// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transcoder/TmvMediaTranscodeStage.h"

#include "Transcoder/TmvMediaFrameProducer.h"
#include "Transcoder/TmvMediaTranscodeJob.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvMediaTranscodeStage)

void UTmvMediaTranscodeStage::SetStageStatus(ETmvMediaTranscodeStageStatus InStatus, UTmvMediaTranscodeJob* InParentJob)
{
	bool bStatusChanged = InStatus != StageStatus;
	StageStatus = InStatus;

	if (bStatusChanged)
	{
		switch (InStatus)
		{
		case ETmvMediaTranscodeStageStatus::Started:
			UTmvMediaTranscodeJob::SafeBroadcastJobEvent(InParentJob, ETmvMediaTranscodeJobEvent::StageStarted, this);
			break;
		case ETmvMediaTranscodeStageStatus::Stopped:
			UTmvMediaTranscodeJob::SafeBroadcastJobEvent(InParentJob, ETmvMediaTranscodeJobEvent::StageStopped, this);
			break;
		default:
			break;
		}
	}
}

void UTmvMediaTranscodeStage::UpdateStatusFromFrameProducer(UTmvMediaTranscodeJob* InParentJob)
{
	if (InParentJob)
	{
		// Currently, the producer stage is the coordinator of the status since we don't have a local queue for this stage.
		if (const UTmvMediaFrameProducer* FrameProducer = InParentJob->GetStage<UTmvMediaFrameProducer>())
		{
			if (FrameProducer->GetStageStatus() == ETmvMediaTranscodeStageStatus::Stopped)
			{
				RequestStop(InParentJob); // stop current stage too.
			}
		}
	}
}
