// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/TmvMediaFrameUtils.h"

#include "HAL/PlatformFileManager.h"
#include "ImageCore.h"
#include "MediaShaders.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "Serialization/CompactBinary.h"
#include "TmvMediaLog.h"

namespace UE::TmvMedia::FrameUtils
{
	// @todo: move to MediaShaders.
	static  const FVector YUVOffset12bits = FVector(256.0f/4095.0f, 2048.0f/4095.0f, 2048.0f/4095.0f);
	static const FVector YUVOffsetNoScale12bits = FVector(0.0f, 2048.0f/4095.0f, 2048.0f/4095.0f);

	FVector GetYuvOffset(ETmvMediaFrameComponentType InType, ETmvMediaFrameColorMatrixRange InColorMatrixRange, uint8 InBitDepth)
	{
		bool bScaled = InColorMatrixRange == ETmvMediaFrameColorMatrixRange::Limited;

		if (InType == ETmvMediaFrameComponentType::Int)
		{
			switch (InBitDepth)
			{
			case 8:
				return bScaled ? MediaShaders::YUVOffset8bits : MediaShaders::YUVOffsetNoScale8bits;
			case 10:
				return bScaled ? MediaShaders::YUVOffset10bits : MediaShaders::YUVOffsetNoScale10bits;
			case 12:
				return bScaled ? YUVOffset12bits : YUVOffsetNoScale12bits;
			case 16:
				return bScaled ? MediaShaders::YUVOffset16bits : MediaShaders::YUVOffsetNoScale16bits;
			default:
				return bScaled ? MediaShaders::YUVOffset16bits : MediaShaders::YUVOffsetNoScale16bits;
			}
		}

		return bScaled ? MediaShaders::YUVOffsetFloat : MediaShaders::YUVOffsetNoScaleFloat;
	}
	
	// @todo: move to MediaShaders.
	static const FMatrix RgbToYuvRec601Unscaled = FMatrix(
		FPlane(0.299f, 0.587f, 0.114f, 0.0f),
		FPlane(-0.147f, -0.289f, 0.436f, 0.0f),
		FPlane(0.615f, -0.515f, -0.100f, 0.0f),
		FPlane(0.0f, 0.0f, 0.0f, 0.0f)
	);

	static const FMatrix RgbToYuvRec601Scaled = FMatrix(
		FPlane(0.256788235295038, 0.5041294117662803, 0.0979058823532755, 0.0),
		FPlane(-0.1482229008985619, -0.2909927853759723, 0.4392156862745341, 0.0),
		FPlane(0.4392156862745341, -0.3677883136136081, -0.071427372660926, 0.0),
		FPlane(0.0f, 0.0f, 0.0f, 0.0f)
	);
 
	static const FMatrix RgbToYuvRec709Unscaled = FMatrix(
		FPlane(0.2126390058710534, 0.7151686787681136, 0.072192315360833, 0.0),
		FPlane(-0.1145921775554566, -0.3854078224443639, 0.4999999999998206, 0.0),
		FPlane(0.499999999999331, -0.454155517037229, -0.045844482962102, 0.0),
		FPlane(0.0f, 0.0f, 0.0f, 0.0f)
		);
	 
	static const FMatrix RgbToYuvRec2020Unscaled = FMatrix(
		FPlane(0.2627002120113437, 0.6779980715188555, 0.0593017164698007, 0.0),
		FPlane(-0.1396304301872184, -0.3603695698128875, 0.5000000000001059, 0.0),
		FPlane(0.5000000000001059, -0.459784529009949, -0.0402154709901569, 0.0),
		FPlane(0.0f, 0.0f, 0.0f, 0.0f)
		);

	FMatrix GetRgbToYuvMatrix(ETmvMediaFrameColorMatrix InColorMatrix, ETmvMediaFrameColorMatrixRange InColorMatrixRange)
	{
		const bool bScaled = InColorMatrixRange == ETmvMediaFrameColorMatrixRange::Limited;

		switch (InColorMatrix)
		{
		case ETmvMediaFrameColorMatrix::None:
		case ETmvMediaFrameColorMatrix::Identity:
			return FMatrix::Identity;

		case ETmvMediaFrameColorMatrix::Rec709:
			return bScaled ? MediaShaders::RgbToYuvRec709Scaled : RgbToYuvRec709Unscaled;

		case ETmvMediaFrameColorMatrix::Rec601:
			return bScaled ? RgbToYuvRec601Scaled : RgbToYuvRec601Unscaled;

		case ETmvMediaFrameColorMatrix::Rec2020:
			return bScaled ? MediaShaders::RgbToYuvRec2020Scaled : RgbToYuvRec2020Unscaled;

		default:
			return FMatrix::Identity;
		}
	}
	
	FMatrix GetYuvToRgbMatrix(ETmvMediaFrameColorMatrix InColorMatrix, ETmvMediaFrameColorMatrixRange InColorMatrixRange)
	{
		const bool bScaled = InColorMatrixRange == ETmvMediaFrameColorMatrixRange::Limited;

		switch (InColorMatrix)
		{
		case ETmvMediaFrameColorMatrix::None:
		case ETmvMediaFrameColorMatrix::Identity:
			return FMatrix::Identity;

		case ETmvMediaFrameColorMatrix::Rec709:
			return bScaled ? MediaShaders::YuvToRgbRec709Scaled: MediaShaders::YuvToRgbRec709Unscaled;

		case ETmvMediaFrameColorMatrix::Rec601:
			return bScaled ? MediaShaders::YuvToRgbRec601Scaled : MediaShaders::YuvToRgbRec601Unscaled;

		case ETmvMediaFrameColorMatrix::Rec2020:
			return bScaled ? MediaShaders::YuvToRgbRec2020Scaled : MediaShaders::YuvToRgbRec2020Unscaled;

		default:
			return FMatrix::Identity;
		}
	}

	bool PopulateImageInfo(const FTmvMediaFrameMipInfo& InMipInfo, FImageInfo& OutImageInfo, FString* OutError)
	{
		if (InMipInfo.Planes.Num() != 1)
		{
			if (OutError)
			{
				*OutError = TEXT("Converter: Planar formats are not supported in this code path.");
			}
			return false;
		}

		const int32 NumComponents = InMipInfo.Planes[0].NumComponents;

		if (NumComponents != 1 && NumComponents != 4)
		{
			if (OutError)
			{
				*OutError = TEXT("Converter: Only formats with 1 or 4 components are supported.");
			}
			return false;
		}

		OutImageInfo.GammaSpace = InMipInfo.ColorInfo.Encoding == Color::EEncoding::Linear ? EGammaSpace::Linear : EGammaSpace::sRGB;
		OutImageInfo.NumSlices = 1;
		OutImageInfo.SizeX = InMipInfo.Width;
		OutImageInfo.SizeY = InMipInfo.Height;

		if (InMipInfo.Planes[0].Type == ETmvMediaFrameComponentType::Float)
		{
			switch (InMipInfo.Planes[0].BitDepth)
			{
			case 16:
				OutImageInfo.Format = NumComponents == 1 ? ERawImageFormat::R16F : ERawImageFormat::RGBA16F;
				break;

			case 32:
				OutImageInfo.Format = NumComponents == 1 ? ERawImageFormat::R32F : ERawImageFormat::RGBA32F;
				break;

			default:
				if (OutError)
				{
					*OutError = TEXT("Converter: Floating point frame formats other than 16 or 32 bits are not supported.");
				}
				return false;
			}
		}
		else // integer component type
		{
			switch (InMipInfo.Planes[0].BitDepth)
			{
			case 8:
				OutImageInfo.Format = NumComponents == 1 ? ERawImageFormat::G8 : ERawImageFormat::BGRA8;
				break;

			case 16:
				OutImageInfo.Format = NumComponents == 1 ? ERawImageFormat::G16 : ERawImageFormat::RGBA16;
				break;

			default:
				if (OutError)
				{
					*OutError = TEXT("Converter: Integer frame formats other than 8 or 16 bits are not supported.");
				}
				return false;
			}
		}
		return true;
	}

	Color::EEncoding GetColorEncoding(EGammaSpace InGammaSpace)
	{
		switch (InGammaSpace)
		{
		case EGammaSpace::Linear:
			return Color::EEncoding::Linear;

		case EGammaSpace::sRGB:
			return Color::EEncoding::sRGB;

		case EGammaSpace::Pow22:
			return Color::EEncoding::Gamma22;

		default:
			return Color::EEncoding::Linear;
		}
	}

	void PopulateMipInfoFromImageInfo(int32 InMipLevel, const FImageInfo& InImageInfo, FTmvMediaFrameMipInfo& OutMipInfo)
	{
		// Populate the layout information.
		OutMipInfo.MipLevel = InMipLevel;
		OutMipInfo.Width = InImageInfo.GetWidth();
		OutMipInfo.Height = InImageInfo.GetHeight();
		OutMipInfo.NumComponents = ERawImageFormat::NumChannels(InImageInfo.Format);
		OutMipInfo.ColorModel = ETmvMediaFrameColorModel::RGB;
		OutMipInfo.ColorInfo.Encoding = GetColorEncoding(InImageInfo.GammaSpace);
		// There is no color space info in the FImageInfo.
		// We will assume sRGB color space for now.
		OutMipInfo.ColorInfo.ColorSpace = UE::Color::EColorSpace::sRGB; 
		OutMipInfo.TileWidth = InImageInfo.GetWidth();
		OutMipInfo.TileHeight = InImageInfo.GetHeight();
		OutMipInfo.NumTiles = 1;
		OutMipInfo.Layout = ETmvMediaFrameBufferLayout::ScanLine;

		// Packed images have only 1 plane with all the components.
		OutMipInfo.Planes.SetNumUninitialized(1);
		FTmvMediaFramePlaneInfo& PlaneInfo = OutMipInfo.Planes[0];

		PlaneInfo.NumComponents = ERawImageFormat::NumChannels(InImageInfo.Format);
		PlaneInfo.BitDepth = 8 * InImageInfo.GetBytesPerPixel() / PlaneInfo.NumComponents;
		PlaneInfo.Width = InImageInfo.GetWidth();
		PlaneInfo.Height = InImageInfo.GetHeight();
		PlaneInfo.Stride = InImageInfo.GetStrideBytes();
		PlaneInfo.NumLines = InImageInfo.GetHeight();
		PlaneInfo.WidthRatio = 1;
		PlaneInfo.HeightRatio = 1;
		PlaneInfo.Type = ERawImageFormat::IsHDR(InImageInfo.Format) ? ETmvMediaFrameComponentType::Float : ETmvMediaFrameComponentType::Int;
	}

	/**
	 * Pre-defined pixel format correspondence matrices matching memory layouts for packed components.
	 */
	namespace PixelFormatMatrices
	{
		constexpr EPixelFormat Skip = PF_Unknown;		// no DXGI/Hardware format exists (for any 3 bytes per component or 8 bits float).
		constexpr EPixelFormat Missing = PF_Unknown;	// No PF defined in UE, but exists in DXGI.

		/**
		 * Matrix of float pixel formats.
		 * Row: Number of Components, Column: Number of Bytes Per Components
		 */
		static const EPixelFormat FloatPixelFormats[4][4] =
		{
			/* 1 component(s) */ {Skip, PF_R16F, Skip, PF_R32_FLOAT},
			/* 2 component(s) */ {Skip, PF_G16R16F, Skip, PF_G32R32F},	// Remark: GR instead of RG. @todo Will need shader swizzle.
			/* 3 component(s) */ {Skip, Skip, Skip, PF_R32G32B32F},		// No packed 24 or 48 bits format exits.
			/* 4 component(s) */ {Skip, PF_FloatRGBA, Skip, Missing}
		};

		/**
		 * Matrix of normalized integer pixel formats.
		 * Row: Number of Components, Column: Number of Bytes Per Components
		 */
		static const EPixelFormat NormPixelFormats[4][4] =
		{
			// Note: all 32 bits formats are missing. Not commonly used for video formats (yet).
			/* 1 component(s) */ {PF_G8, PF_G16, Skip, Missing},
			/* 2 component(s) */ {PF_R8G8, PF_G16R16, Skip, Missing},
			/* 3 component(s) */ {Skip, Skip, Skip, Missing},			// No packed 24 or 48 bits format exits.
			/* 4 component(s) */ {PF_R8G8B8A8, PF_R16G16B16A16_UNORM, Skip, Missing}
		};

		/**
		 * Matrix of integer pixel formats.
		 * Row: Number of Components, Column: Number of Bytes Per Components
		 * @remark Can only be used in UAV for compute shaders. (todo)
		 */
		static const EPixelFormat IntPixelFormats[4][4] =
		{
			/* 1 component(s) */ {PF_R8_UINT, PF_R16_UINT, Skip, PF_R32_UINT},
			/* 2 component(s) */ {PF_R8G8_UINT, PF_R16G16_UINT, Skip, PF_R32G32_UINT},
			/* 3 component(s) */ {Skip, Skip, Skip, PF_R32G32B32_UINT},	// No packed 24 or 48 bits format exits.
			/* 4 component(s) */ {PF_R8G8B8A8_UINT, PF_R16G16B16A16_UINT, Skip, PF_R32G32B32A32_UINT}
		};
		
		EPixelFormat LookupPixelFormat(ETmvMediaFrameComponentType Type, bool bInNormalized, int32 ComponentIndex, int32 ComponentSizeIndex)
		{
			if (Type == ETmvMediaFrameComponentType::Float)
			{
				return FloatPixelFormats[ComponentIndex][ComponentSizeIndex];
			}

			if (bInNormalized)
			{
				// For Pixel shaders, we can only use UNORM formats. Note that all 32 bits formats are missing.
				return NormPixelFormats[ComponentIndex][ComponentSizeIndex];
			}

			// For Compute shaders, we can use _UINT. This is the most complete set of formats.
			return IntPixelFormats[ComponentIndex][ComponentSizeIndex];
		}
	}

	EPixelFormat GetPackedPlanePixelFormat(const FTmvMediaFramePlaneInfo& InPlaneInfo, bool bInNormalized)
	{
		check(InPlaneInfo.ComponentLayout == ETmvMediaFrameComponentLayout::Packed);

		if (InPlaneInfo.NumComponents < 1 || InPlaneInfo.NumComponents > 4)
		{
			UE_LOGF(LogTmvMedia, Verbose, "GetPackedPlanePixelFormat: Invalid Number of Components: %d, should be in [1,4].", InPlaneInfo.NumComponents);
			return PF_Unknown;
		}

		const int32 BytesPerComponent = InPlaneInfo.GetBytesPerComponent();

		if (BytesPerComponent < 1 || BytesPerComponent > 4)
		{
			UE_LOGF(LogTmvMedia, Verbose, "GetPackedPlanePixelFormat: Invalid Number of bytes per components: %d, should be in [1,4].", BytesPerComponent);
			return PF_Unknown;
		}

		return PixelFormatMatrices::LookupPixelFormat(InPlaneInfo.Type, bInNormalized, InPlaneInfo.NumComponents - 1, BytesPerComponent - 1);
	}

	EPixelFormat GetInterleavedPlanePixelFormat(const FTmvMediaFramePlaneInfo& InPlaneInfo, bool bInNormalized)
	{
		check(InPlaneInfo.ComponentLayout == ETmvMediaFrameComponentLayout::Interleaved);

		const int32 BytesPerComponent = InPlaneInfo.GetBytesPerComponent();

		if (BytesPerComponent < 1 || BytesPerComponent > 4)
		{
			UE_LOGF(LogTmvMedia, Verbose, "GetInterleavedPlanePixelFormat: Invalid Number of bytes per components: %d, should be in [1,4].", BytesPerComponent);
			return PF_Unknown;
		}

		// If the plane is interleaved, the format is going to be that of a single component, but we will allocate more pixels.
		constexpr int32 ComponentIndex = 0;
		return PixelFormatMatrices::LookupPixelFormat(InPlaneInfo.Type, bInNormalized, ComponentIndex, BytesPerComponent - 1);
	}

	EPixelFormat GetPlanePixelFormat(const FTmvMediaFramePlaneInfo& InPlaneInfo, bool bInNormalized)
	{
		switch (InPlaneInfo.ComponentLayout)
		{
		case ETmvMediaFrameComponentLayout::Packed:
			return GetPackedPlanePixelFormat(InPlaneInfo, bInNormalized);
		case ETmvMediaFrameComponentLayout::Interleaved:
			return GetInterleavedPlanePixelFormat(InPlaneInfo, bInNormalized);
		default:
			return PF_Unknown;
		}
	}

	bool WriteMipBufferToFile(const FString& InFilepath, const FTmvMediaFrameMipBufferHandle& InMipBuffer)
	{
		if (!InMipBuffer.IsValid())
		{
			return false;
		}
		
		const FTmvMediaFrameMipInfo& MipInfo = InMipBuffer->GetMipInfoRef();
		FString FileFormat;
		FImageInfo ImageInfo;

		TFunction<bool(IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer)> WriteBufferFunction;

		if (MipInfo.ColorModel == ETmvMediaFrameColorModel::RGB)
		{
			// Preferred implementation using ImageView, which can convert any types to rgba32f.
			if (PopulateImageInfo(MipInfo, ImageInfo))
			{
				// Other option: FImageUtils::SaveImageAutoFormat for non float.
				FileFormat = TEXT("pfm");
				WriteBufferFunction = [&ImageInfo](IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer) -> bool
				{
					const FImageView ImageView(ImageInfo, InMipBuffer->GetPlaneBufferForComponent(0));
					return Pfm::WriteFrame(InArchive, ImageView);
				};
			}
			// Fallback to more restrictive implementation (in case of planar layout), only support float types for now.
			else if (Pfm::IsWriteFrameSupported(MipInfo))
			{
				FileFormat = TEXT("pfm");
				WriteBufferFunction = [&ImageInfo](IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer) -> bool
				{
					return Pfm::WriteFrame(InArchive, InMipBuffer);
				};
			}
		}
		else if (MipInfo.ColorModel == ETmvMediaFrameColorModel::YUV)
		{
			// todo: yuv packed if needed.
			if (Y4M::IsWriteFrameSupported(MipInfo))
			{
				FileFormat = TEXT("y4m");
				WriteBufferFunction = [](IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer) -> bool
				{
					if (Y4M::WriteHeader(InArchive, InMipBuffer->GetMipInfoRef()))
					{
						return Y4M::WriteFrame(InArchive, InMipBuffer);
					}
					return false;
				};
			}
		}

		if (!FileFormat.IsEmpty() && WriteBufferFunction.IsSet())
		{
			FString FullFileName = FString::Printf(TEXT("%s.%s"), *InFilepath, *FileFormat);

			TUniquePtr<IFileHandle> File(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*FullFileName));
			if (File)
			{
				return WriteBufferFunction(*File, InMipBuffer);
			}
		}
		return false;
	}
}

namespace UE::TmvMedia::FrameUtils::Y4M
{
	inline uint8 GetBitDepth(const FTmvMediaFrameMipInfo& InMipInfo)
	{
		return !InMipInfo.Planes.IsEmpty() ? InMipInfo.Planes[0].BitDepth : 0;
	}

	// Determines the original color format from the setup of the mip color planes.
	inline const TCHAR* GetColorFormat(const FTmvMediaFrameMipInfo& InMipInfo)
	{
		switch (InMipInfo.Planes.Num())
		{
		case 1:
			return TEXT("mono");
		case 3:
			// Deduce subsampling: 420, 422 0r 444
			if (InMipInfo.Planes[1].Height == InMipInfo.Height) // 422, 444
			{
				if (InMipInfo.Planes[1].Width == InMipInfo.Width)
				{
					return TEXT("444");
				}
				return TEXT("422");
			}
			return GetBitDepth(InMipInfo) == 8 ? TEXT("420mpeg2") : TEXT("420");
		case 4:
			return TEXT("444");	// 4444 is saved as 444. 
		default:
			UE_LOGF(LogTmvMedia, Error, "Unsupported color format by Y4m");
			return nullptr;
		}
	}

	const TCHAR* GetBitDepthFormat(const FTmvMediaFrameMipInfo& InMipInfo)
	{
		switch (GetBitDepth(InMipInfo))
		{
		case 8: return TEXT("");
		case 10: return InMipInfo.Planes.Num() == 1 ? TEXT("10") : TEXT("p10");
		case 12: return TEXT("p12");
		default:
			UE_LOGF(LogTmvMedia, Error, "Unsupported bit depth %d by Y4m.", GetBitDepth(InMipInfo));
			return nullptr;
		}
	}

	bool WriteHeader(IFileHandle& InArchive, const FTmvMediaFrameMipInfo& InMipInfo)
	{
		const TCHAR* ColorFormat = GetColorFormat(InMipInfo);
		const TCHAR* BitDepthFormat = GetBitDepthFormat(InMipInfo);

		if (!ColorFormat || !BitDepthFormat)
		{
			return false;
		}

		FString Y4mHeader = FString::Printf(TEXT("YUV4MPEG2 W%d H%d F%d:%d Ip C%s%s\n"),
			InMipInfo.Width, InMipInfo.Height, 30, 1, ColorFormat, BitDepthFormat);

		auto Y4mHeaderAnsi = StringCast<ANSICHAR>(*Y4mHeader, Y4mHeader.Len());
		InArchive.Write((const uint8*)Y4mHeaderAnsi.Get(), Y4mHeaderAnsi.Length() * sizeof(ANSICHAR));
		return true;
	}

	bool WriteFrameHeader(IFileHandle& InArchive)
	{
		const ANSICHAR* FrameHeader = "FRAME\n";
		InArchive.Write((const uint8*)FrameHeader, 6 * sizeof(ANSICHAR));
		return true;
	}

	bool WriteFrameBody(IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer)
	{
		const FTmvMediaFrameMipInfo& MipInfo = InMipBuffer->GetMipInfoRef();
		const int NumComponents = FMath::Min(MipInfo.NumComponents, 3);
		for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
		{
			const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(ComponentIndex);
			const int32 BytesPerElement = MipInfo.Planes[PlaneIndex].GetBytesPerComponent() * MipInfo.Planes[PlaneIndex].NumComponents;

			const uint8* PlaneBuffer = static_cast<const uint8*>(InMipBuffer->GetPlaneBufferForComponent(ComponentIndex));
			for (uint32 LineIndex=0; LineIndex < MipInfo.Planes[PlaneIndex].Height; ++LineIndex)
			{
				InArchive.Write(PlaneBuffer, MipInfo.Planes[PlaneIndex].Width * BytesPerElement);
				PlaneBuffer += MipInfo.Planes[PlaneIndex].Stride;
			}
		}
		return true;
	}

	bool WriteFrame(IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer)
	{
		if (!WriteFrameHeader(InArchive))
		{
			return false;
		}
		return WriteFrameBody(InArchive, InMipBuffer);
	}

	bool IsWriteFrameSupported(const FTmvMediaFrameMipInfo& InMipInfo)
	{
		// todo: yuv packed if needed.
		if (InMipInfo.Planes.Num() >= 3) // only support planar for now.
		{
			return true;
		}
		return false;
	}
}

// Reference: https://www.pauldebevec.com/Research/HDR/PFM/
namespace UE::TmvMedia::FrameUtils::Pfm
{
	/** Write PFM header for the given frame. */
	void WriteFrameHeader(IFileHandle& InArchive, int32 Width, int32 Height)
	{
#if PLATFORM_LITTLE_ENDIAN
		const TCHAR* EndiannessSignature = TEXT("-1.0");
#else
		const TCHAR* EndiannessSignature = TEXT("1.0");
#endif

		const FString PfmHeader = FString::Printf(TEXT("PF\n%d %d\n%s\n"), Width, Height, EndiannessSignature);
		const auto PfmHeaderAnsi = StringCast<ANSICHAR>(*PfmHeader, PfmHeader.Len());
		InArchive.Write(reinterpret_cast<const uint8*>(PfmHeaderAnsi.Get()), PfmHeaderAnsi.Length() * sizeof(ANSICHAR));
	}

	/** Write the frame body for the specific format of RGBA32F. */
	bool WriteFrameBodyRGBA32F(IFileHandle& InArchive, const FImageView& InImageView)
	{
		// Note: PFM row order is flipped vertically (bottom first).
		for (int32 PixelY = InImageView.SizeY-1; PixelY >= 0 ; --PixelY)
		{
			const float* PixelRow = static_cast<const float*>(InImageView.GetPixelPointer(0, PixelY));
			
			for (int32 PixelX = 0; PixelX < InImageView.SizeX; ++PixelX)
			{
				InArchive.Write(reinterpret_cast<const uint8*>(PixelRow), sizeof(float)*3);
				PixelRow += 4;
			}
		}
		return true;
	}

	bool WriteFrame(IFileHandle& InArchive, const FImageView& InImageView)
	{
		WriteFrameHeader(InArchive, InImageView.SizeX, InImageView.SizeY);

		if (InImageView.Format == ERawImageFormat::RGBA32F)
		{
			return WriteFrameBodyRGBA32F(InArchive, InImageView);
		}

		// If it is not the desired format, we can convert.
		FImage TmpImage;
		InImageView.CopyTo(TmpImage, ERawImageFormat::RGBA32F, InImageView.GammaSpace);
		return WriteFrameBodyRGBA32F(InArchive, TmpImage);
	}

	float ConvertToFloat(float InValue)
	{
		return InValue;
	}

	float ConvertToFloat(FFloat16 InValue)
	{
		return InValue.GetFloat();
	}

	// Note: this *should* support both packed and planar buffer layouts (but not interleaved).
	template<typename ComponentType>
	bool WriteFrameBody(IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer)
	{
		const FTmvMediaFrameMipInfo& MipInfo = InMipBuffer->GetMipInfoRef();

		constexpr int32 NumComponents = 3;

		// Prepare pointers for each component.
		const uint8* ComponentPtrs[NumComponents];

		for(int32 ComponentIndex = 0 ; ComponentIndex < NumComponents; ++ComponentIndex)
		{
			// Retrieve the plane buffer.
			const uint8* PlaneBuffer = static_cast<const uint8*>(InMipBuffer->GetPlaneBufferForComponent(ComponentIndex));

			// Calculate the "start" offset of the component in the plane buffer, in case multiple components are packed
			// in the same plane.
			int32 ComponentIndexInPlane = 0;
			int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(ComponentIndex, &ComponentIndexInPlane);
			ComponentPtrs[ComponentIndex] = PlaneBuffer + MipInfo.Planes[PlaneIndex].GetStartComponentOffsetInBytes(ComponentIndexInPlane); 
		}

		// Note: PFM row order is flipped vertically.
		for (int32 PixelY = MipInfo.Height-1; PixelY >= 0; --PixelY)
		{
			// Prepare row pointers for each component.
			const ComponentType* RowPtrs[NumComponents];
			for(int32 ComponentIndex = 0 ; ComponentIndex < NumComponents; ++ComponentIndex)
			{
				const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(ComponentIndex);
				RowPtrs[ComponentIndex] = reinterpret_cast<const ComponentType*>(ComponentPtrs[ComponentIndex] + MipInfo.Planes[PlaneIndex].Stride * PixelY);
			}

			// Write the row's RGB components from each component planes.
			for (int32 PixelX = 0; PixelX < MipInfo.Width; ++PixelX)
			{
				float RGB[NumComponents];
				for(int32 ComponentIndex = 0 ; ComponentIndex < NumComponents; ++ComponentIndex)
				{
					RGB[ComponentIndex] = ConvertToFloat(RowPtrs[ComponentIndex][PixelX]);
				}
				InArchive.Write((const uint8*)RGB, sizeof(float)*NumComponents);
			}
		}

		return true;
	}

	bool IsWriteFrameSupported(const FTmvMediaFrameMipInfo& MipInfo, FString* OutError)
	{
		if (MipInfo.NumComponents < 3)
		{
			if (OutError)
			{
				*OutError = TEXT("PFM requires at least 3 components (RGB).");
			}
			return false;
		}

		if (MipInfo.Planes.IsEmpty())
		{
			if (OutError)
			{
				*OutError = TEXT("No memory planes defined.");
			}
			return false;
		}

		uint8 BitDepth = MipInfo.Planes[0].BitDepth;
		
		for (const FTmvMediaFramePlaneInfo& PlaneInfo : MipInfo.Planes)
		{
			// Component interleaving is not supported (yet?).
			if (PlaneInfo.ComponentLayout == ETmvMediaFrameComponentLayout::Interleaved)
			{
				if (OutError)
				{
					*OutError = TEXT("Interleaved components are not currently supported.");
				}
				return false;
			}

			// Only support float type.
			if (PlaneInfo.Type != ETmvMediaFrameComponentType::Float)
			{
				if (OutError)
				{
					*OutError = TEXT("Non Float formats not supported by PFM.");
				}
				return false;
			}

			// Only support float 16 and 32 bits.
			if (PlaneInfo.BitDepth != 16 && PlaneInfo.BitDepth != 32)
			{
				if (OutError)
				{
					*OutError = TEXT("This implementation only supports 16 and 32 bit floats.");
				}
				return false;
			}

			// All planes must be the same bit depth.
			if (PlaneInfo.BitDepth != BitDepth)
			{
				if (OutError)
				{
					*OutError = TEXT("This implementation requires all planes to have the same bit depth.");
				}
				return false;
			}
		}
		return true;
	}

	bool WriteFrame(IFileHandle& InArchive, const FTmvMediaFrameMipBufferHandle& InMipBuffer, FString* OutError)
	{
		const FTmvMediaFrameMipInfo& MipInfo = InMipBuffer->GetMipInfoRef();
		uint8 BitDepth = MipInfo.Planes[0].BitDepth;

		if (!IsWriteFrameSupported(MipInfo, OutError))
		{
			return false;
		}

		WriteFrameHeader(InArchive, MipInfo.Width, MipInfo.Height);

		if (BitDepth == 16)
		{
			return WriteFrameBody<FFloat16>(InArchive, InMipBuffer);
		}

		if (BitDepth == 32)
		{
			return WriteFrameBody<float>(InArchive, InMipBuffer);
		}

		return false;
	}
}
