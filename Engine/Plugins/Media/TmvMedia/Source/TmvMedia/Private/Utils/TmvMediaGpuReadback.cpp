// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaGpuReadback.h"
#include "RHITransition.h"
#include "TmvMediaLog.h"

bool FTmvMediaGpuTextureReadback::SupportsAnyThreadReadback()
{
#if PLATFORM_MAC
	return false; // Not supported on Metal.
#else  // PLATFORM_MAC
	return true;
#endif // PLATFORM_MAC
}

FTmvMediaGpuTextureReadback::FTmvMediaGpuTextureReadback(FName InRequestName) 
: FTmvMediaGpuReadback(InRequestName)
{
}

void FTmvMediaGpuTextureReadback::EnqueueCopy(FRHICommandList& InRHICmdList, FRHITexture* InSourceTexture, const FTmvMediaTextureReadbackParams& InReadbackParams)
{
	Fence->Clear();
	LastCopyGPUMask = InRHICmdList.GetGPUMask();

	if (InSourceTexture)
	{
		check(!InSourceTexture->IsMultisampled());

		for (uint32 GPUIndex : LastCopyGPUMask)
		{
			SCOPED_GPU_MASK(InRHICmdList, FRHIGPUMask::FromIndex(GPUIndex));

			// Assume for now that every enqueue happens on a texture of the same format and size (when reused).
			bool bNeedTextureCreation = true;

			if (DestinationStagingTextures[GPUIndex])
			{
				if (GRHIGlobals.SupportLinearTextureVolumeFormat)
				{
					bNeedTextureCreation = DestinationStagingTextures[GPUIndex]->GetDesc().Dimension != InSourceTexture->GetDesc().Dimension;
				}
				else
				{
					// Some platform support only 2d texture for readback, so assuming that if we have a staging texture is always a 2d texture 
					bNeedTextureCreation = false;
				}
			}

			if (bNeedTextureCreation)
			{
				FIntVector StagingTextureSize;

				// Passing 0 or negative for size means read back the entire texture.
				if (InReadbackParams.Size.X > 0 && InReadbackParams.Size.Y > 0)
				{
					StagingTextureSize = InReadbackParams.Size;
				}
				else
				{
					StagingTextureSize = InSourceTexture->GetSizeXYZ();
				}

				FString FenceName = Fence->GetFName().ToString();

				FRHITextureCreateDesc Desc = FRHITextureCreateDesc(InSourceTexture->GetDesc(), ERHIAccess::CopyDest, *FenceName)
					.SetExtent(StagingTextureSize.X, StagingTextureSize.Y)
					.SetFlags(ETextureCreateFlags::CPUReadback | ETextureCreateFlags::HideInVisualizeTexture);

				if (GRHIGlobals.SupportLinearTextureVolumeFormat)
				{
					switch (Desc.Dimension)
					{
					case ETextureDimension::Texture2DArray:
					case ETextureDimension::TextureCubeArray:
						ensureMsgf(InReadbackParams.Size.Z <= 1, TEXT("Readback for texture arrays supports only one slice at a time. Texture Name: %s, SourcePosition: (%d, %d, %d), SourceSlice: %u, Size: (%d, %d, %d)."),
							*InSourceTexture->GetName().ToString(), InReadbackParams.SourcePosition.X, InReadbackParams.SourcePosition.Y, InReadbackParams.SourcePosition.Z, 
							InReadbackParams.SourceSlice, InReadbackParams.Size.X, InReadbackParams.Size.Y, InReadbackParams.Size.Z);
						Desc.SetArraySize(1);
						break;
					case ETextureDimension::Texture3D:
						Desc.SetDepth(StagingTextureSize.Z);
						break;
					default:
						break;
					}
				}
				else
				{
					ensureMsgf(InReadbackParams.Size.Z <= 1, TEXT("Readback for texture arrays/volume texture supports only one slice at a time. Texture Name: %s, SourcePosition: (%d, %d, %d), SourceSlice: %u, Size: (%d, %d, %d)."),
						*InSourceTexture->GetName().ToString(), InReadbackParams.SourcePosition.X, InReadbackParams.SourcePosition.Y, InReadbackParams.SourcePosition.Z, 
						InReadbackParams.SourceSlice, InReadbackParams.Size.X, InReadbackParams.Size.Y, InReadbackParams.Size.Z);

					// Some platforms only support 2d texture for readback, create a 2d texture staging texture and force the slice size to 1
					Desc.SetArraySize(1);
					Desc.SetDepth(1);
					Desc.SetDimension(ETextureDimension::Texture2D);
				}

				DestinationStagingTextures[GPUIndex] = InRHICmdList.CreateTexture(Desc);
			}
			else
			{
				InRHICmdList.Transition(FRHITransitionInfo(DestinationStagingTextures[GPUIndex], ERHIAccess::CPURead, ERHIAccess::CopyDest));
			}
			
			FRHICopyTextureInfo CopyInfo;

			// Make sure we're not passing negative coordinates or size.
			if (InReadbackParams.SourcePosition.X >= 0 && InReadbackParams.SourcePosition.Y >= 0)
			{
				CopyInfo.SourcePosition = InReadbackParams.SourcePosition;
			}

			if (InReadbackParams.Size.X > 0 && InReadbackParams.Size.Y > 0)
			{
				CopyInfo.Size = InReadbackParams.Size;
			}

			CopyInfo.SourceSliceIndex = InReadbackParams.SourceSlice;
			CopyInfo.DestSliceIndex = 0;
			CopyInfo.NumSlices = 1;
			
			CopyInfo.SourceMipIndex = InReadbackParams.SourceMip;
			CopyInfo.DestMipIndex = 0;
			CopyInfo.NumMips = 1;

			InRHICmdList.CopyTexture(InSourceTexture, DestinationStagingTextures[GPUIndex], CopyInfo);
			InRHICmdList.Transition(FRHITransitionInfo(DestinationStagingTextures[GPUIndex], ERHIAccess::CopyDest, ERHIAccess::CPURead));
			InRHICmdList.WriteGPUFence(Fence);
		}
	}
}

void* FTmvMediaGpuTextureReadback::Lock(int32& OutRowPitchInPixels, int32 *OutBufferHeight)
{
	uint32 GPUIndex = LastCopyGPUMask.GetFirstIndex();

	if (DestinationStagingTextures[GPUIndex])
	{
		LastLockGPUIndex = GPUIndex;

		void* ResultsBuffer = nullptr;
		int32 BufferWidth = 0, BufferHeight = 0;

		if (IsInRenderingThread())
		{
			FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
			RHICmdList.MapStagingSurface(DestinationStagingTextures[GPUIndex], Fence.GetReference(), ResultsBuffer, BufferWidth, BufferHeight, LastLockGPUIndex);
		}
		else if (GDynamicRHI && SupportsAnyThreadReadback())
		{
			// Remark: Not using the fence to lock the access. This implies the fence has been checked and is "ready". 
			GDynamicRHI->RHIMapStagingSurface(DestinationStagingTextures[GPUIndex], nullptr, ResultsBuffer, BufferWidth, BufferHeight, LastLockGPUIndex);
		}
		else
		{
			UE_LOGF(LogTmvMedia, Error, "Locking Readback Staging buffer outside the render thread is not supported on this system.");
			return nullptr;
		}

		if (OutBufferHeight)
		{
			*OutBufferHeight = BufferHeight;
		}

		OutRowPitchInPixels = BufferWidth;

		return ResultsBuffer;
	}

	OutRowPitchInPixels = 0;
	if (OutBufferHeight)
	{
		*OutBufferHeight = 0;
	}
	return nullptr;
}

void FTmvMediaGpuTextureReadback::Unlock()
{
	ensure(DestinationStagingTextures[LastLockGPUIndex]);

	if (IsInRenderingThread())
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		RHICmdList.UnmapStagingSurface(DestinationStagingTextures[LastLockGPUIndex], LastLockGPUIndex);
	}
	else if (GDynamicRHI && SupportsAnyThreadReadback())
	{
		GDynamicRHI->RHIUnmapStagingSurface(DestinationStagingTextures[LastLockGPUIndex], LastLockGPUIndex);
	}
	else
	{
		UE_LOGF(LogTmvMedia, Fatal, "Unlocking Readback Staging buffer outside the render thread is not supported on this system.");
	}
}

uint64 FTmvMediaGpuTextureReadback::GetGPUSizeBytes() const
{
	uint64 TotalSize = 0;
	for (uint32 TextureIndex = 0; TextureIndex < UE_ARRAY_COUNT(DestinationStagingTextures); TextureIndex++)
	{
		if (DestinationStagingTextures[TextureIndex].IsValid())
		{
			TotalSize += DestinationStagingTextures[TextureIndex]->GetDesc().CalcMemorySizeEstimate();
		}
	}
	return TotalSize;
}
