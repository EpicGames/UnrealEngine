// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
  TmvMediaGpuReadback.h: classes for managing fences and staging buffers for
  asynchronous GPU memory updates and readbacks with minimal stalls and no
  RHI thread flushes.
  
  Tmv Specific:
  - Supports mip index in the request.
  - Using this api to support all texture formats (int16, float16, etc).
  - Directly supports "any thread" locking/unlocking.
=============================================================================*/

#pragma once

#include "DynamicRHI.h"
#include "MultiGPU.h"
#include "RHI.h"
#include "RHICommandList.h"

/** Parameters for the readback copy request. */
struct FTmvMediaTextureReadbackParams
{
	/** Number of texels to copy. By default, it will copy the whole resource if no size is specified. */
	FIntVector Size = FIntVector::ZeroValue;

	/** Position to read from in the source texture. */
	FIntVector SourcePosition = FIntVector::ZeroValue;

	/** Indicate which slice to read from. */
	uint32 SourceSlice = 0;

	/** Indicate which mip to read from. */
	uint32 SourceMip = 0;
};

/**
 * Represents a readback request scheduled with CopyToStagingBuffer
 * Wraps a staging buffer with a FRHIGPUFence for synchronization.
 * 
 * Copied from FRHIGPUMemoryReadback. Difference is, mip index is supported.
 */
class FTmvMediaGpuReadback
{
public:
	FTmvMediaGpuReadback(FName InRequestName)
	{
		Fence = RHICreateGPUFence(InRequestName);
		LastLockGPUIndex = 0;
	}

	virtual ~FTmvMediaGpuReadback() = default;

	/** Indicates if the data is in place and ready to be read. */
	bool IsReady()
	{
		return !Fence || (Fence->NumPendingWriteCommands.GetValue() == 0 && Fence->Poll());
	}

	/** Indicates if the data is in place and ready to be read on a subset of GPUs. */
	bool IsReady(FRHIGPUMask InGPUMask)
	{
		return !Fence || Fence->Poll(InGPUMask);
	}

	/** Reset the ready state */
	void ResetReady()
	{
		if (Fence)
		{
			Fence->Clear();
		}
	}

	/** Block until the readback is ready. */
	void Wait(FRHICommandListImmediate& InRHICmdList, FRHIGPUMask InGPUMask) const
	{
		if (Fence)
		{
			Fence->Wait(InRHICmdList, InGPUMask);
		}
	}

	/**
	 * Copy the current state of the resource to the readback data.
	 * @param InRHICmdList The command list to enqueue the copy request on.
	 * @param InSourceTexture The texture holding the source data.
	 * @param InReadbackParams Description of all source parameters (size, position, mip, slice) to readback from.
	 */
	virtual void EnqueueCopy(FRHICommandList& InRHICmdList, FRHITexture* InSourceTexture, const FTmvMediaTextureReadbackParams& InReadbackParams) = 0;

	const FRHIGPUMask& GetLastCopyGPUMask() const { return LastCopyGPUMask; }

	FName GetName() const { return Fence ? Fence->GetFName() : NAME_None; }

protected:

	FGPUFenceRHIRef Fence;
	FRHIGPUMask LastCopyGPUMask;

	// We need to separately track which GPU buffer was last locked.  It's possible for a new copy operation to
	// be enqueued (writing to LastCopyGPUMask) while the buffer is technically locked, with the unlock and enqueued
	// copy on the GPU itself happening later, during pass execution in FRDGBuilder::Execute (for example, this
	// happens with Nanite streaming).  It's not unsafe, because the operations are occurring in order on both the
	// render thread and later pass Execute, but our locking logic needs to handle that scenario.
	uint32 LastLockGPUIndex;
};

/** Texture readback implementation. */
class FTmvMediaGpuTextureReadback final : public FTmvMediaGpuReadback
{
public:
	FTmvMediaGpuTextureReadback(FName InRequestName);

	using FTmvMediaGpuReadback::EnqueueCopy;

	//~ Begin FTmvMediaGPUReadback
	virtual void EnqueueCopy(FRHICommandList& InRHICmdList, FRHITexture* InSourceTexture, const FTmvMediaTextureReadbackParams& InReadbackParams) override;
	//~ End FTmvMediaGPUReadback

	/** 
	 * Some RHI implementation support locking/unlocking the staging buffers outside the render thread.
	 * If it is the case, this function will return true.
	 */
	static bool SupportsAnyThreadReadback();

	/** Locks the staging buffer and returns a memory pointer. Can be called on any thread if supported. */
	void* Lock(int32& OutRowPitchInPixels, int32* OutBufferHeight = nullptr);

	/** Releases the staging buffer. Can be called on any thread if supported. */
	void Unlock();

	/** Returns the total size in bytes of the memory buffer. */
	uint64 GetGPUSizeBytes() const;

	/** Array of staging textures per gpu. */
	FTextureRHIRef DestinationStagingTextures[MAX_NUM_GPUS];
};
