// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/TmvMediaMessageContext.h"


FText FTmvMediaMessageContext::ToText() const
{
	if (Messages.IsEmpty())
	{
		return FText::GetEmpty();
	}

	return FText::Join(FText::FromString("\n"), Messages);
}
