// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/TmvMediaPathUtils.h"

#include "MediaSource.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Templates/Function.h"

namespace UE::TmvMedia::PathUtils
{
	namespace Private
	{
		/** Definition of a path "token". */
		struct FPathToken
		{
			bool IsValid() const
			{
				return !Token.IsEmpty() && GetPathFunction.IsSet();
			}
			
			/** Token string. Ex: "{project_dir}". **/
			FString Token;

			/** Get the string the token resolve as. */
			TFunction<FString()> GetPathFunction;
		};

		/** PathToken object used to return invalid token in path token api below. */
		static FPathToken InvalidPathToken;

		/**
		 * List of currently supported path tokens.
		 * For now, supporting the same tokens as ImgMediaSource.
		 * See UImgMediaSource::ExpandSequencePathTokens.
		 */
		static const TArray<FPathToken> PathTokens =
		{
			{ TEXT("{project_dir}"), [](){ return FPaths::ProjectDir();} },
			{ TEXT("{engine_dir}"),[](){ return FPaths::EngineDir();} }
		};

		/**
		 * Replaces existing tokens for their full paths.
		 * Ex: "{project_dir}/Movies" -> "C:/MyProject/Movies"
		 * 
		 * @remark Because this expands the tokens as "full paths", we are only supporting the paths
		 * starting with the tokens and not in arbitrary positions.
		 */
		FString ExpandPathTokensToFull(const FString& InPath)
		{
			for (const FPathToken& PathToken : PathTokens)
			{
				if (InPath.StartsWith(PathToken.Token) && PathToken.GetPathFunction.IsSet())
				{
					const FString BaseDirectory = FPaths::ConvertRelativePathToFull(PathToken.GetPathFunction());
					FString ExpandedPath = FPaths::Combine(BaseDirectory, InPath.RightChop(PathToken.Token.Len()));
					FPaths::NormalizeDirectoryName(ExpandedPath); 
					FPaths::RemoveDuplicateSlashes(ExpandedPath); // Removes potential double slashes because of Combine.
					return ExpandedPath;
				}
			}
			return InPath;
		}

		/**
		 * Finds the token this path string begins with.
		 * @returns found token or invalid token if not found. Validate token with IsValid() before use.
		 */
		const FPathToken& FindFirstPathToken(const FString& InPath)
		{
			for (const FPathToken& PathToken : PathTokens)
			{
				if (InPath.StartsWith(PathToken.Token))
				{
					return PathToken;
				}
			}
			return InvalidPathToken;
		}

		bool PathExists(const FString& InPath)
		{
			return FPaths::DirectoryExists(InPath) || FPaths::FileExists(InPath);
		}

		static TArray<TFunction<FString()>> BasePathFunctions =
		{
			[](){ return FPaths::ProjectContentDir();},
			[](){ return FPaths::ProjectDir();},
		};
	}

	FString EnsureStartWithDotSlash(const FString& InPath)
	{
		if (!InPath.StartsWith(TEXT("./")))
		{
			return FPaths::Combine(TEXT("."), InPath);
		}
		return InPath;
	}

	FString GetDirectoryFullPath(const FString& InSanitizedPath)
	{
		FString PathExpanded = Private::ExpandPathTokensToFull(InSanitizedPath);

		// Relative paths are assumed to be under project directory, only support sanitized paths.
		if (PathExpanded.StartsWith(TEXT("./")) || FPaths::IsRelative(PathExpanded))
		{
			const FString ProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
			PathExpanded = FPaths::ConvertRelativePathToFull(ProjectDir, PathExpanded);
		}
		
		// Check if it is an existing directory.
		if (FPaths::DirectoryExists(PathExpanded))
		{
			return PathExpanded;
		}

		// If we reach here, we assume this is a filepath.
		return FPaths::GetPath(PathExpanded);
	}

	FString GetSanitizedPath(const FString& InPath)
	{
		// Do nothing with empty strings.
		if (InPath.IsEmpty())
		{
			return InPath;
		}

		FString SanitizedPath = InPath.TrimStartAndEnd().Replace(TEXT("\""), TEXT(""));
		FPaths::NormalizeDirectoryName(SanitizedPath);

		// If already a sanitized path, no need to process further.
		// This will work even if the path doesn't exist.
		if (SanitizedPath.StartsWith(TEXT("./")))
		{
			return SanitizedPath;
		}

		// Path strings using tokens also don't need further processing.
		const Private::FPathToken& FirstToken = Private::FindFirstPathToken(SanitizedPath);
		if (FirstToken.IsValid())
		{
			return SanitizedPath;
		}

		// Check if the path is under the current project
		const FString ProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

		// Test for existing relative path that might be under the project directory.
		if (FPaths::IsRelative(SanitizedPath))
		{
			const FString ContentDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
			TArray<const FString*, TInlineAllocator<2>> BasePaths = { &ProjectDir, &ContentDir };

			for (const FString* BasePath : BasePaths)
			{
				// This can return an absolute path that is not necessarily under the requested base path. Need to verify.
				FString ConvertedPath = FPaths::ConvertRelativePathToFull(*BasePath, SanitizedPath);
				
				if (FPaths::IsUnderDirectory(ConvertedPath, *BasePath) && Private::PathExists(ConvertedPath))
				{
					// Make the path relative to project dir.
					if (FPaths::MakePathRelativeTo(ConvertedPath, *ProjectDir))
					{
						return EnsureStartWithDotSlash(ConvertedPath);
					}
				}
			}
		}

		// If the path is either absolute or not relative to project dir.
		// We will attempt to make it absolute anyway (it may end up being under current process base dir if all else fails).
		FString ConvertedPath = FPaths::ConvertRelativePathToFull(SanitizedPath);

		// If the path is under the project directory, use relative path.
		if (FPaths::IsUnderDirectory(ConvertedPath, ProjectDir))
		{
			// If so, make it relative to that.
			if (FPaths::MakePathRelativeTo(ConvertedPath, *ProjectDir))
			{
				ConvertedPath = EnsureStartWithDotSlash(ConvertedPath);
			}
		}

		return ConvertedPath;
	}
	
	FString ConvertSanitizedPathToFull(const FString& InSanitizedPath)
	{
		if (InSanitizedPath.IsEmpty())
		{
			return FString();
		}
	
		// Expand the common known path tokens.
		FString PathExpanded = Private::ExpandPathTokensToFull(InSanitizedPath);

		// If already a sanitized path, we consider it under project without further processing.
		// This will work even if the path doesn't exist (for output paths for instance).
		if (InSanitizedPath.StartsWith(TEXT("./")))
		{
			const FString BasePathFull = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
			return FPaths::ConvertRelativePathToFull(BasePathFull, PathExpanded);
		}
		
		// If the path is relative, i.e. no tokens nor sanitized relative path.
		if (FPaths::IsRelative(PathExpanded))
		{
			// Attempt to resolve against possible base paths. But this will only work if the resolved path exists.
			for (const TFunction<FString()>& BasePathFunction : Private::BasePathFunctions)
			{
				const FString BasePathFull = FPaths::ConvertRelativePathToFull(BasePathFunction());
				const FString PossiblePath = FPaths::ConvertRelativePathToFull(BasePathFull, PathExpanded);
				if (FPaths::DirectoryExists(PossiblePath) || FPaths::FileExists(PossiblePath))
				{
					return PossiblePath;
				}
			}
		}

		// Final fallback: try to make it absolute (in case it was still relative to process base dir).
		return FPaths::ConvertRelativePathToFull(PathExpanded);
	}

	FString MakeFrameFilename(const FString& InBaseName, int32 InFrameIndex, int32 InZeroPadFrameNumbers, const FString& InExtension)
	{
		static const FString FrameNumberToken = TEXT("{frame_number}");
		
		// Specify an upper bound (somewhere around typical max filename length for Os) for protection against DOS or OOM.
		constexpr int32 ZeroPadFrameNumbersUpperBound = 255;
		const int32 ZeroPadFrameNumber = FMath::Min(InZeroPadFrameNumbers, ZeroPadFrameNumbersUpperBound);
		FString FormattedName = InBaseName;
		FString FrameNumber = FString::Printf(TEXT("%0*d"), ZeroPadFrameNumber, InFrameIndex);

		if (FormattedName.Contains(FrameNumberToken))
		{
			FormattedName.ReplaceInline(*FrameNumberToken, *FrameNumber);
		}
		else
		{
			FormattedName += FrameNumber;
		}

		// Append the extension
		if (!InExtension.IsEmpty())
		{
			if (InExtension.StartsWith(TEXT(".")))
			{
				if (FormattedName.EndsWith(TEXT(".")))
				{
					FormattedName.LeftChopInline(1);
				}
			}
			else if (!FormattedName.EndsWith(TEXT(".")))
			{
				FormattedName += TEXT(".");	
			}
			FormattedName += InExtension;
		}
		
		return FPaths::MakeValidFileName(FormattedName);
	}
	
	FString GetMediaSourceMediaFullPath(const UMediaSource* InMediaSource)
	{
		if (InMediaSource)
		{
			// For ImgMediaSource and FileMediaSource, the url will already return the full path.
			const FString Url = InMediaSource->GetUrl();

			// Clean up to get just the path.
			const FString FilePrefix(TEXT("file://"));
			const FString ImgPrefix(TEXT("img://"));

			if (Url.StartsWith(FilePrefix))
			{
				return ConvertSanitizedPathToFull(Url.RightChop(FilePrefix.Len()));
			}
		
			if (Url.StartsWith(ImgPrefix))
			{
				return ConvertSanitizedPathToFull(Url.RightChop(ImgPrefix.Len()));
			}
		}
		return FString();
	}
}