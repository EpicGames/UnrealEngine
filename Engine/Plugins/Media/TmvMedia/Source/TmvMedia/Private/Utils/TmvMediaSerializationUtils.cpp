// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/TmvMediaSerializationUtils.h"

#include "JsonObjectConverter.h"
#include "StructUtils/InstancedStruct.h"
#include "TmvMediaLog.h"

namespace UE::TmvMedia::SerializationUtils
{
	/** Utility function to log the json type. */
	const TCHAR* GetJsonTypeName(EJson InType)
	{
		switch (InType)
		{
		case EJson::None: return TEXT("None");
		case EJson::Null: return TEXT("Null");
		case EJson::String: return TEXT("String");
		case EJson::Number: return TEXT("Number");
		case EJson::Boolean: return TEXT("Boolean");
		case EJson::Array: return TEXT("Array");
		case EJson::Object: return TEXT("Object");
		default: return TEXT("Invalid");
		}
	}
	
	/**
	 * Serialization callback to support InstancedStruct.
	 * @param InProperty Property descriptor
	 * @param InValue Property value
	 * @return serialized json value.
	 */
	static TSharedPtr<FJsonValue> ExportPropertyCallback(FProperty* InProperty, const void* InValue)
	{
		TSharedPtr<FJsonValue> Result = nullptr;

		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			if (StructProperty->Struct == FInstancedStruct::StaticStruct())
			{
				const FInstancedStruct& InstancedStruct = *static_cast<const FInstancedStruct*>(InValue);
				if (const TSharedPtr<FJsonObject> LoadedStruct = SerializeToJson(InstancedStruct.GetScriptStruct(), InstancedStruct.GetMemory()))
				{
					const TSharedPtr<FJsonObject> InstancedStructObj = MakeShared<FJsonObject>();
					const FSoftObjectPath StructPath(InstancedStruct.GetScriptStruct());
					InstancedStructObj->SetField(StructPath.ToString(), MakeShared<FJsonValueObject>(LoadedStruct));
					
					Result = MakeShared<FJsonValueObject>(InstancedStructObj);
				}
			}
		}
		return Result;
	}
	
	/**
	 * Deserialization callback to handle the instanced struct. We want to support it and instance it properly.
	 * @remark The deserialization error will get lost, so we log instead.
	 * 
	 * @param InJsonValue Json value to deserialize
	 * @param InProperty Property
	 * @param InValue Pointer to Property value
	 * @return Returning false means the callback doesn't handle the value, so the fallback code does.
	 * However, if this is our case we handle, even if there is an error, and return true to indicate it is handled.
	 */
	bool ImportPropertyCallback(const TSharedPtr<FJsonValue>& InJsonValue, FProperty* InProperty, void* InValue)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			if (StructProperty->Struct == FInstancedStruct::StaticStruct())
			{
				constexpr bool bHandledButError = true;	// There is no way to indicate an error or return the error message.
				
				FInstancedStruct& InstancedStruct = *static_cast<FInstancedStruct*>(InValue);

				if (InJsonValue->Type !=  EJson::Object)
				{
					UE_LOGF(LogTmvMedia, Error, "Json Parsing FInstancedStruct Property: Json value should be of type Object, is of type: %ls", GetJsonTypeName(InJsonValue->Type));
					return bHandledButError;
				}
				
				// json value should be an object with 1 field
				const TSharedPtr<FJsonObject>* InstancedStructObject;
				if (InJsonValue->TryGetObject(InstancedStructObject))
				{
					if ((*InstancedStructObject)->Values.IsEmpty())
					{
						UE_LOGF(LogTmvMedia, Error, "Json Parsing FInstancedStruct Property: Json Object should have at least 1 field (has %d)", (*InstancedStructObject)->Values.Num());
						return bHandledButError;
					}

					if ((*InstancedStructObject)->Values.Num() > 1)
					{
						UE_LOGF(LogTmvMedia, Warning, "Json Parsing FInstancedStruct Property: Json Object should have only 1 field (has %d)", (*InstancedStructObject)->Values.Num());
					}
						
					for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Value : (*InstancedStructObject)->Values)
					{
						if (!Value.Value.IsValid())
						{
							UE_LOGF(LogTmvMedia, Error, "Json Parsing FInstancedStruct Property: Json value is invalid");
							continue;
						}

						FSoftObjectPath ScriptPath(FString(Value.Key));
						if (const UScriptStruct* Struct = Cast<UScriptStruct>(ScriptPath.ResolveObject()))	// so we got a valid struct
						{
							if (Value.Value->Type !=  EJson::Object)
							{
								UE_LOGF(LogTmvMedia, Error, "Json Parsing FInstancedStruct Property: Json value should be of type Object, is of type: %ls", GetJsonTypeName(Value.Value->Type));
							}
							
							const TSharedPtr<FJsonObject>* StructObject;
							if (Value.Value->TryGetObject(StructObject))
							{
								InstancedStruct.InitializeAs(Struct);
								FText DeserializationError;
								if (DeserializeFromJson((*StructObject).ToSharedRef(), Struct, InstancedStruct.GetMutableMemory(), DeserializationError))
								{
									constexpr bool bHandledSuccess = true;
									return bHandledSuccess;
								}
								
								UE_LOGF(LogTmvMedia, Error, "Json Parsing FInstancedStruct Property: Failed to deserialize: \"%ls\"", *DeserializationError.ToString());
							}
						}
						else
						{
							UE_LOGF(LogTmvMedia, Error, "Json Parsing FInstancedStruct Property: Unknown Script Type: \"%ls\"", *Value.Key);
						}
					}
				}
				return bHandledButError;
			}
		}
		
		return false;
	}


	TSharedPtr<FJsonObject> SerializeToJson(const UStruct* InStruct, const void* InObject)
	{
		if (!InStruct || !InObject)
		{
			return nullptr;
		}
		
		FJsonObjectConverter::CustomExportCallback CustomCB;
		CustomCB.BindStatic(&ExportPropertyCallback);

		const TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		constexpr int64 CheckFlags = 0;
		constexpr int64 SkipFlags = 0;
		const bool bLoaded = FJsonObjectConverter::UStructToJsonObject(InStruct, InObject, JsonObject.ToSharedRef(), CheckFlags, SkipFlags, &CustomCB, EJsonObjectConversionFlags::SkipStandardizeCase);
		return bLoaded ? JsonObject : nullptr;
	}
	
	bool DeserializeFromJson(const TSharedRef<FJsonObject>& InJsonObject, const UStruct* InStruct, void* InNativeObject, FText& OutErrorMessage)
	{
		FJsonObjectConverter::CustomImportCallback CustomCB;
		CustomCB.BindStatic(&ImportPropertyCallback);

		constexpr int64 CheckFlags = 0;
		constexpr int64 SkipFlags = 0;
		constexpr bool bStrictMode = false;
		return FJsonObjectConverter::JsonObjectToUStruct(InJsonObject, InStruct, InNativeObject, CheckFlags, SkipFlags, bStrictMode, &OutErrorMessage, &CustomCB);
	}
	
	FString BytesToString(const TArray<uint8>& InValueAsBytes)
	{
		// Reinterpret as TCHAR array. FJsonStructSerializerBackend uses UCS2CHAR which is compatible with TCHAR.
		const TCHAR* ValueAsChars = reinterpret_cast<const TCHAR*>(InValueAsBytes.GetData());
		int32 SizeInChar = InValueAsBytes.Num() / sizeof(TCHAR);
	
		// Exclude terminating character, if any.
		// Remark: From observation, the arrays from FJsonStructSerializerBackend don't seem to have a terminating character, but we check anyway.
		if (SizeInChar > 0 && ValueAsChars[SizeInChar-1] == 0)
		{
			--SizeInChar; // Exclude terminating character.
		}
	
		FString OutValueAsString;
		OutValueAsString.Reset(SizeInChar);
		OutValueAsString.AppendChars(ValueAsChars, SizeInChar);
		return OutValueAsString;
	}
}