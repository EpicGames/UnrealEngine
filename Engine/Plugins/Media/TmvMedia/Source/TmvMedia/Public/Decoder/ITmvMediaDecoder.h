// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/IntPoint.h"
#include "SampleConverter/TmvMediaFrameMipBufferFwd.h"
#include "TmvMediaFrameInfo.h"

#define UE_API TMVMEDIA_API

/**
 * Decoder return code.
 */
enum class ETmvMediaDecoderResult
{
	Success,
	Fail,
	Cancelled,
	Skipped
};

/**
 * Interface for the decoder's access unit.
 * This is used by the decoder to read data from the input buffer.
 * It is abstracted as an input stream so that the access unit doesn't have to be fully loaded in memory.
 * With this, we should be able to encapsulate access units from different parser implementations (containers or separate files).
 */
class ITmvMediaDecoderAccessUnit
{
public:
	virtual ~ITmvMediaDecoderAccessUnit() = default;

	/**
	 * Returns the offset in bytes from the start of the access unit. 
	 */
	virtual int64 Tell() const = 0;

	/**
	 * Returns the total size in bytes of the access unit.
	 */
	virtual int64 GetTotalSize() const = 0;
	
	/**
	 * Seek at the specified offset in bytes from the start of the access unit.
	 * @param InOffset Offset in bytes.
	 * @return true if the operation completed successfully, false otherwise.
	 */
	virtual bool Seek(int64 InOffset) = 0;

	/**
	 * Read from the current position the specified amount of data.
	 * 
	 * @param OutBuffer Destination buffer
	 * @param InSize Size in bytes that will be read.
	 * @return Size read.
	 */
	virtual int64 Read(void *OutBuffer, int64 InSize) = 0;

	/**
	 * Returns the frame id for this access unit.
	 */
	virtual int32 GetFrameId() const = 0;

	/**
	 * Returns the filename. (TMP)
	 */
	virtual const FString& GetFilename() const = 0;

	/**
	 * Directly access the underlying archive. (TMP)
	 * This is expected to be an archive for only the access unit (not the whole container).
	 */
	virtual FArchive* GetUnderlyingArchive() const = 0;
};

/**
 * Mip decoding request.
 */
struct FTmvMediaDecoderMipRequest
{
	/** Mip Info */
	FTmvMediaFrameMipInfo MipInfo;

	/** Specify the tile regions to load. */
	TArray<FIntRect> TileRegions;

	/** Destination mip buffer. */
	FTmvMediaFrameMipBufferHandle MipBuffer;

	/** (Set by decoder) Result of the decoding operation. */
	ETmvMediaDecoderResult OutResult = ETmvMediaDecoderResult::Fail;

	/** (Set by decoder) Number of tiles decoded. */
	int32 OutNumTilesDecoded = 0;
};

// @todo: Error handling: provide a callback for errors.

/**
 * TMV access unit parser interface.
 */
class ITmvMediaParser
{
public:
	virtual ~ITmvMediaParser() {}

	/**
	 * Parse the access unit for the mip infos.
	 * @param InAccessUnit Access unit for the current frame's encoded stream.
	 * @param OutMipInfos Populated array of mip information.
	 * @return result of the operation.
	 */
	virtual ETmvMediaDecoderResult ParseMipInfos(ITmvMediaDecoderAccessUnit& InAccessUnit, TArray<FTmvMediaFrameMipInfo>& OutMipInfos) = 0;

	// @todo: Add api to parse additional attributes. Ex for Exr: Compression.
	// ETmvMediaDecoderResult ParseAttributes(ITmvMediaDecoderAccessUnit& InAccessUnit, TMap<FName, FVariant>& OutAttributes) = 0;
};

/**
 * TMV (Tile-Based) Decoder interface.
 * 
 * Reads data from the input access unit and decode the requested mips and tiles. 
 */
class ITmvMediaDecoder
{
public:
	virtual ~ITmvMediaDecoder() {}

	/**
	 * Decode the given mip requests.
	 * @param InAccessUnit Access unit for the current frame's encoded stream.
	 * @param InMipRequests Array of mip decoding requests.
	 * @return Result code depending on the success of the operation.
	 */
	virtual ETmvMediaDecoderResult Decode(ITmvMediaDecoderAccessUnit& InAccessUnit, TArrayView<FTmvMediaDecoderMipRequest> InMipRequests) = 0;
};

#undef UE_API