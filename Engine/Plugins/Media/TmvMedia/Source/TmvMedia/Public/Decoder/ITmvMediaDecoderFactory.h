// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/Variant.h"
#include "Templates/SharedPointer.h"

#define UE_API TMVMEDIA_API

class ITmvMediaDecoder;
class ITmvMediaParser;

/**
 * Interface for a ITmvMediaDecoder factory.
 */
class ITmvMediaDecoderFactory
{
public:
	virtual ~ITmvMediaDecoderFactory() = default;

	/**
	 * Returns the name of this decoder implementation.
	 */
	virtual const FString& GetName() const = 0;

	/**
	 * Return the list of file extensions this factory supports.
	 */
	virtual TArray<FString> GetSupportedFileExtensions() const = 0;
	
	/**
	* Queries support status of the given codec.
	* @param InCodecFormat FOURCC of the codec, ex: "aPv1" 
	* @param InOptions Additional options may be provided to specify the format more closely, which may help selecting the best suited implementation.
	* @return 0 if not supported. If supported, the return value indicates a priority value for the implementation.
	*/
	virtual int32 SupportsFormat(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions) const = 0;

	/**
	 * Populates the provided map with parser configuration options.
	 * @param OutOptions Populated options.
	 */
	virtual void GetParserOptions(TMap<FString, FVariant>& OutOptions) const  = 0;

	/**
	 * Creates a parser for the given access unit format.
	 * @param InCodecFormat Either FOURCC of codec or filename extension of the container.
	 * @param InOptions Additional options may be provided to specify the format more closely, which may help selecting the best suited implementation.
	 * @return created parser or null if failed.
	 */
	virtual TSharedPtr<ITmvMediaParser, ESPMode::ThreadSafe> CreateParser(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions) = 0;

	/**
	 * Populates the provided map with decoder configuration options.
	 * @param OutOptions Populated options. 
	 */
	virtual void GetDecoderOptions(TMap<FString, FVariant>& OutOptions) const  = 0;
	
	/**
	* Creates a decoder for the given format and options.
	* @param InCodecFormat Either FOURCC of the codec, ex: "aPv1", or file extension (if each AU is a separate file). 
	* @param InOptions Additional options may be provided to specify the format more closely, which may help selecting the best suited implementation.
	* @return created decoder, or null if failed.
	*/
	virtual TSharedPtr<ITmvMediaDecoder, ESPMode::ThreadSafe> CreateDecoder(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions) = 0;
};

#undef UE_API