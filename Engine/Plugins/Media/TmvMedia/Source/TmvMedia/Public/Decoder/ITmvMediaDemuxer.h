// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"
#include "TmvMediaTrackInfo.h"


/**
 * Information about a single track in a demuxed container.
 * Extends the common track descriptor with demuxer-specific fields.
 */
struct FTmvMediaDemuxerTrackInfo : public FTmvMediaTrackInfo
{
	/** Track index (0-based). */
	int32 TrackIndex = -1;

	/** Total duration of the track in its timescale units. */
	int64 Duration = 0;

	/** Total number of samples in this track. */
	uint32 NumSamples = 0;

	/** Language code (3-letter ISO 639-2T). */
	FString LanguageCode;
};

/**
 * A single sample read from a demuxed container.
 * Extends the common sample metadata with demuxer-specific fields.
 */
struct FTmvMediaDemuxerSample : public FTmvMediaTrackSampleInfo
{
	/** Raw coded sample data. */
	TArray64<uint8> Data;

	/** File byte offset of this sample. */
	int64 FileOffset = 0;

	/** Size in bytes (matches Data.Num() when data is populated). */
	int64 SampleSize = 0;
};

/**
 * Abstract interface for demuxing media samples from a container.
 *
 * Workflow:
 *   1. Open() the container (from file path or from memory buffer).
 *   2. GetTrackCount() / GetTrackInfo() to enumerate available tracks.
 *   3. ReadSample() to iterate through samples on a given track.
 *   4. Seek() to jump to a particular time or keyframe.
 *   5. Close() when done.
 *
 * The demuxer extracts raw coded samples (access units) from the container
 * that can then be fed into ITmvMediaDecoder for decoding.
 *
 * Thread safety: implementations are NOT internally synchronized. Callers must
 * serialize all calls to the same demuxer instance (e.g. with a mutex).
 */
class ITmvMediaDemuxer
{
public:
	virtual ~ITmvMediaDemuxer() = default;

	/**
	 * Open a container from a file path.
	 * Parses the container structure and prepares track metadata.
	 * @param InFilePath Absolute path to the container file.
	 * @return Success if the container was parsed successfully.
	 */
	virtual ETmvMediaContainerResult OpenFile(const FString& InFilePath) = 0;

	/**
	 * Open a container from an in-memory buffer.
	 * @param InData Buffer containing the complete container data.
	 * @return Success if the container was parsed successfully.
	 */
	virtual ETmvMediaContainerResult OpenBuffer(TConstArrayView<uint8> InData) = 0;

	/**
	 * Returns the number of tracks found in the container.
	 */
	virtual int32 GetTrackCount() const = 0;

	/**
	 * Retrieve metadata for a specific track.
	 * @param InTrackIndex Track index (0-based, must be < GetTrackCount()).
	 * @param OutTrackInfo Populated track information.
	 * @return Success if the track info was retrieved.
	 */
	virtual ETmvMediaContainerResult GetTrackInfo(int32 InTrackIndex, FTmvMediaDemuxerTrackInfo& OutTrackInfo) const = 0;

	/**
	 * Read the next sample from the specified track.
	 * Advances the internal read position for that track.
	 * @param InTrackIndex Track index.
	 * @param OutSample Populated with the sample data and metadata.
	 * @return Success if a sample was read, EndOfStream if no more samples.
	 */
	virtual ETmvMediaContainerResult ReadSample(int32 InTrackIndex, FTmvMediaDemuxerSample& OutSample) = 0;

	/**
	 * Read only the sample metadata (no data copy) from the specified track.
	 * Useful for building sample tables or scanning without loading sample data.
	 * @param InTrackIndex Track index.
	 * @param OutSample Populated with metadata (Data array will be empty, SampleSize set).
	 * @return Success if metadata was read, EndOfStream if no more samples.
	 */
	virtual ETmvMediaContainerResult ReadSampleInfo(int32 InTrackIndex, FTmvMediaDemuxerSample& OutSample) = 0;

	/**
	 * Seek to a keyframe at or before the given time.
	 * After seeking, the next ReadSample call will return the keyframe sample.
	 * @param InTrackIndex Track index.
	 * @param InTime Target time.
	 * @param InLaterTimeThreshold If a keyframe is found within this threshold after InTime, prefer it.
	 * @return Success if a valid seek point was found.
	 */
	virtual ETmvMediaContainerResult Seek(int32 InTrackIndex, FTimespan InTime, FTimespan InLaterTimeThreshold = FTimespan::Zero()) = 0;

	/**
	 * Seek to a specific sample number.
	 * @param InTrackIndex Track index.
	 * @param InSampleNumber Target sample number (0-based).
	 * @return Success if the sample number is valid.
	 */
	virtual ETmvMediaContainerResult SeekToSample(int32 InTrackIndex, uint32 InSampleNumber) = 0;

	/**
	 * Close the container and release resources.
	 */
	virtual void Close() = 0;

	/**
	 * Returns a human-readable error message if the last operation failed.
	 */
	virtual FString GetLastError() const = 0;

	/**
	 * Returns the start timecode as a formatted string (e.g., "01:00:00:00"), if available.
	 * Must be called after Open.
	 */
	virtual TOptional<FString> GetStartTimecode() const { return {}; }

	/**
	 * Returns the start timecode frame rate as a string (e.g., "30000/1001"), if available.
	 * Must be called after Open.
	 */
	virtual TOptional<FString> GetStartTimecodeRate() const { return {}; }
};
