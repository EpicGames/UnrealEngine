// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ITmvMediaDemuxer;

/**
 * Interface for an ITmvMediaDemuxer factory.
 */
class ITmvMediaDemuxerFactory
{
public:
	virtual ~ITmvMediaDemuxerFactory() = default;

	/** Returns the name of this demuxer implementation. */
	virtual FName GetName() const = 0;

	/** Returns the display name of this demuxer implementation. */
	virtual FText GetDisplayName() const = 0;

	/**
	 * Return the list of container format identifiers this factory supports.
	 * Ex: "mp4", "fmp4", "mov"
	 */
	virtual TArray<FString> GetSupportedContainerFormats() const = 0;

	/**
	 * Creates a demuxer instance.
	 * @return Created demuxer, or null if failed.
	 */
	virtual TSharedPtr<ITmvMediaDemuxer, ESPMode::ThreadSafe> CreateDemuxer() = 0;
};
