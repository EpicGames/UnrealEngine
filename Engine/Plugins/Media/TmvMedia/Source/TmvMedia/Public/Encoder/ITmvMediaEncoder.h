// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Misc/FrameRate.h"
#include "SampleConverter/TmvMediaFrameMipBufferFwd.h"
#include "TmvMediaFrameInfo.h"

class FArchive;
class FString;
class FTmvMediaMessageContext;
struct FTmvMediaFrameTimeInfo;

/**
 * Encoder return code.
 */
enum class ETmvMediaEncoderResult
{
	Success,
	Fail,
	Cancelled,
	Skipped
};

/**
 * Interface for the encoder's access unit.
 * This is used by the encoder to write data to the output buffer.
 */
class ITmvMediaEncoderAccessUnit
{
public:
	virtual ~ITmvMediaEncoderAccessUnit() = default;

	/**
	 * Returns the offset in bytes from the start of the access unit. 
	 */
	virtual int64 Tell() const = 0;

	/**
	 * Returns the total size in bytes of the access unit.
	 */
	virtual int64 GetTotalSize() const = 0;
	
	/**
	 * Seek at the specified offset in bytes from the start of the access unit.
	 * @param InOffset Offset in bytes.
	 * @return true if the operation completed successfully, false otherwise.
	 */
	virtual bool Seek(int64 InOffset) = 0;

	/**
	 * Write the give data to the current position the specified amount of data.
	 * 
	 * @param InBuffer Source buffer
	 * @param InSize Size in bytes that will be written.
	 * @return Size written.
	 */
	virtual int64 Write(const void *InBuffer, int64 InSize) = 0;

	/**
	 * Returns the frame id for this access unit.
	 */
	virtual int32 GetFrameId() const = 0;

	/**
	 * Returns the filename. (TMP)
	 */
	virtual const FString& GetFilename() const = 0;

	/**
	 * Directly access the underlying archive. (TMP)
	 * This is expected to be an archive for only the access unit (not the whole container).
	 */
	virtual FArchive* GetUnderlyingArchive() const = 0;
};

/**
 * Mip encoding request.
 */
struct FTmvMediaEncoderMipRequest
{
	/** Mip buffer to encode. */
	FTmvMediaFrameMipBufferHandle MipBuffer;

	/** (Set by encoder) Result of the encoding operation. */
	ETmvMediaEncoderResult OutResult = ETmvMediaEncoderResult::Fail;
};

// @todo: Error handling: provide a callback for errors.

/**
 * All the parameters needed to request a mip memory layout for the encoder.
 */
struct FTmvMediaEncoderMipInfo
{
	/** Width of the largest mip for the frame. */
	int32 Width = 0;
	
	/** Height of the largest mip for the frame. */
	int32 Height = 0;

	/**
	 * Indicate if mip info must be generated.
	 * The encoder will determine which mip can be produced (if possible at all).
	 */
	bool bEnableMips = false;
	
	/** 
	 *  Color Specification to be encoded in the frame headers.
	 *  Note: Not all encoders can save all the color space info (if any at all).
	 */
	FTmvMediaFrameColorInfo ColorInfo;
};

/**
 * TMV (Tile-Based) Encoder interface.
 * 
 * Encodes data from the provided mip buffers and writes encoded bytes to the output access unit. 
 */
class ITmvMediaEncoder
{
public:
	virtual ~ITmvMediaEncoder() {}

	/**
	 * Queries the encoder for the memory layout and format for frames of the given size, and doing so for each mips.
	 * Can fail if the encoder doesn't support the given input format.
	 * 
	 * @param InTimeInfo Time information needed by some encoders to estimate bitrate and compression.
	 * @param InFrameInfo Mip0 information of incoming frame.
	 * @param OutFrameMipInfo output array populated with the request mip info.
	 * @param OutMessageContext Optional message context for the given operation.
	 * @return Result code depending on the success of the operation.
	 */
	virtual ETmvMediaEncoderResult RequestMipInfos(
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		const FTmvMediaEncoderMipInfo& InFrameInfo,
		TArray<FTmvMediaFrameMipInfo>& OutFrameMipInfo,
		FTmvMediaMessageContext* OutMessageContext) = 0;

	/**
	 * Encode the given mip requests.
	 * 
	 * @param InTimeInfo Time information needed by some encoders to encode in the frame.
	 * @param InAccessUnit Access unit for the current frame's encoded stream.
	 * @param InMipRequests Array of mip encoding requests.
	 * @param OutMessageContext Optional message context for the given operation.
	 * @return Result code depending on the success of the operation.
	 */
	virtual ETmvMediaEncoderResult Encode(
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		ITmvMediaEncoderAccessUnit& InAccessUnit,
		TArrayView<FTmvMediaEncoderMipRequest> InMipRequests,
		FTmvMediaMessageContext* OutMessageContext) = 0;
};