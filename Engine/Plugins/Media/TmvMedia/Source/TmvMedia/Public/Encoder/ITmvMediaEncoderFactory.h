// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "Templates/SharedPointer.h"

class ITmvMediaEncoder;
struct FInstancedStruct;
struct FTmvMediaEncoderOptions;

/**
 * Interface for a ITmvMediaEncoder factory.
 */
class ITmvMediaEncoderFactory
{
public:
	virtual ~ITmvMediaEncoderFactory() = default;

	/**
	 * Returns the name of this encoder implementation.
	 */
	virtual FName GetName() const = 0;

	/**
	 * Returns the display name of this encoder implementation.
	 */
	virtual FText GetDisplayName() const = 0;

	/**
	 * Return true if the encoder supports memory only access unit interface.
	 * Some encoders can only write to file because of third party implementations and don't support
	 * having an intermediate memory access unit that can be muxed independently.
	 */
	virtual bool SupportsMemoryAccessUnit() const = 0;

	/**
	 * Populates the provided instanced struct with encoder configuration options.
	 * @param OutOptions Populated options.
	 */
	virtual void GetEncoderOptions(TInstancedStruct<FTmvMediaEncoderOptions>& OutOptions) const  = 0;
	
	/**
	* Creates an encoder for the given format and options.
	* @param InCodecFormat Either FOURCC of the codec, ex: "apv1", or file extension (if each AU is a separate file). 
	* @param InOptions Additional options may be provided to specify the format more closely, which may help selecting the best suited implementation.
	* @return created encoder, or null if failed.
	*/
	virtual TSharedPtr<ITmvMediaEncoder, ESPMode::ThreadSafe> CreateEncoder(const FString& InCodecFormat, const TInstancedStruct<FTmvMediaEncoderOptions>& InOptions) = 0;
};