// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "Misc/Timecode.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"
#include "TmvMediaTrackInfo.h"


/**
 * Configuration for a single track in the muxer.
 * Extends the common track descriptor with muxer-specific fields.
 */
struct FTmvMediaMuxerTrackConfig : public FTmvMediaTrackInfo
{
	/**
	 * If true, every sample in this track is independently decodable (sync sample).
	 * If false, samples may depend on other samples.
	 */
	bool bIsAllKeyframes = true;

	/** Optional compressor name string (video tracks only). */
	FString CompressorName;

	/**
	 * If set, enables subsample information for each sample.
	 * The value is the 24-bit flags for the subsample box.
	 */
	TOptional<uint32> SubSampleFlags;
};

/**
 * Configuration for the muxer output.
 */
struct FTmvMediaMuxerConfig
{
	/** Output mode for the container. */
	enum class EOutputMode : uint8
	{
		/** Standard mode. Container metadata at end. */
		Standard,
		/** Web-optimized mode. Container metadata at front (requires temporary file + 2nd pass). */
		WebOptimized,
		/** Fragmented mode. Metadata interleaved with data. */
		Fragmented
	};

	/** Output file path. */
	FString OutputFilename;

	/** Temporary file path (required for WebOptimized mode). */
	FString TemporaryFilename;

	/** Output mode. */
	EOutputMode OutputMode = EOutputMode::Standard;

	/**
	 * Duration for interleaving chunks of sample data.
	 * Samples are collected per track until this duration is reached, then written.
	 * Typical value: 500ms.
	 */
	FTimespan InterleaveDuration = FTimespan::FromMilliseconds(500);
};

/**
 * A single sample to be muxed into a track.
 * Extends the common sample metadata with muxer-specific fields.
 */
struct FTmvMediaMuxerSample : public FTmvMediaTrackSampleInfo
{
	/** Subsample information entry. */
	struct FSubSampleInfo
	{
		uint32 CodecSpecificParameters = 0;
		uint32 SubSampleSize = 0;
		uint8 SubSamplePriority = 0;
		uint8 Discardable = 0;
	};

	/** Raw coded sample data. Written verbatim to the container. */
	TConstArrayView64<uint8> Data;

	/** Whether this is the final sample for this track. */
	bool bIsFinalSample = false;

	/** Optional subsample information. */
	TArray<FSubSampleInfo> SubSamples;
};

/**
 * Abstract interface for muxing encoded access units into a media container.
 *
 * Workflow:
 *   1. Configure() with output settings.
 *   2. AddTrack() for each media track, receiving a track index.
 *   3. Start() to begin the muxing process.
 *   4. AddSample() for each sample on each track (called from the sample request callback).
 *   5. Finalize() once all tracks signal their final sample.
 *
 * Sample delivery is asynchronous: the muxer calls back via FSampleRequestDelegate
 * to request samples. The caller provides them via AddSample().
 */
class ITmvMediaMuxer
{
public:
	virtual ~ITmvMediaMuxer() = default;

	/** Defines an entry in the sample request array indicating which track needs a sample. */
	struct FSampleRequest
	{
		int32 TrackIndex = -1;
		uint32 SampleNumber = 0;
	};

	/** Delegate invoked when the muxer needs samples for one or more tracks. */
	DECLARE_DELEGATE_OneParam(FSampleRequestDelegate, const TArray<FSampleRequest>& /*PendingRequests*/);

	/** Status reported upon completion or failure. */
	enum class EStatus
	{
		Failed,
		Finished
	};

	/** Delegate invoked when muxing completes or fails. */
	DECLARE_DELEGATE_OneParam(FStatusDelegate, EStatus /*Status*/);

	/**
	 * Configure the muxer with output settings.
	 * Must be called before AddTrack().
	 * @param InConfig Muxer configuration.
	 * @return Success if configuration is valid, Fail otherwise.
	 */
	virtual ETmvMediaContainerResult Configure(const FTmvMediaMuxerConfig& InConfig) = 0;

	/**
	 * Add a track to the container.
	 * Must be called after Configure() and before Start().
	 * @param InTrackConfig Track configuration.
	 * @return Track index (>= 0) on success, or -1 on failure.
	 */
	virtual int32 AddTrack(const FTmvMediaMuxerTrackConfig& InTrackConfig) = 0;

	/**
	 * Whether this muxer supports writing a start timecode into the container.
	 */
	virtual bool SupportsStartTimecode() const { return false; }

	/**
	 * Set the starting timecode to be written into the container.
	 * Must be called after Configure() and before Start().
	 * Implementations decide how to represent the timecode in their container format
	 * (e.g., as a timecode track, user-data metadata, or not at all).
	 * @param InTimecode The starting timecode.
	 * @param InFrameRate The frame rate associated with the timecode.
	 * @return true if the timecode was accepted, false on error.
	 */
	virtual bool SetStartTimecode(const FTimecode& InTimecode, const FFrameRate& InFrameRate) { return false; }

	/**
	 * Start the muxing process.
	 * The muxer will begin calling InSampleRequestDelegate to request samples.
	 * @param InSampleRequestDelegate Called when the muxer needs more sample data.
	 * @param InStatusDelegate Called when muxing completes or fails.
	 * @return Success if started, Fail otherwise.
	 */
	virtual ETmvMediaContainerResult Start(FSampleRequestDelegate InSampleRequestDelegate, FStatusDelegate InStatusDelegate) = 0;

	/**
	 * Provide a sample to the muxer for the specified track.
	 * Called in response to FSampleRequestDelegate.
	 * @param InTrackIndex Index returned by AddTrack().
	 * @param InSample The sample data and metadata.
	 * @return Success if the sample was accepted.
	 */
	virtual ETmvMediaContainerResult AddSample(int32 InTrackIndex, const FTmvMediaMuxerSample& InSample) = 0;

	/**
	 * Stop muxing and finalize the output file.
	 * @return Success if the file was closed properly.
	 */
	virtual ETmvMediaContainerResult Finalize() = 0;

	/**
	 * Returns a human-readable error message if the last operation failed.
	 */
	virtual FString GetLastError() const = 0;
};
