// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ITmvMediaMuxer;

/**
 * Interface for an ITmvMediaMuxer factory.
 */
class ITmvMediaMuxerFactory
{
public:
	virtual ~ITmvMediaMuxerFactory() = default;

	/** Returns the name of this muxer implementation. */
	virtual FName GetName() const = 0;

	/** Returns the display name of this muxer implementation. */
	virtual FText GetDisplayName() const = 0;

	/**
	 * Return the list of container format identifiers this factory supports.
	 * Ex: "tmv"
	 */
	virtual TArray<FString> GetSupportedContainerFormats() const = 0;

	/**
	 * Returns the file extension for the output container (without leading dot).
	 * Ex: "tmv"
	 */
	virtual FString GetFileExtension() const = 0;

	/**
	 * Creates a muxer instance.
	 * @return Created muxer, or null if failed.
	 */
	virtual TSharedPtr<ITmvMediaMuxer, ESPMode::ThreadSafe> CreateMuxer() = 0;
};
