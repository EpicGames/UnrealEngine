// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/TextureDefines.h"
#include "UObject/NameTypes.h"

#include "TmvMediaEncoderOptions.generated.h"

/** 
 * List of supported encoder transfer functions (a.k.a encodings), matching the list in ColorManagementDefines.h.
 */
UENUM()
enum class ETmvMediaEncoderEncoding : uint8
{
	/** Unspecified (no override). */
	None = 0,
	/** Linear encoding. */
	Linear = 1,
	/** sRGB encoding. */
	sRGB = 2 UMETA(DisplayName="sRGB"),
	/** SMPTE ST 2084/PQ: input is absolute nits. Paired with ReferenceWhite BT.2408 by default. */
	ST2084 = 3  UMETA(DisplayName="ST 2084/PQ"),
	/** Sony S-Log3 encoding. */
	SLog3 = 12,
	/** HLG (BT.2100): input is normalized in the [0, 1] range. Paired with ReferenceWhite BT.2408 by default. */
	HLG = 17,
};

/**
 * Reference-white standards for HDR encoder input. Integer values match UE::Color::EReferenceWhite.
 */
UENUM()
enum class ETmvMediaEncoderReferenceWhite : uint8
{
	/** The source reference white is not overridden, defer to system/default behavior. */
	None                 = 0,
	/** Explicitly preserve input/native scaling. */
	DisableNormalization = 1 UMETA(DisplayName = "Disable Normalization"),
	/** ITU-R BT.2408 Reference White. 203 cd/m2 for PQ (ST2084), 75% signal for HLG scale to scene linear white (1.0). */
	BT2408               = 2 UMETA(DisplayName = "BT.2408 (203 nits)"),
	/** ITU-R BT.1886 SDR reference white (100 cd/m2). Scales PQ (100-nit paper white) and HLG (in-house 100-nit convention on a 1000 cd/m2 BT.2100 reference display) to scene linear white (1.0). */
	BT1886               = 3 UMETA(DisplayName = "BT.1886 (100 nits)"),

	Max                  = 4 UMETA(Hidden),
};

/**
 * Carries the information about the YUV conversion matrix.
 * Note: this is independent of the color space. YUV conversion is done prior to the application of the transfer function.
 */
UENUM()
enum class ETmvMediaEncoderColorMatrix : uint8
{
	/** Unspecified (no override). */
	None = 0,
	/** Identity, i.e. no yuv conversion (rgb color model). */
	Identity = 1,
	/** Rec. ITU-R BT.601-7 525 (Kr = 0.299, Kb = 0.114) */
	Rec601 = 2 UMETA(DisplayName="Rec601"),
	/** Rec. ITU-R BT.709-6 (Kr = 0.2126, Kb = 0.0722) */
	Rec709 = 3 UMETA(DisplayName="Rec709"),
	/** Rec. ITU-R BT.2020-2 (Kr = 0.2627, Kb = 0.0593) */
	Rec2020 = 4 UMETA(DisplayName="Rec2020"),
};

/**
 * Carries the information about the range of the YUV conversion matrix (Scaled vs UnScaled)
 */
UENUM()
enum class ETmvMediaEncoderColorMatrixRange : uint8
{
	/** 
	 * Limited (studio) range, defined by spec. Typical [16, 235] for luma on 8 bits limited range.
	 * This range definition corresponds to the "scaled" yuv matrices and offsets from MediaShaders.
	 */
	Limited,
	/** 
	 * Full [0, 255] range.
	 * This range definition corresponds to the "unscaled" yuv matrices and offsets from MediaShaders.
	 */
	Full
};

/**
 * Encoder options.
 */
USTRUCT()
struct FTmvMediaEncoderOptions
{
	GENERATED_BODY()

	/** Returns the encoder name used to find the corresponding factory. */
	virtual FName GetEncoderName() const { return NAME_None; }

	/** Returns the extension used for saving to file sequence. */
	virtual FString GetFileSequenceExtension() const { return FString(); }

	/** Returns the FOURCC sample entry format identifying the codec (e.g. 'apv1'). Used for container muxing. */
	virtual uint32 GetCodecFourCC() const { return 0; }

	/**
	 * Returns the destination color spaces supported by this encoder. An empty array means all values are allowed.
	 */
	virtual void GetSupportedDestinationColorSpaces(TArray<ETextureColorSpace>& OutSupportedColorSpaces) const {}

	/**
	 * Returns the destination encodings (transfer functions) supported by this encoder. An empty array means all values are allowed.
	 */
	virtual void GetSupportedDestinationEncodings(TArray<ETmvMediaEncoderEncoding>& OutSupportedEncodings) const {}

	virtual ~FTmvMediaEncoderOptions() = default;

	/** 
	 * Enables the color management specification for the encoder.
	 */
	UPROPERTY(EditAnywhere, Category = "Color Management")
	bool bEnableColorManagement = false;
	
	/** Specify the destination color space. */
	UPROPERTY(EditAnywhere, Category = "Color Management", meta = (EditCondition = "bEnableColorManagement", EditConditionHides))
	ETextureColorSpace DestinationColorSpace = ETextureColorSpace::TCS_None;

	/** Specify the destination color encoding. */
	UPROPERTY(EditAnywhere, Category = "Color Management", meta = (EditCondition = "bEnableColorManagement", EditConditionHides))
	ETmvMediaEncoderEncoding DestinationEncoding = ETmvMediaEncoderEncoding::Linear;

	/** Reference-white standard used to scale UE scene-linear 1.0 to diffuse white. */
	UPROPERTY(EditAnywhere, Category = "Color Management", meta = (EditCondition = "bEnableColorManagement && (DestinationEncoding == ETmvMediaEncoderEncoding::ST2084 || DestinationEncoding == ETmvMediaEncoderEncoding::HLG)", EditConditionHides))
	ETmvMediaEncoderReferenceWhite ReferenceWhite = ETmvMediaEncoderReferenceWhite::BT2408;

	/** Specify the YUV encoding color matrix. */
	UPROPERTY(EditAnywhere, Category = "Color Management", meta = (EditCondition = "bEnableColorManagement", EditConditionHides))
	ETmvMediaEncoderColorMatrix YuvMatrix = ETmvMediaEncoderColorMatrix::Identity;

	/** Specify the YUV encoding color matrix range. */
	UPROPERTY(EditAnywhere, Category = "Color Management", meta = (EditCondition = "bEnableColorManagement", EditConditionHides))
	ETmvMediaEncoderColorMatrixRange YuvMatrixRange = ETmvMediaEncoderColorMatrixRange::Full;
};
