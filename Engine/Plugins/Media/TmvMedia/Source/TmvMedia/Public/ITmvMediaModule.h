// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/Variant.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

#define UE_API TMVMEDIA_API

class ITmvMediaDecoderFactory;
class ITmvMediaDemuxerFactory;
class ITmvMediaEncoderFactory;
class ITmvMediaMuxerFactory;
class ITmvMediaTranscodeJobManager;
class ITmvMediaTranscodeJobRunner;

/**
 * Interface for the Tiled-Mipmap Video (TMV) module.
 */
class ITmvMediaModule : public IModuleInterface
{
public:
	/**
	 * Returns a pointer to the module instance if loaded, null otherwise.
	 */
	static UE_API ITmvMediaModule* Get();

	/**
	 * Returns existing instance of the module, loads it if it was not already loaded.
	 * Beware of calling this during the shutdown phase, though. The module might have been unloaded already.
	 */
	static UE_API ITmvMediaModule& GetOrLoad();
	
	/**
	 * Register a decoder factory in the module.
	 */
	virtual void RegisterDecoderFactory(TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> InDecoderFactory) = 0;

	/**
	 * Unregister a decoder factory from the module.
	 */
	virtual void UnregisterDecoderFactory(TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> InDecoderFactory) = 0;

	/**
	 * Get a list of all currently registered factories. 
	 */
	virtual void GetDecoderFactories(TArray<TWeakPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe>>& OutDecoderFactories) const = 0;

	/**
	 * Returns the best decoder factory for the given format.
	 * @param InCodecFormat FOURCC of the codec, ex: "aPv1" 
	 * @param InOptions Additional options may be provided to specify the format more closely, which may help selecting the best suited implementation.
	 * @return Highest priority decoder factory if found, null otherwise.
	 */
	virtual TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> GetBestDecoderFactoryForFormat(const FString& InCodecFormat, const TMap<FString, FVariant>& InOptions) const = 0;

	/**
	 * Register an encoder factory in the module.
	 */
	virtual void RegisterEncoderFactory(TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> InEncoderFactory) = 0;

	/**
	 * Unregister an encoder factory from the module.
	 */
	virtual void UnregisterEncoderFactory(TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> InEncoderFactory) = 0;

	/**
	 * Get a list of all currently registered encoder factories. 
	 */
	virtual void GetEncoderFactories(TArray<TWeakPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe>>& OutEncoderFactories) const = 0;

	/**
	 * Find the encoder factory by name.
	 * @param InEncoderFactoryName Name of the factory.
	 * @return The requested factory if found.
	 */
	virtual TSharedPtr<ITmvMediaEncoderFactory, ESPMode::ThreadSafe> FindEncoderFactory(FName InEncoderFactoryName) const = 0;

	/**
	 * Register a muxer factory in the module.
	 */
	virtual void RegisterMuxerFactory(TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> InMuxerFactory) = 0;

	/**
	 * Unregister a muxer factory from the module.
	 */
	virtual void UnregisterMuxerFactory(TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> InMuxerFactory) = 0;

	/**
	 * Get a list of all currently registered muxer factories.
	 */
	virtual void GetMuxerFactories(TArray<TWeakPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>>& OutMuxerFactories) const = 0;

	/**
	 * Register a demuxer factory in the module.
	 */
	virtual void RegisterDemuxerFactory(TSharedPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe> InDemuxerFactory) = 0;

	/**
	 * Unregister a demuxer factory from the module.
	 */
	virtual void UnregisterDemuxerFactory(TSharedPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe> InDemuxerFactory) = 0;

	/**
	 * Get a list of all currently registered demuxer factories.
	 */
	virtual void GetDemuxerFactories(TArray<TWeakPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe>>& OutDemuxerFactories) const = 0;

	/**
	 * Find the demuxer factory that supports the given file extensions.
	 * @param InFileExtension File extension without dot
	 * @return Demuxer factory if found, null otherwise.
	 */
	virtual TSharedPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe> FindDemuxerFactoryForExtension(const FString& InFileExtension) const = 0;

	/**
	 * Access the global transcode job manager.
	 */
	virtual ITmvMediaTranscodeJobManager* GetTranscodeJobManager() const = 0;

	/**
	 * Access the global transcode job runner, which owns the active and pending jobs and
	 * ticks them from an engine system independent of Slate.
	 */
	virtual ITmvMediaTranscodeJobRunner* GetTranscodeJobRunner() const = 0;
};

#undef UE_API
