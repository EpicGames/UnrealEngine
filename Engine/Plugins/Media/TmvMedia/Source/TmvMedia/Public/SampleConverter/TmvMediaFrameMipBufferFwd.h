// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FTmvMediaFrameMipBuffer;

/**
* A shared mip buffer acquired from a pool that handles returning to the pool upon release.
*/
typedef TSharedPtr<FTmvMediaFrameMipBuffer, ESPMode::ThreadSafe> FTmvMediaFrameMipBufferHandle;
