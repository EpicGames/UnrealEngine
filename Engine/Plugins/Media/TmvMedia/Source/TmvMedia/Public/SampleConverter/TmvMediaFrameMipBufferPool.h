// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "SampleConverter/TmvMediaFrameMipBufferFwd.h"
#include "Templates/SharedPointer.h"
#include "TmvMediaFrameInfo.h"

// Full definition lives in Internal/SampleConverter/TmvMediaFrameMipBuffer.h.
// The pool's public API only needs the handle (above) and a raw pointer for
// internal pool storage; the .cpp pulls in the full type.
class FTmvMediaFrameMipBuffer;

#define UE_API TMVMEDIA_API

/**
 * Tmv Frame Mip Buffer Pool.
 * 
 * This pool is held by the tmv player to allow recycling of frame buffers between decoded samples. 
 */
class FTmvMediaFrameMipBufferPool : public TSharedFromThis<FTmvMediaFrameMipBufferPool, ESPMode::ThreadSafe>
{
public:
	FTmvMediaFrameMipBufferPool() = default;
	UE_API ~FTmvMediaFrameMipBufferPool();

	/** Acquire a buffer suitable for the given frame mip. */
	UE_API FTmvMediaFrameMipBufferHandle AcquireBuffer(const FTmvMediaFrameMipInfo& InMipInfo);

private:
	/** Either return or Add new chunk of memory to the pool based on its size. */
	void ReleaseBuffer(SIZE_T InAllocSize, FTmvMediaFrameMipBuffer* InBuffer);

	// Non-copyable
	FTmvMediaFrameMipBufferPool(const FTmvMediaFrameMipBufferPool&) = delete;
	FTmvMediaFrameMipBufferPool& operator=(const FTmvMediaFrameMipBufferPool&) = delete;

private:
	/** A critical section used for memory allocation and pool management. */
	FCriticalSection MemoryPoolCriticalSection;

	/** Memory pool from where we are allowed to take buffers. */
	TMultiMap<SIZE_T, FTmvMediaFrameMipBuffer*> MemoryPool;
};

#undef UE_API