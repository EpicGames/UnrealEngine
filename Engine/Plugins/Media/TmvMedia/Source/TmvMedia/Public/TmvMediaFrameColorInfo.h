// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorManagement/ColorManagementDefines.h"

/**
 * Carries the information about the YUV conversion matrix.
 * Note: this is independent of the final RGB color space. YUV conversion is done prior to the application of the transfer function.
 */
enum class ETmvMediaFrameColorMatrix : uint8
{
	/** Unspecified. */	
	None = 0,
	/** Identity, i.e. no yuv conversion (rgb color model). */
	Identity = 1,
	/** Rec. ITU-R BT.601-7 525 (Kr = 0.299, Kb = 0.114) */
	Rec601 = 2,
	/** Rec. ITU-R BT.709-6 (Kr = 0.2126, Kb = 0.0722) */
	Rec709 = 3,
	/** Rec. ITU-R BT.2020-2 (Kr = 0.2627, Kb = 0.0593) */
	Rec2020 = 4
};

/**
 * Carries the information about the range of the YUV conversion matrix (Scaled vs UnScaled)
 */
enum class ETmvMediaFrameColorMatrixRange : uint8
{
	/** 
	 * Typical [16,219] (on 8 bits) limited range. Defined by spec. 
	 * This range definition corresponds to the "scaled" yuv matrices and offsets from MediaShaders.
	 */
	Limited = 0,

	/** 
	 * Full [0, 255] range. 
	 * This range definition corresponds to the "unscaled" yuv matrices and offsets from MediaShaders.
	 */
	Full = 1
};

/** 
 * Defines the color space, encoding and yuv matrix of a frame buffer in the tmv framework.
 */
struct FTmvMediaFrameColorInfo
{
	/** Information about the color encoding of the corresponding frame buffer. */
	UE::Color::EEncoding Encoding = UE::Color::EEncoding::None;

	/**
	 * Information about the color space of the corresponding frame buffer.
	 * @remark For color space conversions, If the color space is left to "None",
	 * it will be assumed to be the current engine's "working" color space by default.
	 */
	UE::Color::EColorSpace ColorSpace = UE::Color::EColorSpace::None;

	/** Reference-white standard used to scale UE scene-linear 1.0 to diffuse white. */
	UE::Color::EReferenceWhite ReferenceWhiteOverride = UE::Color::EReferenceWhite::None;

	/** Information about the YUV color matrix of the corresponding frame buffer. */
	ETmvMediaFrameColorMatrix YuvMatrix = ETmvMediaFrameColorMatrix::Rec709;

	/** Information about the YUV color matrix range of the corresponding frame buffer. */
	ETmvMediaFrameColorMatrixRange YuvMatrixRange = ETmvMediaFrameColorMatrixRange::Limited;

	/**
	 * Make a neutral "no-overrides" color info that leaves everything unspecified.
	 * This is used when requesting the desired format of the encoder in case there are no overrides.
	 */
	static FTmvMediaFrameColorInfo MakeNeutral()
	{
		using namespace UE::Color;
		return{EEncoding::None, EColorSpace::None, EReferenceWhite::None, ETmvMediaFrameColorMatrix::None, ETmvMediaFrameColorMatrixRange::Full};
	}

	/**
	 * Apply the color info overrides, i.e. everything that has a value set.
	 *
	 * @param InOverrides Specify the overrides to apply to current color settings.
	 */
	void ApplyOverrides(const FTmvMediaFrameColorInfo& InOverrides)
	{
		if (InOverrides.Encoding != UE::Color::EEncoding::None)
		{
			Encoding = InOverrides.Encoding;
		}
		if (InOverrides.ColorSpace != UE::Color::EColorSpace::None)
		{
			ColorSpace = InOverrides.ColorSpace;
		}
		if (InOverrides.ReferenceWhiteOverride != UE::Color::EReferenceWhite::None)
		{
			ReferenceWhiteOverride = InOverrides.ReferenceWhiteOverride;
		}
		if (InOverrides.YuvMatrix != ETmvMediaFrameColorMatrix::None)
		{
			YuvMatrix = InOverrides.YuvMatrix;
			YuvMatrixRange = InOverrides.YuvMatrixRange;
		}
	}
};
