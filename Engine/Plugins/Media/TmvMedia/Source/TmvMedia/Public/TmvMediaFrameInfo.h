// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathUtility.h"
#include "TmvMediaFrameColorInfo.h"

#define UE_API TMVMEDIA_API

// Free helpers exposed from TmvMediaFrameMipBufferPool.cpp so the inline
// FTmvMediaFrameMipInfo methods below can consult the runtime CVars without
// pulling the IConsoleManager header into every translation unit that touches
// FTmvMediaFrameMipInfo. Only the accessors that inline-public methods reach
// live here; sparse-output and shared-pool accessors are declared in
// Internal/TmvMediaTuningCVars.h.
namespace UE::TmvMedia
{
	/** True when mip-buffer virtualization (r.TmvMedia.VirtualizeMipBuffers) is on. */
	TMVMEDIA_API bool IsVirtualizationEnabled();

	/**
	 * Legacy fixed-formula per-mip slot budget:
	 * budget(mip) = max(MipBudgetMin, MipBudgetBase >> mip).
	 * Used only when MipBudgetBase > 0 (explicit override). Callers should
	 * clamp against the mip's worst-case tile count themselves.
	 */
	TMVMEDIA_API int32 GetMipBudget(int32 MipLevel);

	/**
	 * UHD-viewport heuristic for the per-mip slot budget:
	 * budget = K * (3840 * 2160) / (TileWidth * TileHeight),
	 * floored by MipBudgetMin. Same value applies at every mip; per-mip
	 * worst-case clamping (in FTmvMediaFrameMipInfo) handles small mips.
	 * Used when MipBudgetBase == 0 (the default).
	 */
	TMVMEDIA_API int32 ComputeHeuristicMipBudget(int32 TileWidth, int32 TileHeight);

	/** True when r.TmvMedia.GrowMipBuffersOnDemand allows runtime growth. */
	TMVMEDIA_API bool IsGrowOnDemandEnabled();

	/**
	 * Returns the per-mip slot growth ceiling (r.TmvMedia.MipBudgetMax). A
	 * return of 0 means "no cap from the CVar"; callers must still clamp to
	 * the mip's worst-case tile count.
	 */
	TMVMEDIA_API int32 GetMipBudgetMax();
}

/** Defines currently supported component data types. */
enum class ETmvMediaFrameComponentType : int8
{
	/** Component is stored as an integer. */
	Int = 0,
	/** Component is stored as a float. */
	Float
};

/**
 * Defines buffer memory data layout, i.e. the overall way the memory is organized in the
 * whole frame buffer. Tiling allows for better memory locality for certain type of processing,
 * especially for decoding. 
 */
enum class ETmvMediaFrameBufferLayout : int8
{
	/** Scan line layout */
	ScanLine = 0,
	/** Tiled layout */
	Tiled
};

/** Defines the layout of components in a memory plane. */
enum class ETmvMediaFrameComponentLayout : int8
{
	/**
	 * Components are packed. Ex: R0G0B0|R1G1B1|...
	 * Example of formats with packed components in a plane: NV12 has UV packed.
	 */
	Packed = 0,
	/**
	 * Components are interleaved on each scan line. Ex: R0R1...|G0G1...|B0B1...
	 * Example of formats with interleaved components in a plane: IMC2, IMC4
	 */
	Interleaved
};

/**
 * Defines the color model the components are representing. 
 * The usage is for the sample converter implementation to convert to the final RGBA format for the final media texture.
 * It is not intended to be a fully generalized format.
 */
enum class ETmvMediaFrameColorModel : int8
{
	YUV = 0,
	RGB,
	// @todo We may have to add variations for component order and packing. Ex: BGR, YVU, YUYV, UYVY, VYUY
};

/**
 * Defines the layout and format of a video frame memory plane.
 * 
 * A memory plane is one contiguous memory region that can contain more than one component.
 * Ex: NV12 has a packed UV plane (semi-planar formats), or IMC2 has interleaved U and V
 * scan lines in a single memory plane.
 * 
 * A "sample" is an element of the current plane (interleaved or not).
 * 
 * No start offset is provided, we assume the valid pixels start at offset 0 in the plane.
 * The stride is the offset between the start of 2 scan lines, includes padding and interleaved components.
 * 
 * If there is padding between planes, it is included in the extra lines (see NumLines). It is assumed those
 * extra lines are at the end of the plane buffer.
 */
struct FTmvMediaFramePlaneInfo
{
	/** Number of components packed per sample in this plane. */
	uint8 NumComponents = 0;

	/**
	 * Number of bits per component of this plane.
	 * Used by the sample converter. 
	 * @remark Memory layouts are always aligned to the next full byte. Ex: 10, 12 bits are still memory aligned to 16 bits.
	 */
	uint8 BitDepth = 0;

	/**
	 * Defines the components data type.
	 * Used by the sampler converter.
	 */
	ETmvMediaFrameComponentType Type = ETmvMediaFrameComponentType::Int;

	/** Number of horizontal samples in this plane. */
	uint32 Width = 0;

	/** Number of vertical samples in this plane. */
	uint32 Height = 0;

	/**
	 * Number of memory bytes per scan line of this memory plane.
	 * This must include padding at the end of the line.
	 * In case of interleaved component layouts, this stride is for the whole scan line including all components
	 * and any additional padding need by the decoder for macroblock or cache alignment.
	 */
	uint32 Stride = 0;

	/**
	 * Number of scan lines in this plane.
	 * This could include more lines for inter-plane (or inter-tile) padding. 
	 */
	uint32 NumLines = 0;
	
	/**
	 * Defines the component layout within the plane.
	 * This is used to define planes with interleaved scan lines, such as IMC2 or IMC4
	 * where the U and V planes have the scan lines interleaved.
	 */ 
	ETmvMediaFrameComponentLayout ComponentLayout = ETmvMediaFrameComponentLayout::Packed;
	
	/**
	 * Width ratio defined as Frame Width / Plane Width.
	 * Used for chroma subsampling. Ex: for 422 or 420, width ratio is 2 for U and V planes. 
	 */
	int8 WidthRatio = 1;

	/**
	 * Height ratio defined as Frame Height / Plane Height.
	 * Used for chroma subsampling. Ex: for 420, height ratio is 2 for U and V planes. 
	 */
	int8 HeightRatio = 1;

	/**
	 * Returns the number of bytes per component for this memory plane.
	 * The memory layouts are always aligned to bytes.
	 */
	uint32 GetBytesPerComponent() const
	{
		return (BitDepth+7)>>3;
	}

	/**
	 * Returns the memory size in bytes of this memory plane.
	 */
	SIZE_T GetMemorySizeInBytes() const
	{
		return static_cast<SIZE_T>(Stride) * static_cast<SIZE_T>(NumLines);
	}

	/**
	 * Computes the offset, in bytes, to the start of the given component in the plane buffer.
	 * @param InComponentIndexInPlane Index of the given component in the current plane. Ex: in a UV plane, U would have index 0 and V, index 1.
	 * @return Offset to apply to the start of the plane buffer to get the first pixel of the given component.
	 */
	uint32 GetStartComponentOffsetInBytes(int32 InComponentIndexInPlane) const
	{
		if (InComponentIndexInPlane <= 0 || NumComponents == 0) 
		{ 
			return 0; 
		}

		// Clamp to last valid component to avoid out-of-range offsets. 
		const uint32 ClampedIndex = static_cast<uint32>(FMath::Min<int32>(InComponentIndexInPlane, static_cast<int32>(NumComponents) - 1)); 

		if (ComponentLayout == ETmvMediaFrameComponentLayout::Interleaved)
		{
			// For interleaved components, add the interleaving offset.
			return ClampedIndex * (Stride / NumComponents);
		}
		
		// For packed components, add the offset of the component.
		return ClampedIndex * GetBytesPerComponent();
	}
};

/**
 * This struct describes information (memory layout, color model, component type)
 * about a mipmap buffer for a video frame. A complete TMV frame for a given time index will be composed
 * of multiple mips each of which is described by this struct. The goal is to allow decoders to provide
 * the sample conversion implementation enough information to prepare mip frame buffers with the needed layout
 * for the decoder, and then be able to handle the conversion on the GPU through PS or CS shaders.
 *
 * Memory Surface Layout:
 * The pixel components may be packed in a single memory plane, separated in different planes
 * or mixed (semi-planar ex: NV12).
 * 
 * Chroma subsampling for YUV color model is supported by having planes with different dimensions.
 *
 * The memory data layout can be either tiled or scanline. If the layout is tiled, the plane definitions
 * (dimensions, stride, padding, etc) are per tile rather than for the whole frame.
 *
 * FOURCC is not enough:
 * There are FOURCC types for fully planar YUV types such as P010 (420 10 bits) or P210 (422, 10 bits).
 * However, those formats don't describe custom padding for decoder tile or macro block memory alignment for best performance.
 * This is why at this level of the api, it is preferable to not try to align with existing FOURCC. The current descriptor
 * is still probably incomplete as it only targets a subset of implementations, but it is expected to evolve over time.
 * 
 * Some reference on the subject:
 * - https://github.com/torvalds/linux/blob/master/include/uapi/drm/drm_fourcc.h -> FOURCC + Modifier to account for memory layout.
 *
 * Component Type:
 * All components have the same type, either float or int.
 *
 * Color Model:
 * Defines the meaning of the first 3 components: RGB, YUV
 * Alpha is supported as the 4th component in the sampler converter, regardless of the color model.
 */
struct FTmvMediaFrameMipInfo
{
	/** Mip level */
	int32 MipLevel = 0;

	/** Number of horizontal samples in this mip. */
	int32 Width = 0;

	/** Number of vertical samples in this mip. */
	int32 Height = 0; 

	/** Total number of components per sample. May be packed in planes. */
	int32 NumComponents = 0;

	/** Indicate the color model that allow to determine how the component should be interpreted. */
	ETmvMediaFrameColorModel ColorModel = ETmvMediaFrameColorModel::RGB;

	/** Specifies the color information (color space, encoding, yuv matrix and range) for the given frame. */
	FTmvMediaFrameColorInfo ColorInfo;

	/** Number of horizontal samples in a tile. */
	int32 TileWidth = 0;

	/** Number of vertical samples in a tile. */
	int32 TileHeight = 0;

	/** Number of tiles on each dimensions in this mip. */
	FIntPoint NumTiles = FIntPoint(0,0);

	/** Defines the memory layout of all mip planes. */
	ETmvMediaFrameBufferLayout Layout = ETmvMediaFrameBufferLayout::ScanLine;

	// @todo: For tiled layout, might have to expose tile order, i.e. zig-zag vs linear (+ inverse).
	// But we will try to hide this in the decoder for now, and have the decoder implementations always load tiles in a linear order.

	/**
	 * Description of memory planes.
	 * The planes are defined in this array in the order they are in memory.
	 * Interleaved planes must be in order and grouped in the planes array as well.
	 * For tiled layout, defines the planes per-tile.
	 * For YUV 420, 422, planes have different sizes because of chroma subsampling.
	 */
	TArray<FTmvMediaFramePlaneInfo, TInlineAllocator<4>> Planes;

	/**
	 * Returns the memory size in bytes for all planes.
	 * @remark This will include padding (either between planes or between tiles).
	 */
	SIZE_T GetAllPlaneMemorySizeInBytes() const
	{
		SIZE_T TotalSize = 0;
		for (const FTmvMediaFramePlaneInfo& PlaneInfo : Planes)
		{
			TotalSize += PlaneInfo.GetMemorySizeInBytes();
		}
		return TotalSize;
	}

	/**
	 * Gets the plane buffer offset accounting for the size of all previous planes.
	 * This is used for memory layout where all the planes are in one large memory block. 
	 * @param InPlaneIndex Index of the plane we want to get the start offset of.
	 * @param OutPlaneBufferOffset Value populated with the plane buffer offset.
	 * @return true if the plane buffer offset could be calculated, false otherwise.
	 */
	bool GetPlaneBufferOffset(int32 InPlaneIndex, SIZE_T& OutPlaneBufferOffset) const
	{
		if (Planes.IsValidIndex(InPlaneIndex))
		{
			SIZE_T PlaneOffset = 0;
			for (int32 PlaneIndex = 0; PlaneIndex < Planes.Num(); ++PlaneIndex)
			{
				if (InPlaneIndex == PlaneIndex)
				{
					OutPlaneBufferOffset = PlaneOffset;
					return true;
				}
				PlaneOffset += Planes[PlaneIndex].GetMemorySizeInBytes();
			}
		}
		return false;
	}

	/**
	 * Returns the total memory size in bytes.
	 */
	SIZE_T GetMemorySizeInBytes() const
	{
		switch (Layout)
		{
		// In tiled layout, the planes define the layout of the tiles, so we need to multiply by the number of tiles.
		case ETmvMediaFrameBufferLayout::Tiled:
			return GetAllPlaneMemorySizeInBytes() * NumTiles.X * NumTiles.Y;

		// In scan line layout, the planes are already defining the full size of the memory buffer.
		case ETmvMediaFrameBufferLayout::ScanLine:
		default:
			return GetAllPlaneMemorySizeInBytes();
		}
	}

	/**
	 * Returns the worst-case tile count for this mip (NumTiles.X * NumTiles.Y).
	 * Used as the clamp for the slot budget so we never allocate MORE slots
	 * than the mip actually has tiles.
	 */
	int32 GetWorstCaseTileCount() const
	{
		return NumTiles.X * NumTiles.Y;
	}

	/**
	 * Returns the effective number of resident tile slots for this mip,
	 * accounting for the global virtualization CVar and the per-mip budget.
	 * When virtualization is off OR the worst-case tile count is already
	 * below the budget, this returns the worst case (i.e. legacy behavior).
	 *
	 * Budget source:
	 *  - r.TmvMedia.MipBudgetBase > 0 -> legacy fixed formula via GetMipBudget
	 *  - r.TmvMedia.MipBudgetBase == 0 (default) -> UHD heuristic via
	 *    ComputeHeuristicMipBudget(TileWidth, TileHeight), same value at
	 *    every mip and clamped to worst case here.
	 */
	int32 GetBudgetedSlotCount() const
	{
		const int32 WorstCase = GetWorstCaseTileCount();
		if (Layout != ETmvMediaFrameBufferLayout::Tiled || !UE::TmvMedia::IsVirtualizationEnabled())
		{
			return WorstCase;
		}
		// MipBudgetBase>0 -> fixed formula (explicit override).
		// MipBudgetBase==0 (default, returned as 0 sentinel by GetMipBudget) -> UHD heuristic.
		const int32 FixedFormula = UE::TmvMedia::GetMipBudget(MipLevel);
		const int32 ChosenBudget = (FixedFormula > 0)
			? FixedFormula
			: UE::TmvMedia::ComputeHeuristicMipBudget(TileWidth, TileHeight);
		return FMath::Min(WorstCase, ChosenBudget);
	}

	/**
	 * Returns true when this mip is actually virtualized - i.e. the budget is
	 * strictly below the worst-case tile count, so a logical-to-physical tile
	 * indirection is needed. When false, the buffer layout is identity-mapped
	 * (slot index == tile index) and decode/sample paths can take the fast
	 * path that pre-dates virtualization.
	 */
	bool IsVirtualized() const
	{
		// Use GetBudgetedSlotCount() - not GetMipBudget() directly - because
		// GetMipBudget() returns 0 as a sentinel meaning "use the heuristic",
		// which makes the comparison "0 < WorstCase" pass unconditionally
		// even when the heuristic actually exceeds worst case. That bug
		// forced every Tiled mip through the virtualized path (per-frame
		// descriptor rebuild, residency tracking) even when the effective
		// budget was at or above worst case - paying the overhead with none
		// of the benefit.
		return Layout == ETmvMediaFrameBufferLayout::Tiled
			&& UE::TmvMedia::IsVirtualizationEnabled()
			&& GetBudgetedSlotCount() < GetWorstCaseTileCount();
	}

	/**
	 * Returns the memory size in bytes accounting for the virtualization
	 * budget. Equivalent to GetMemorySizeInBytes() unless this mip is
	 * virtualized, in which case the size shrinks from
	 * (WorstCaseTiles * AllPlaneBytes) to (BudgetedSlots * AllPlaneBytes).
	 */
	SIZE_T GetBudgetedMemorySizeInBytes() const
	{
		if (Layout == ETmvMediaFrameBufferLayout::Tiled)
		{
			return GetAllPlaneMemorySizeInBytes() * static_cast<SIZE_T>(GetBudgetedSlotCount());
		}
		return GetMemorySizeInBytes();
	}

	/**
	 * Returns the maximum slot count the buffer can grow to for this mip,
	 * given the r.TmvMedia.MipBudgetMax CVar and the mip's worst-case tile
	 * count. When grow-on-demand is disabled this matches GetBudgetedSlotCount()
	 * (no growth possible). When MipBudgetMax is 0 (default), the ceiling is
	 * the worst-case tile count - past which there's nothing to virtualize.
	 */
	int32 GetMaxGrowSlotCount() const
	{
		const int32 WorstCase = GetWorstCaseTileCount();
		if (Layout != ETmvMediaFrameBufferLayout::Tiled || !UE::TmvMedia::IsVirtualizationEnabled())
		{
			return WorstCase;
		}
		if (!UE::TmvMedia::IsGrowOnDemandEnabled())
		{
			// No growth: ceiling == initial budget.
			return GetBudgetedSlotCount();
		}
		const int32 CVarCap = UE::TmvMedia::GetMipBudgetMax();
		const int32 EffectiveCap = (CVarCap > 0) ? FMath::Min(CVarCap, WorstCase) : WorstCase;
		return FMath::Max(GetBudgetedSlotCount(), EffectiveCap);
	}

	/**
	 * Finds the plane index for the given component.
	 * @param InComponentIndex Component index to find.
	 * @param OutComponentIndexInPlane If specified, this will be populated with the index of the specified component within the plane.
	 * @return plane index or INDEX_NONE if error.
	 */
	int32 GetPlaneIndexForComponent(int32 InComponentIndex, int32* OutComponentIndexInPlane = nullptr) const
	{
		if (InComponentIndex >= 0)
		{
			int32 ComponentCount = 0;
			for (int32 PlaneIndex = 0; PlaneIndex < Planes.Num(); ++PlaneIndex)
			{
				if (InComponentIndex < ComponentCount + Planes[PlaneIndex].NumComponents)
				{
					if (OutComponentIndexInPlane)
					{
						*OutComponentIndexInPlane = InComponentIndex - ComponentCount;
					}
					return PlaneIndex;
				}
				ComponentCount += Planes[PlaneIndex].NumComponents;
			}
		}
		return INDEX_NONE;
	}
};

#undef UE_API