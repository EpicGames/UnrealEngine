// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"

/**
 * Track type for muxer/demuxer tracks.
 */
enum class ETmvMediaTrackType : uint8
{
	Unspecified,
	Video,
	Audio
};

/**
 * Common descriptor for a media track, shared by muxer and demuxer.
 * Describes the codec, format, and media-type-specific properties of a track.
 */
struct FTmvMediaTrackInfo
{
	/** Track type. */
	ETmvMediaTrackType TrackType = ETmvMediaTrackType::Unspecified;

	/**
	 * FOURCC sample entry format identifying the codec.
	 * Ex: 'apv1' for APV, 'avc1' for H.264, 'hev1' for HEVC.
	 */
	uint32 SampleEntryFormat = 0;

	/**
	 * Number of ticks per second for this track's timing.
	 * All DTS, PTS, and duration values are measured in this timescale.
	 */
	uint32 Timescale = 0;

	/**
	 * If every sample has the same duration, this holds the constant value in timescale units.
	 * 0 means variable or unknown duration.
	 * Frame rate can be computed as FFrameRate(Timescale, ConstantSampleDuration).
	 */
	uint32 ConstantSampleDuration = 0;

	/** Display width (video tracks only). */
	uint32 DisplayWidth = 0;

	/** Display height (video tracks only). */
	uint32 DisplayHeight = 0;

	/** Sampling rate in Hz (audio tracks only). */
	uint32 SamplingRate = 0;

	/** Number of channels (audio tracks only). */
	uint16 NumberOfChannels = 0;

	/**
	 * Codec-specific configuration boxes, pre-built as raw box data.
	 * Each entry is a complete box (size + type + payload).
	 */
	TArray<TArray<uint8>> CodecSpecificBoxes;

	/** Optional human-readable track name. */
	FString TrackName;
};

/**
 * Common result code for container muxer/demuxer operations.
 */
enum class ETmvMediaContainerResult
{
	Success,
	Fail,
	EndOfStream
};

/**
 * Common sample metadata shared by muxer and demuxer sample structs.
 */
struct FTmvMediaTrackSampleInfo
{
	/** Presentation duration of this sample, in the track's timescale units. */
	uint32 Duration = 0;

	/** Decode timestamp in the track's timescale units. */
	int64 DTS = 0;

	/** Presentation (composition) timestamp in the track's timescale units. */
	int64 PTS = 0;

	/** Sequential sample number (0-based). */
	uint32 SampleNumber = 0;

	/** Whether this sample is a sync sample (keyframe). */
	bool bIsKeyframe = false;
};
