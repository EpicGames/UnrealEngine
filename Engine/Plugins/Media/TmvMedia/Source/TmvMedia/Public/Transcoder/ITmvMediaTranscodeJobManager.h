// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API TMVMEDIA_API

class UTmvMediaTranscodeJob;
struct FGuid;

/**
 * Singleton manager for all transcode jobs.
 */
class ITmvMediaTranscodeJobManager
{
public:
	/** Returns a pointer to the global transcode job manager. */
	UE_API static ITmvMediaTranscodeJobManager* Get();

	/** Register transcode job to the global transcode job manager. */
	static void SafeRegisterTranscodeJob(UTmvMediaTranscodeJob* InTranscodeJob)
	{
		if (ITmvMediaTranscodeJobManager* Manager = Get())
		{
			Manager->RegisterTranscodeJob(InTranscodeJob);
		}
	}

	/** Unregister transcode job from the global transcode job manager. */
	static void SafeUnregisterTranscodeJob(UTmvMediaTranscodeJob* InTranscodeJob)
	{
		if (ITmvMediaTranscodeJobManager* Manager = Get())
		{
			Manager->UnregisterTranscodeJob(InTranscodeJob);
		}
	}

	/**
	 * Get a list of all currently registered jobs. 
	 */
	virtual void GetTranscodeJobs(TArray<TWeakObjectPtr<UTmvMediaTranscodeJob>>& OutTranscodeJobs) const = 0;

	/**
	 * Get the transcode job with the given JobId, if found.
	 */
	virtual UTmvMediaTranscodeJob* GetTranscodeJob(const FGuid& InJobId) const = 0;

	/**
	 * Register the given transcode job to this manager.
	 */
	virtual void RegisterTranscodeJob(UTmvMediaTranscodeJob* InTranscodeJob) = 0;

	/**
	 * Unregister the given transcode job from this manager.
	 */
	virtual void UnregisterTranscodeJob(UTmvMediaTranscodeJob* InTranscodeJob) = 0;

	/** Job Event Delegate. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTranscodeJobEvent, UTmvMediaTranscodeJob* /*InJob*/)

	/** Delegate called when a job is added/registered to this manager. */
	virtual FOnTranscodeJobEvent& GetOnTranscodeJobAdded() = 0;

	/** Delegate called when a job is removed/unregistered from this manager. */
	virtual FOnTranscodeJobEvent& GetOnTranscodeJobRemoved() = 0;

protected:
	virtual ~ITmvMediaTranscodeJobManager() = default;
};

#undef UE_API