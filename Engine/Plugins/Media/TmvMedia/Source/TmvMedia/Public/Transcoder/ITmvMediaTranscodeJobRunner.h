// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

#define UE_API TMVMEDIA_API

class UTmvMediaTranscodeJob;
struct FGuid;

/**
 * Fired on the game thread immediately before the runner calls Start() on the job.
 * Lets callers attach per-job state (notification handler, listeners, ...) only for jobs that
 * are actually about to run, instead of eagerly at enqueue time.
 */
DECLARE_DELEGATE_OneParam(FOnTranscodeJobStarting, UTmvMediaTranscodeJob* /*InJob*/);

/** Per-job runtime options applied when a job is enqueued in the runner. */
struct FTmvMediaTranscodeJobRunOptions
{
	/** Optional wall-clock timeout. <= 0 disables the watchdog. */
	double TimeoutSeconds = 0.0;

	/** Hook fired immediately before the runner starts the job (see FOnTranscodeJobStarting). */
	FOnTranscodeJobStarting OnPreStart;
};

/**
 * Singleton runner that owns the active transcode job and a pending queue, and ticks
 * the active job from an engine system (FTickableGameObject) so jobs progress regardless
 * of Slate window visibility.
 *
 * Sequential: at most one job runs at a time; the next is started when the current completes.
 */
class ITmvMediaTranscodeJobRunner
{
public:
	/** Returns a pointer to the global transcode job runner, if the TmvMedia module is loaded. */
	UE_API static ITmvMediaTranscodeJobRunner* Get();

	/**
	 * Enqueue a job for execution. If the runner is idle, the job is started immediately;
	 * otherwise it is queued behind the active and earlier-pending jobs. The runner takes
	 * over the job's tick / stop / discard lifecycle until completion.
	 */
	virtual void EnqueueJob(UTmvMediaTranscodeJob* InJob, const FTmvMediaTranscodeJobRunOptions& InOptions = {}) = 0;

	/** Cancel the active job and discard all pending jobs. */
	virtual void CancelAll() = 0;

	/** Cancel a specific job by id, whether it is currently active or pending. */
	virtual void CancelJob(const FGuid& InJobId) = 0;

	/** Returns true if the job with the given id is currently active or queued. */
	virtual bool IsJobActiveOrPending(const FGuid& InJobId) const = 0;

	/** Returns true if there is at least one job active or pending. */
	virtual bool HasActiveOrPendingJobs() const = 0;

	/**
	 * Delegate fired (game thread) when a job's Start() returned true and the job is now active.
	 * Not fired for jobs whose Start() failed synchronously — those go straight to OnJobFinished.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRunnerJobEvent, UTmvMediaTranscodeJob* /*InJob*/);
	virtual FOnRunnerJobEvent& GetOnJobStarted() = 0;

	/**
	 * Delegate fired (game thread) when a job leaves the runner. Fires for completed jobs,
	 * cancelled active jobs, cancelled pending jobs, and jobs whose Start() failed synchronously.
	 * Safe to access the job pointer for read-only inspection until the broadcast returns.
	 */
	virtual FOnRunnerJobEvent& GetOnJobFinished() = 0;

	/**
	 * Delegate fired (game thread) when the runner observes the last active job complete via Tick
	 * with no pending jobs left. Does NOT fire when EnqueueJob() drains the queue synchronously
	 * (e.g., a single submission whose Start() fails); callers wanting to detect that case should
	 * check HasActiveOrPendingJobs() after EnqueueJob() instead.
	 */
	DECLARE_MULTICAST_DELEGATE(FOnAllJobsFinished);
	virtual FOnAllJobsFinished& GetOnAllJobsFinished() = 0;

protected:
	virtual ~ITmvMediaTranscodeJobRunner() = default;
};

#undef UE_API
