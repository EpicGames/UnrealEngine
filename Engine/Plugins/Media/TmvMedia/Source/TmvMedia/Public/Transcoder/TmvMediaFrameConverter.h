// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transcoder/TmvMediaTranscodeStage.h"
#include "TmvMediaFrameConverter.generated.h"

#define UE_API TMVMEDIA_API

struct FTmvMediaFrameMips;

/**
 * Base class for converter stage of transcode pipeline.
 * @see UTmvMediaTranscodeJob
 */
UCLASS(MinimalAPI)
class UTmvMediaFrameConverter : public UTmvMediaTranscodeStage
{
	GENERATED_BODY()
public:
	/** Receive mips for processing. */
	UE_API virtual void ReceiveMips(UTmvMediaTranscodeJob* InParentJob, TUniquePtr<FTmvMediaFrameMips>&& InMips);
};

#undef UE_API