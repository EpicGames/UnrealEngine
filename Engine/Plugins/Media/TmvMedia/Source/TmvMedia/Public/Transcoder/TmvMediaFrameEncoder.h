// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "Transcoder/TmvMediaTranscodeStage.h"

#include "TmvMediaFrameEncoder.generated.h"

#define UE_API TMVMEDIA_API

struct FTmvMediaEncoderOptions;
struct FTmvMediaFrameMipInfo;
struct FTmvMediaFrameMips;
struct FTmvMediaFrameTimeInfo;

/**
 * Base class for encoder stage of the transcode pipeline.
 * @see UTmvMediaTranscodeJob
 */
UCLASS(MinimalAPI)
class UTmvMediaFrameEncoder : public UTmvMediaTranscodeStage
{
	GENERATED_BODY()
public:
	/**
	 * Set the encoder options. This should be done before the stage is started.
	 * @param InOptions Encoder options. This is used to create internal encoder and must not change during encoding.
	 */
	virtual void SetEncoderOptions(const TInstancedStruct<FTmvMediaEncoderOptions>& InOptions) {  }

	/**
	 * Returns the current encoder options if set or null otherwise.
	 */
	virtual const FTmvMediaEncoderOptions* GetEncoderOptions() const { return nullptr; }
	
	/**
	 * Queries the encoder for the memory layout and format for frames of the given size, and doing so for each mips.
	 * @param InParentJob Pointer to the parent job that contains the overall job settings.
	 * @param InSourceTimeInfo Source sample time info, needed for frame duration (frame rate). Encoder might need it to compute some bitrate related settings.
	 * @param InSourceMipInfo Source Mip Info from the prior stage (ex Frame Producer).
	 * @param OutFrameMipInfo Populated mip info with the memory layout the encoder needs.
	 * @return true if the requested info can be produced, or false if there is a validation error or the encoder doesn't support requested format.
	 */
	virtual bool RequestMipInfos(UTmvMediaTranscodeJob* InParentJob, const FTmvMediaFrameTimeInfo& InSourceTimeInfo, const FTmvMediaFrameMipInfo& InSourceMipInfo, TArray<FTmvMediaFrameMipInfo>& OutFrameMipInfo) 
	{
		return false;
	}

	/**
	 * Receive mips for processing.
	 * @param InParentJob Pointer to the parent job that contains the overall job settings.
	 * @param InMips Mips from prior stage. Note that ownership is transferred.
	 */
	virtual void ReceiveMips(UTmvMediaTranscodeJob* InParentJob, TUniquePtr<FTmvMediaFrameMips>&& InMips) { }
};

#undef UE_API