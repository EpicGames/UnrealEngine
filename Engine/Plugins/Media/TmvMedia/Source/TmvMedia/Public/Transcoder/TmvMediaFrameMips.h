// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/FrameRate.h"
#include "Misc/Timespan.h"
#include "SampleConverter/TmvMediaFrameMipBufferFwd.h"
#include "Templates/SharedPointer.h"

#define UE_API TMVMEDIA_API

struct FImage;

/** Time information for a frame. */
struct FTmvMediaFrameTimeInfo
{
	/** Frame index (for index based muxers). */
	int32 FrameIndex = 0;
	
	/** Zero-based Frame index (for index based muxers). */
	int32 FrameIndexNoOffset = 0;

	/** Frame time stamp (for time span based muxers). */
	FTimespan FrameTime;

	/** Frame time duration. */
	FTimespan FrameDuration;
};

/**
 * Container of an array of frame mip buffers with time info for the transcoder pipeline.
 */
struct FTmvMediaFrameMips
{
	/** Time indexing information. */
	FTmvMediaFrameTimeInfo TimeInfo;

	/** Array of mips buffers. */
	TArray<FTmvMediaFrameMipBufferHandle> MipBuffers;

	/**
	 * Helper to initialize the mip buffers from the provided mip0.
	 * @param InMip0 Image to use for mip 0. Keeps a reference to the image.
	 * @param bInGenerateMips if true, mips will be generated from mip 0.
	 */
	UE_API bool Init(const TSharedPtr<FImage>& InMip0, bool bInGenerateMips);
};

#undef UE_API