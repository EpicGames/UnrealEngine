// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "Misc/Timecode.h"
#include "Transcoder/TmvMediaTranscodeStage.h"
#include "TmvMediaFrameProducer.generated.h"

#define UE_API TMVMEDIA_API

struct FTmvMediaFrameMips;

/**
 * Track information for the currently playing tracks from the frame producer stage.
 */
USTRUCT()
struct FTmvMediaFrameProducerTrackInfo
{
	GENERATED_BODY()

	/** Track's frame rate. */
	UPROPERTY()
	FFrameRate FrameRate = FFrameRate(24, 1);

	/** Track's duration. */
	UPROPERTY()
	FTimespan Duration = FTimespan::Zero();
};

/**
 * Base class for producer stage of transcode pipeline.
 * @see UTmvMediaTranscodeJob
 */
UCLASS(MinimalAPI)
class UTmvMediaFrameProducer : public UTmvMediaTranscodeStage
{
	GENERATED_BODY()
public:
	// todo: Add Media Info Request API for UI purposes.

	/** Returns a copy of the current video track info. */
	UE_API FTmvMediaFrameProducerTrackInfo GetVideoTrackInfo() const;

	/** Set the video track info. */
	UE_API void SetVideoTrackInfo(const FTmvMediaFrameProducerTrackInfo& InVideoTrackInfo);

	/** Returns the start timecode for the current media, if available. */
	UE_API TOptional<FTimecode> GetStartTimecode() const;

	/** Returns the start timecode rate for the current media, if available. */
	UE_API TOptional<FFrameRate> GetStartTimecodeRate() const;

	/** Set the start timecode for the current media. */
	UE_API void SetStartTimecode(const FTimecode& InStartTimecode);
	
	/** Set the start timecode rate for the current media. */
	UE_API void SetStartTimecodeRate(const FFrameRate& InStartTimecodeRate);

protected:
	/** Track Info critical section. */
	mutable FCriticalSection TrackInfoLock;

	/** Information about the currently playing video track. */
	FTmvMediaFrameProducerTrackInfo VideoTrackInfo;
	
	/** The current media's start timecode, if available. */
	TOptional<FTimecode> StartTimecode;
	
	/** The current media's start timecode rate, if available. */
	TOptional<FFrameRate> StartTimecodeRate;
};

#undef UE_API