// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "Transcoder/TmvMediaTranscodeStage.h"
#include "UObject/SoftObjectPtr.h"

#include "TmvMediaTranscodeJob.generated.h"

#define UE_API TMVMEDIA_API

class UMediaSource;

/**
 * Interface for the notifications.
 * This can be implemented by a slate notification or logs.
 * @remark Important: the implementations must be thread safe. Notifications may be called from worker threads.
 */
class ITmvMediaTranscodeNotification
{
public:
	virtual ~ITmvMediaTranscodeNotification() = default;

	/** Sets the text to be displayed in the notification. */
	virtual void SetText(const FText& InText) = 0;

	/** Close the notification. */
	virtual void Close(bool bInSuccess) = 0;
};

/**
 * Selection of the input source/
 */
UENUM(BlueprintType)
enum class ETmvMediaTranscodeInputSource : uint8
{
	/** External media file in any of the supported format. */
	File,
	/** Existing media source asset in the project. */
	MediaSource
};

/**
 * Selection of the output format.
 */
UENUM(BlueprintType)
enum class ETmvMediaTranscodeOutputFormat : uint8
{
	/** Output each access unit as a separate file. */
	FileSequence UMETA(DisplayName="TMV Image Sequence"),
	/** Mux access units into a single container file. */
	Container UMETA(DisplayName="TMV Video File"),
};

/** Structure for muxer settings used to select the muxer. */
USTRUCT(BlueprintType)
struct FTmvMediaTranscodeMuxerSettings
{
	GENERATED_BODY()

	/**
	 * The name of the selected muxer
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FName Name;
};

/**
 * Common Transcode Job Settings
 */
USTRUCT(BlueprintType)
struct FTmvMediaTranscodeJobSettings
{
	GENERATED_BODY()

	/** Returns the input url (either media source or file). */
	UE_API FString GetInputPath() const;

	/** Returns the absolute file path of the media. */
	UE_API FString GetAbsoluteInputPath() const;

	/** Returns true if the input path is set. */
	UE_API bool IsInputPathSet() const;

	/** 
	 * Returns the input frame rate setting. 
	 * @remark will return the frame rate from the media source if specified.
	 */
	UE_API FFrameRate GetInputFramerate() const;

	/** Returns true if the output path is set. */
	UE_API bool IsOutputPathSet() const;

	/** Returns the absolute output directory path. */ 
	UE_API FString GetAbsoluteOutputPath() const;

	/** Select the input source. */
	UPROPERTY(EditAnywhere, Category = "Input Sequence", meta = (SegmentedDisplay))
	ETmvMediaTranscodeInputSource InputSource = ETmvMediaTranscodeInputSource::File;
	
	/**
	 * The directory that contains the source media (image sequence files or container).
	 */
	UPROPERTY(EditAnywhere, Category = "Input Sequence", meta = (EditCondition = "InputSource ==  ETmvMediaTranscodeInputSource::File", EditConditionHides))
	FFilePath InputPath;

	/**
	 * Allow to specify a frame rate if the media sources don't provide it.
	 * Remark: Overriding the source frame rate is not possible with the current implementation.
	 */
	UPROPERTY(EditAnywhere, Category = "Input Sequence", meta = (EditCondition = "InputSource ==  ETmvMediaTranscodeInputSource::File", EditConditionHides))
	FFrameRate FrameRate = FFrameRate(24, 1);

	/**
	 * Specify an existing media source as the input.
	 *
	 * @remark Restricted to UFileMediaSource and UImgMediaSource: these are the only source types the
	 * transcoder pipeline can consume.
	 */
	UPROPERTY(EditAnywhere, Category = "Input Sequence", meta = (EditCondition = "InputSource ==  ETmvMediaTranscodeInputSource::MediaSource", EditConditionHides, AllowedClasses = "/Script/MediaAssets.FileMediaSource,/Script/ImgMedia.ImgMediaSource"))
	TSoftObjectPtr<UMediaSource> InputMediaSource;

	UPROPERTY(EditAnywhere, Category = "Input Sequence", AdvancedDisplay, meta = (InlineEditConditionToggle))
	bool bEnableStartTimecodeOverride = false;

	/** Specify a start time code for the input sequence. This will override the start time code from the input media. */
	UPROPERTY(EditAnywhere, Category = "Input Sequence", AdvancedDisplay, meta = (EditCondition = "bEnableStartTimecodeOverride"))
	FTimecode StartTimecodeOverride;

	/** Select the output format. */
	UPROPERTY(EditAnywhere, Category = "Output Sequence", meta = (SegmentedDisplay))
	ETmvMediaTranscodeOutputFormat OutputFormat = ETmvMediaTranscodeOutputFormat::Container;
	
	UPROPERTY(EditAnywhere, Category = "Output Sequence", AdvancedDisplay, meta = (EditCondition = "OutputFormat == ETmvMediaTranscodeOutputFormat::Container", EditConditionHides))
	FTmvMediaTranscodeMuxerSettings Muxer;

	/** The directory to output the processed image sequence files to. */
	UPROPERTY(EditAnywhere, Category = "Output Sequence")
	FDirectoryPath OutputPath;

	/** Specify the output base name. Supports {frame_number} token. */
	UPROPERTY(EditAnywhere, Category = "Output Sequence")
	FString OutputBaseName;

	/** Specify the zero padding on the frame number in the output sequence files. */
	UPROPERTY(EditAnywhere, Category = "Output Sequence", AdvancedDisplay, meta=(ClampMin=0, UIMin=0, EditCondition = "OutputFormat == ETmvMediaTranscodeOutputFormat::FileSequence", EditConditionHides))
	int32 ZeroPadFrameNumbers = 5;

	/** If enabled, a media source asset will be made for the output sequence. */
	UPROPERTY(EditAnywhere, Category = "Output Sequence")
	bool bMakeOutputAsset = false;

	/**
	 * Destination content directory for the MediaSource asset to create or update (e.g. /Game/MediaSources) for the output sequence.
	 * The asset name itself will be the current job name.
	 */
	UPROPERTY(EditAnywhere, Category = "Output Sequence", meta = (EditCondition = "bMakeOutputAsset", EditConditionHides, ContentDir))
	FDirectoryPath OutputAssetDirectory;

	/**
	 * If checked, mip maps will be generated and encoded in the final destination.
	 * The encoder may restrict the generated mip chain depending on the format requirements.
	 */
	UPROPERTY(EditAnywhere, Category = "Frame Producer")
	bool bEnableMipMapping = true;

	/** Use a media player to read the source media. */
	UPROPERTY(EditAnywhere, Category = "Frame Producer")
	bool bUseMediaPlayer = true;

	/** Number of threads for the frame producer pipeline. */
	UPROPERTY(EditAnywhere, Category = "Frame Producer")
	int32 NumProducerThreads = 0;
};

/** Job event enumeration used in delegates (for UI refresh mostly) */
enum class ETmvMediaTranscodeJobEvent : uint8
{
	JobStarted,
	JobStopped,
	StageStarted,
	StageStopped,
};

/** Transcode Job Status. */
enum class ETmvMediaTranscodeJobStatus : int8
{
	None,
	Running,
	Stopping,
	Stopped
};

/**
 * Information about the reason for job to be stopped.
 * This is used for monitoring the job's state.
 */
UENUM(BlueprintType)
enum class ETmvMediaTranscodeJobStopReason : uint8
{
	None,
	Completed,
	Cancelled,
	Discarded
};

USTRUCT()
struct FTmvMediaTranscodingJobStats
{
	GENERATED_BODY()	
	
	/** Absolute start time. Allows to measure how long it has been running. */
	UPROPERTY()
	double StartTime = 0.0;
	
	/** Absolute stop time. Allows to measure how long it has been running. Only valid once the job is stopped. */
	UPROPERTY()
	double StopTime = 0.0;

	/** Current processed frame. */
	UPROPERTY()
	int32 ProcessedFrame = 0;

	/** Total number of frames to process. */
	UPROPERTY()
	int32 TotalFramesToProcess = 0;
};

/**
 * The transcode job implements a staged transcoding pipeline.
 * 
 * The stages are pre-determined and described as follows (in that order):
 * - FrameProducer: media source, produces the frames that are pushed to the frame converter.
 * - FrameConverter: Bridges the producer and encoder. It will inspect the needed format for encoder and produce the appropriate conversion.
 * - FrameEncoder: Receives frames ready (in the right format) to be encoded by a tmv encoder. It will query the muxer stage to get an access unit.
 * - FrameMuxer: Final stage of the pipeline that will typically write to file or media container.
 *
 * There can only be one stage of each class per pipeline and the order is fixed as listed above.
 *
 * Todo - Audio and other streams:
 * The current implementation only deals with a single video stream.
 * For audio streams, we will need to make the pipeline more complex and deal with different streams into a common muxer stage.
 */
UCLASS(MinimalAPI)
class UTmvMediaTranscodeJob : public UObject
{
	GENERATED_BODY()
public:
	UE_API UTmvMediaTranscodeJob();
	
	/** Update the id of a job. */
	UE_API void SetId(const FGuid& InId);

	/** Returns Job Id. */
	const FGuid& GetId() const
	{
		return Id;
	}
	
	/**
	 * Start the job.
	 * @param InCurrentTime Absolute application time
	 */
	UE_API bool Start(double InCurrentTime);

	/**
	 * Request Stop (cancel) the job.
	 * This will request a stop for all stages and return immediately.
	 * The async tasks will not stop immediately. The job should keep ticking until then.
	 * 
	 * @param InCurrentTime Absolute application time
	 * @param InReason Indicate the reason for stopping (completed, cancelled, etc)
	 */ 
	UE_API void RequestStop(double InCurrentTime, ETmvMediaTranscodeJobStopReason InReason);

	/**
	 * When the job is done and will not be used anymore, explicitly discarding it
	 * to remove it from the job manager.
	 * 
 	 * @param InCurrentTime Absolute application time
	 */
	UE_API void Discard(double InCurrentTime);

	/** Tick called in the main thread. Performs sync tasks. */
	UE_API void Tick(const FTmvMediaTranscodeJobTime& InTime);

	/** Determine the global status of the jobs to see if it is completed or not. */
	UE_API bool IsCompleted() const;

	/** Returns true if all the stages are stopped. */
	UE_API bool AreStagesStopped() const;

	/** Returns true if overall job status is running. */
	bool IsRunning() const
	{
		return JobStatus == ETmvMediaTranscodeJobStatus::Running;
	}
	
	/**
	 * Get the specified stage from the stage class.
	 * There is only one stage of each type in the pipeline.
	 */
	template <typename TStageClass>
	TStageClass* GetStage() const
	{
		static_assert(TIsDerivedFrom<TStageClass, UTmvMediaTranscodeStage>::IsDerived, "TSubsystemClass must be derived from TBaseType");
		for (UTmvMediaTranscodeStage* Stage : Stages)
		{
			if (Stage->GetClass()->IsChildOf(TStageClass::StaticClass()))
			{
				return static_cast<TStageClass*>(Stage);
			}
		}
		return nullptr;
	}

	/** Add the specified stage to the pipeline. */
	bool AddStage(UTmvMediaTranscodeStage* InStage)
	{
		if (InStage)
		{
			// TODO: make sure we are only adding 1 stage of each primary class.
			Stages.Add(InStage);
			return true;
		}
		return false;
	}

	/** (Optional) Set a notification handler that will be used for progress update display. */
	void SetNotificationHandler(const TSharedPtr<ITmvMediaTranscodeNotification>& InNotification)
	{
		Notification = InNotification;
	}

	/**
	 * Called by the processing stages in case a fatal error requires the job to stop.
	 * Checks if the job is valid before reporting the error.
	 */
	static void SafeReportError(UTmvMediaTranscodeJob* InJob, const FText& InErrorMessage)
	{
		if (IsValid(InJob))
		{
			const TSharedPtr<ITmvMediaTranscodeNotification> Notification = InJob->Notification;
			InJob->bIsError = true;

			if (!InErrorMessage.IsEmpty() && Notification.IsValid())
			{
				Notification->SetText(InErrorMessage);
			}
		}
	}

	/** Job Event Delegate. */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnTranscodeJobEvent, UTmvMediaTranscodeJob* /*InJob*/, ETmvMediaTranscodeJobEvent /*InEvent*/, UTmvMediaTranscodeStage* /*InStage*/)
	FOnTranscodeJobEvent& GetOnTranscodeJobEvent()
	{
		return TranscodeJobEventDelegate;
	}

	/** Job Finished Delegate. */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnTranscodeJobFinished, UTmvMediaTranscodeJob* /*InJob*/)
	FOnTranscodeJobFinished& GetOnTranscodeJobFinished()
	{
		return TranscodeJobFinishedDelegate;
	}

	/** Helper function to broadcast a job event. */
	static void SafeBroadcastJobEvent(UTmvMediaTranscodeJob* InJob, ETmvMediaTranscodeJobEvent InEvent, UTmvMediaTranscodeStage* InStage)
	{
		if (IsValid(InJob))
		{
			InJob->TranscodeJobEventDelegate.Broadcast(InJob, InEvent, InStage);
		}
	}

	/** Helper function to broadcast job finished event. */
	static void SafeBroadcastJobFinished(UTmvMediaTranscodeJob* InJob)
	{
		if (IsValid(InJob))
		{
			InJob->TranscodeJobFinishedDelegate.Broadcast(InJob);
		}
	}

	/** Helper function to update the job progress stats. */
	static void SafeUpdateProgress(UTmvMediaTranscodeJob* InJob, int32 InCurrentFrame, int32 InTotalFrames)
	{
		if (IsValid(InJob))
		{
			InJob->UpdateProgress(InCurrentFrame, InTotalFrames);
		}
	}

	/** Update job progress stats. */
	UE_API void UpdateProgress(int32 InCurrentFrame, int32 InTotalFrames);

	/** Returns a copy of job stats (thread safe). */
	UE_API FTmvMediaTranscodingJobStats GetJobStats() const;

	//~ Begin UObject
	UE_API virtual void BeginDestroy() override;
	//~ End UObject
	
private:
	/** Called when the job stopped (all stages have stopped). */
	void OnStopped(double InCurrentTime);
	
public:
	/** Original Job Item Name. */
	FString JobName;
	
	/** Common job settings. */
	UPROPERTY()
	FTmvMediaTranscodeJobSettings Settings;

	/** Stages in this pipeline. */
	UPROPERTY()
	TArray<TObjectPtr<UTmvMediaTranscodeStage>> Stages;	// Might use a map lookup with class
	
	/**
	 * Overall job state.
	 * It is driven by control functions like Start, Stop and then synchronized with the
	 * stage's state in the Tick function.
	 */
	std::atomic<ETmvMediaTranscodeJobStatus> JobStatus = ETmvMediaTranscodeJobStatus::None;

	/** Indicate a stage had an error somewhere that requires the job to stop. */
	std::atomic<bool> bIsError = false;

	/** Keep track of the stopping reason. This is mostly used for UI and logging. */
	ETmvMediaTranscodeJobStopReason StopReason = ETmvMediaTranscodeJobStopReason::None;

protected:
	/** Notification object that lets the job propagate notifications to UI. */
	TSharedPtr<ITmvMediaTranscodeNotification> Notification;

	FOnTranscodeJobFinished TranscodeJobFinishedDelegate;
	FOnTranscodeJobEvent TranscodeJobEventDelegate;

	/** Job Stats critical section. */
	mutable FCriticalSection JobStatsLock;

	/** Job statistics. */
	FTmvMediaTranscodingJobStats JobStats;
	
private:
	/** 
	 * Unique Id of this job item (from a corresponding job list). 
	 * Used to find the executing job in the job manager (local thread, remote process).  
	 */
	UPROPERTY()
	FGuid Id;
};

#undef UE_API