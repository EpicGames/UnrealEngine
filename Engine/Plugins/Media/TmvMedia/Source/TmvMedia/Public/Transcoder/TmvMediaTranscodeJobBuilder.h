// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectPtr.h"

#define UE_API TMVMEDIA_API

class FTmvMediaMessageContext;
class UObject;
class UTmvMediaFrameConverter;
class UTmvMediaFrameEncoder;
class UTmvMediaFrameProducer;
class UTmvMediaTranscodeJob;
class UTmvMediaTranscodeMuxer;
struct FTmvMediaTranscodeListItem;

/** Delegate invoked on a fully-assembled job just before the builder returns it (Start has not been called). */
DECLARE_DELEGATE_OneParam(FOnTranscodeJobBuilt, UTmvMediaTranscodeJob* /*Job*/);

/**
 * Fluent builder for UTmvMediaTranscodeJob.
 *
 * Assembles the standard pipeline stages (frame producer, frame converter, frame encoder, muxer)
 * from a transcode list item. Each stage class can be overridden independently; omitted overrides
 * pick sensible defaults based on the item's settings:
 *   - Frame producer : UTmvMediaPlayerFrameProducer or UTmvMediaFileSequenceFrameProducer based on
 *                      Settings.bUseMediaPlayer.
 *   - Frame converter: UTmvMediaFrameConverter.
 *   - Frame encoder  : UTmvMediaTmvFrameEncoder. Receives the item's EncoderOptions.
 *   - Muxer          : UTmvMediaContainerTranscodeMuxer when Settings.OutputFormat == Container,
 *                      UTmvMediaFileSequenceTranscodeMuxer otherwise.
 *
 * A post-build hook can be registered to run additional configuration after every stage has been
 * created and the encoder options applied (e.g. setting track info on the producer, adding
 * editor-only extension tasks).
 *
 * The builder captures the provided job item by reference; callers must keep it alive until Build() returns.
 * Binding to a temporary is rejected at compile time (see the deleted rvalue constructor overload below).
 * The returned job is NOT started.
 *
 * Example:
 * @code
 *   UTmvMediaTranscodeJob* Job = FTmvMediaTranscodeJobBuilder(Item)
 *       .WithFrameProducerClass(UTmvMediaFrameProducer::StaticClass())
 *       .OnPostBuild(FOnTranscodeJobBuilt::CreateLambda([Rate](UTmvMediaTranscodeJob* Job)
 *       {
 *           if (UTmvMediaFrameProducer* P = Job->GetStage<UTmvMediaFrameProducer>())
 *           {
 *               FTmvMediaFrameProducerTrackInfo Info; Info.FrameRate = Rate;
 *               P->SetVideoTrackInfo(Info);
 *           }
 *       }))
 *       .Build();
 * @endcode
 */
class FTmvMediaTranscodeJobBuilder
{
public:
	/** Construct a builder for the given job item. The item must outlive Build(). */
	UE_API explicit FTmvMediaTranscodeJobBuilder(const FTmvMediaTranscodeListItem& InJobItem);

	/** The builder stores InJobItem by reference, so a temporary would dangle immediately. Reject at compile time. */
	FTmvMediaTranscodeJobBuilder(FTmvMediaTranscodeListItem&&) = delete;

	/** Outer for the created job. Defaults to the transient package when not set. */
	UE_API FTmvMediaTranscodeJobBuilder& WithOuter(UObject* InOuter);

	/** Override the frame producer class. */
	UE_API FTmvMediaTranscodeJobBuilder& WithFrameProducerClass(TSubclassOf<UTmvMediaFrameProducer> InClass);

	/** Override the frame converter class. */
	UE_API FTmvMediaTranscodeJobBuilder& WithFrameConverterClass(TSubclassOf<UTmvMediaFrameConverter> InClass);

	/** Override the frame encoder class. */
	UE_API FTmvMediaTranscodeJobBuilder& WithFrameEncoderClass(TSubclassOf<UTmvMediaFrameEncoder> InClass);

	/** Override the muxer class. */
	UE_API FTmvMediaTranscodeJobBuilder& WithMuxerClass(TSubclassOf<UTmvMediaTranscodeMuxer> InClass);

	/** Register a hook invoked on the assembled job before Build returns. */
	UE_API FTmvMediaTranscodeJobBuilder& OnPostBuild(FOnTranscodeJobBuilt InDelegate);

	/**
	 * Instantiate the job and return it. Does not call Start.
	 * @param OutMessageContext Optional message context to accumulate build errors.
	 * @return Instantiated job or null if there is a build error.
	 */
	[[nodiscard]] UE_API UTmvMediaTranscodeJob* Build(FTmvMediaMessageContext* OutMessageContext = nullptr) const;

private:
	const FTmvMediaTranscodeListItem& JobItem;
	TObjectPtr<UObject> Outer;

	/** Caller override for the frame producer stage class. When null, Build picks UTmvMediaPlayerFrameProducer
	 *  or UTmvMediaFileSequenceFrameProducer based on JobItem.Settings.bUseMediaPlayer. */
	TSubclassOf<UTmvMediaFrameProducer> FrameProducerClass;

	/** Caller override for the frame converter stage class. When null, Build uses UTmvMediaFrameConverter. */
	TSubclassOf<UTmvMediaFrameConverter> FrameConverterClass;

	/** Caller override for the frame encoder stage class. When null, Build uses UTmvMediaTmvFrameEncoder. */
	TSubclassOf<UTmvMediaFrameEncoder> FrameEncoderClass;

	/** Caller override for the muxer stage class. When null, Build picks UTmvMediaContainerTranscodeMuxer or
	 *  UTmvMediaFileSequenceTranscodeMuxer based on JobItem.Settings.OutputFormat. */
	TSubclassOf<UTmvMediaTranscodeMuxer> MuxerClass;

	/** Delegate fired on the assembled job just before Build returns. Used by callers to perform extra wiring
	 *  (e.g. SetVideoTrackInfo on the producer, adding editor-only extension tasks) on top of the standard pipeline. */
	FOnTranscodeJobBuilt PostBuildDelegate;
};

#undef UE_API
