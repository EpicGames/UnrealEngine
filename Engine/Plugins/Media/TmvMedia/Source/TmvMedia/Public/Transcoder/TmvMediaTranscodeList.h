// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TmvMediaTranscodeJob.h"
#include "Encoder/TmvMediaEncoderOptions.h"
#include "StructUtils/InstancedStruct.h"

#include "TmvMediaTranscodeList.generated.h"

#define UE_API TMVMEDIA_API

/** 
 * Transcode Job item
 * Holds all the settings to be able to execute a transcode job.
 */
USTRUCT(BlueprintType)
struct FTmvMediaTranscodeListItem
{
	GENERATED_BODY()

public:
	/** 
	 * Unique Id of this job item. 
	 * Used to find the executing job in the job manager (local or external, etc).  
	 */
	UPROPERTY()
	FGuid Id;

	/** Job name for display purposes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Media")
	FString Name;

	/** Common job settings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Media")
	FTmvMediaTranscodeJobSettings Settings;

	/** Encoder specific settings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Media")
	TInstancedStruct<FTmvMediaEncoderOptions> EncoderOptions;
};

/** Transcode list editing events */
enum class ETmvMediaTranscodeListItemEventType
{
	/** Unspecified event type. */
	None,
	/** Item(s) added to the list. */
	ItemsAdded,
	/** Item(s) removed from the list. */ 
	ItemsRemoved,
	/** Item(s) reordered in the list. */ 
	ItemsReordered,
	/** Item(s) properties modified. */
	ItemsModified,
};

/** Transcode list editing event arguments */
struct FTmvMediaTranscodeListItemEventArgs
{
	ETmvMediaTranscodeListItemEventType Type = ETmvMediaTranscodeListItemEventType::None;
	TArray<int32, TInlineAllocator<1>> ItemIndices;
};

/**
 * Transcode Job (settings) List
 *
 * This object is a container for transcode job settings.
 */
UCLASS(MinimalAPI, NotBlueprintable, BlueprintType)
class UTmvMediaTranscodeList : public UObject
{
	GENERATED_BODY()
	
public:
	UE_API static FTmvMediaTranscodeListItem InvalidItem;

	/** Returns the number of job items in the list. */	
	int32 GetNumItems() const
	{
		return Items.Num();
	}

	/** Returns true if the given index is a valid item. */
	bool IsValidItemIndex(int32 InItemIndex) const
	{
		return Items.IsValidIndex(InItemIndex);
	}

	/** Returns the job item at given index. */
	const FTmvMediaTranscodeListItem& GetItem(int32 InItemIndex) const
	{
		return Items.IsValidIndex(InItemIndex) ? Items[InItemIndex] : InvalidItem;
	}

	/** Returns the job item at given index. */
	FTmvMediaTranscodeListItem* GetItemMutable(int32 InItemIndex)
	{
		return Items.IsValidIndex(InItemIndex) ? &Items[InItemIndex] : nullptr;
	}

	/**
	 * Insert a new item or duplicate provided one at the given index in the list.
	 * @param InItemIndex Index in the list where to insert the new item.
	 * @param InItem Item to be duplicated if provided.
	 */
	UE_API void InsertItemAt(int32 InItemIndex, const FTmvMediaTranscodeListItem* InItem = nullptr);

	/** Remove item at the given index. */
	UE_API bool RemoveItemAt(int32 InItemIndex);

	/** Remove items at given indices. */
	UE_API void RemoveItems(TConstArrayView<int32> InItemIndices);

	/**
	 * Reorder an item from the given source index to the destination index.
	 * @remark This function will broadcast the FOnItemEvent.
	 * @return true if the operation completed, false if the indices where not valid.
	 */
	UE_API bool ReorderItems(int32 InFromIndex, int32 InToIndex);

	/**
	 * Returns true if the given job item has the minimum configuration required to be executed.
	 * Checks that the input/output paths are set and that bMakeOutputAsset is consistent with OutputAssetDirectory.
	 *
	 * @param InJobItem Item to validate.
	 * @param OutError  Optional. When the item is invalid, receives a newline-separated list of reasons.
	 * @return true if the item is valid.
	 */
	UE_API static bool ValidateJobItem(const FTmvMediaTranscodeListItem& InJobItem, FString* OutError = nullptr);

	/** Delegate for item editing events. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnItemEvent, const UTmvMediaTranscodeList* /*List*/, const FTmvMediaTranscodeListItemEventArgs&);
	FOnItemEvent& GetOnItemEvent()
	{
		return OnItemEvent;
	}
	
	//~ Begin UObject
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditUndo() override;
#endif
	//~ End UObject

private:
	/** Generates a unique new item name of the form "JobItem_N". */
	FString GenerateNewItemName() const;

	/** Generates a unique duplicate name for the given source name, e.g. "Foo_Dup", "Foo_Dup2". */
	FString GenerateDuplicateName(const FString& InSourceName) const;

	/** Main list of all job items. */
	UPROPERTY()
	TArray<FTmvMediaTranscodeListItem> Items;

	/** Delegate for item editing events. */
	FOnItemEvent OnItemEvent;
};

#undef UE_API