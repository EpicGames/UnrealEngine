// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Transcoder/TmvMediaTranscodeStage.h"

#include "TmvMediaTranscodeMuxer.generated.h"

#define UE_API TMVMEDIA_API

class ITmvMediaMuxerFactory;
struct FTmvMediaFrameTimeInfo;

/**
 * Abstract base class for muxer stage of transcode pipeline.
 * @see UTmvMediaTranscodeJob
 */
UCLASS(MinimalAPI, Abstract)
class UTmvMediaTranscodeMuxer : public UTmvMediaTranscodeStage
{
	GENERATED_BODY()
public:
	/**
	 * Opens a stream.
	 *
	 * @param InParentJob Parent Job this stage is part of.
	 * @param InStreamName Name given to the stream (will be used for filename prefix in case of file sequence muxer).
	 * @param InExtension Extension for the filename in case of file sequence muxer.
	 * @return Index of the stream or INDEX_NONE in case of error.
	 */
	virtual int32 OpenStream(UTmvMediaTranscodeJob* InParentJob, const FString& InStreamName, const FString& InExtension) PURE_VIRTUAL(UTmvMediaTranscodeMuxer::OpenStream, return INDEX_NONE;);

	/**
	 * Push an access unit buffer in the stream at the given index (must call OpenStream first).
	 */
	virtual void ReceiveAccessUnit(
		UTmvMediaTranscodeJob* InParentJob,
		int32 InStreamId,
		const FTmvMediaFrameTimeInfo& TimeInfo,
		TSharedPtr<TArray64<uint8>>&& InAccessUnit) PURE_VIRTUAL(UTmvMediaTranscodeMuxer::ReceiveAccessUnit,);

	/**
	 * Utility function to get the absolute container path from job settings.
	 * @param InParentJob Parent job to get settings from.
	 * @param InMuxerFactory Optional muxer factory. If not specified, the factory is retrieve from job settings.
	 * @return Absolute container path, including the extension.
	 */
	UE_API static FString GetContainerOutputFilePath(const UTmvMediaTranscodeJob* InParentJob, const TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>& InMuxerFactory = nullptr);

	/**
	 * Muxer Utility function to find a muxer factory by name.
	 * @param InFactoryName Name of the muxer factory to find.
	 * @return muxer factory if found, nullptr otherwise.
	 */
	UE_API static TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> FindMuxerFactoryByName(FName InFactoryName);

protected:
	/** Stream information */
	struct FStreamInfo
	{
		FString StreamName;
		FString Extension;
	};

	/** Array of currently opened streams. */
	TArray<FStreamInfo> Streams;

	/** Critical section for the stream array. */
	FCriticalSection StreamsCS;
};

#undef UE_API