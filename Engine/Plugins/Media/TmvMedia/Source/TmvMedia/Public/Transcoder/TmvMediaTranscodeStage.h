// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/Atomic.h"

#include "TmvMediaTranscodeStage.generated.h"

#define UE_API TMVMEDIA_API

class UTmvMediaTranscodeJob;

/** Time arguments for job management. */
struct FTmvMediaTranscodeJobTime
{
	/** Absolute application time. */
	double CurrentTime = 0.0;
	/** Time since last tick */
	float DeltaTime = 0.0f;
};

/** Transcode stage status. */
enum class ETmvMediaTranscodeStageStatus : int8
{
	Stopped,
	Started,
	Stopping
};

/**
 * Base class for a media transcoding pipeline stage.
 * @see UTmvMediaTranscodeJob
 */
UCLASS(MinimalAPI)
class UTmvMediaTranscodeStage : public UObject
{
	GENERATED_BODY()
public:
	/** Start the stage. */
	virtual bool Start(UTmvMediaTranscodeJob* InParentJob)
	{
		// Derived class must set this status appropriately.
		SetStageStatus(ETmvMediaTranscodeStageStatus::Started, InParentJob);
		return true;
	}

	/**
	 * This will request the stage to stop.
	 * The state of the stage will be set to either Stopping or Stopped.
	 * If the stage has async tasks, it will not stop immediately.
	 * The stage should be kept ticking until all async tasks stop.
	 */
	virtual void RequestStop(UTmvMediaTranscodeJob* InParentJob)
	{
		// Derived classes with async work should override and set Stopping instead.
		SetStageStatus(ETmvMediaTranscodeStageStatus::Stopped, InParentJob);
	}

	/** Tick the job in the main thread to perform the main thread coordination. */
	virtual void Tick(UTmvMediaTranscodeJob* InParentJob, const FTmvMediaTranscodeJobTime& InTime)
	{
		// By default, the stages will update their status to mirror the producer stage.
		if (!bHasAsyncQueue)
		{
			UpdateStatusFromFrameProducer(InParentJob);
		}
	}

	/** Returns the current status of the stage. */
	ETmvMediaTranscodeStageStatus GetStageStatus() const { return StageStatus.load(); }

	/** Change the status of the stage. This will propagate job events. */
	UE_API void SetStageStatus(ETmvMediaTranscodeStageStatus InStatus, UTmvMediaTranscodeJob* InParentJob);

protected:
	/** Flag indicating if the stage is implementing an async queue. */ 
	bool bHasAsyncQueue = false;
	
	/**
	 * Updates the status of this stage to mirror that of the producer stage.
	 * This is the default behavior unless a stage is implementing an async queue.
	 */
	UE_API void UpdateStatusFromFrameProducer(UTmvMediaTranscodeJob* InParentJob);
	
private:
	/** Indicate the status of the stage. */
	std::atomic<ETmvMediaTranscodeStageStatus> StageStatus = ETmvMediaTranscodeStageStatus::Stopped;
};

#undef UE_API