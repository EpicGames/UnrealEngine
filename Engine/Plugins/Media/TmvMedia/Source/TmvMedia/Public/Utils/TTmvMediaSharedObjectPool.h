// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

/**
* Pool for worker thread context objects that can be reused within the same stream
* (i.e. no options changes) to the next when available.
*/
template<class ObjectType>
class TTmvMediaSharedObjectPool : public TSharedFromThis<TTmvMediaSharedObjectPool<ObjectType>>
{
public:

	/** Auto clean up handle, puts the context back in the pool. */
	struct FHandle 
	{
		FHandle() = default;

		// Disable copy to avoid adding object back to the pool multiple times.
		FHandle(const FHandle&) = delete;
		FHandle& operator=(const FHandle&) = delete;

		FHandle(FHandle&& Other) noexcept
			: ReaderObject(MoveTemp(Other.ReaderObject))
			, PoolWeak(MoveTemp(Other.PoolWeak))
		{
		}

		FHandle& operator=(FHandle&& Other) noexcept
		{
			if (this != &Other)
			{
				Reset();
				ReaderObject = MoveTemp(Other.ReaderObject);
				PoolWeak = MoveTemp(Other.PoolWeak);
			}
			return *this;
		} 
		
		~FHandle()
		{
			Reset();
		}

		void Reset()
		{
			if (ReaderObject)
			{
				if (TSharedPtr<TTmvMediaSharedObjectPool> Pool = PoolWeak.Pin())
				{
					Pool->Release(ReaderObject);
				}
				ReaderObject.Reset();
			}
		}

		bool IsValid() const
		{
			return ReaderObject.IsValid();
		}

		ObjectType* operator->() const
		{
			return ReaderObject.Get();
		}

		TSharedPtr<ObjectType> ReaderObject;

	private:
		TWeakPtr<TTmvMediaSharedObjectPool> PoolWeak;
		friend TTmvMediaSharedObjectPool;
	};

	/**
	 * Acquire an available object from the pool or create a new one.
	 * @remark we assume this is for the same stream, no-sub pooling needed.
	 */
	FHandle Acquire(TFunction<TSharedPtr<ObjectType>()> InAllocatorFunc)
	{
		UE::TUniqueLock<UE::FMutex> Lock(FreeObjectsMutex);

		FHandle Handle;
		if (!FreeObjects.IsEmpty())
		{
			Handle.ReaderObject = FreeObjects.Top();
			FreeObjects.Pop();
		}

		if (!Handle.ReaderObject)
		{
			Handle.ReaderObject = InAllocatorFunc();
		}

		Handle.PoolWeak = this->AsWeak();
		return Handle;
	}

	/** Returns unused objects to the pool. */
	void Release(const TSharedPtr<ObjectType>& InObject)
	{
		if (InObject)
		{
			UE::TUniqueLock<UE::FMutex> Lock(FreeObjectsMutex);
			FreeObjects.Add(InObject);
		}
	}

private:
	UE::FMutex FreeObjectsMutex;
	TArray<TSharedPtr<ObjectType>> FreeObjects;
};