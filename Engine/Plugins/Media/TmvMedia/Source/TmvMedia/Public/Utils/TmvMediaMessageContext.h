// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"

#define UE_API TMVMEDIA_API

/**
 * Container for error messages that get accumulated during operations and can be compiled
 * into a report for display in UI or dumped to Logs.
 */
class FTmvMediaMessageContext
{
public:
	/** Adds the given message at the end of the report. */
	void Add(const FText& InMessage)
	{
		Messages.Add(InMessage);
	}

	/** Adds the given message at the end of the report. */
	void Add(FText&& InMessage)
	{
		Messages.Add(MoveTemp(InMessage));
	}

	/**
	 * Compiles the set of errors in a formatted report with each error message on a different line.
	 */
	UE_API FText ToText() const;
	
	/**
	 * List of error messages for the given context.
	 */
	TArray<FText> Messages;
};

/**
 * Helper macro to reduce boilerplate for message reporting.
 * In most cases, we want to both log the message in a log category and add it to the report.
 * We have to use a macro for this to preserve the file/line functionality of the logging system.
 * 
 * @param MessageContext Message Report Context to accumulate messages
 * @param LogCategory Log category to log to.
 * @param Verbosity Verbosity of the log.
 * @param SectionString Name of the section this log will be part of.
 * @param MessageText Formatted message.
 */
#define UE_TMV_MEDIA_MESSAGE_LOG(MessageContext, LogCategory, Verbosity, SectionString, MessageText) \
	{ \
		UE_LOGF(LogCategory, Verbosity, "%ls: %ls.", SectionString, *MessageText.ToString()); \
		if (MessageContext) \
		{ \
			MessageContext->Add(MoveTemp(MessageText)); \
		} \
	}

#undef UE_API
