// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/UnrealType.h"

/**
 * Collections of small utility functions to help implement Tmv related features.
 *
 * @remark Avoid requiring engine includes (if possible).
 */
struct FTmvMediaUtils
{
	/**
	 * Returns the maximum number of mips that constitute a full mip chain for the given dimensions.
	 * 
	 * @param InWidth Width in pixels of mip 0 to be considered.
	 * @param InHeight Height in of mip 0 to be considered.
	 * @return Number of mips, 0 in case of invalid calculation.
	 */
	static int32 GetMaxMipCountFromDimensions(int32 InWidth, int32 InHeight)
	{
		const int32 MaxDimension = FMath::Max(InWidth, InHeight);

		// log2 doesn't support negative values.
		if (MaxDimension < 0)
		{
			return 0;
		}

		return 1 + FMath::FloorLog2(static_cast<uint32>(MaxDimension));
	}
};

namespace UE::TmvMedia::Utils
{
	/**
	 * Converts an enum value to a string for logging purposes.
	 * 
	 * @tparam InEnumType Enum type, must be UENUM defined. 
	 * @param InValue Value
	 * @return short name of the value.
	 */
	template<typename InEnumType>
	FString StaticEnumToString(InEnumType InValue)
	{
		if (UEnum* Enum = StaticEnum<InEnumType>())
		{
			return Enum->GetNameStringByValue(static_cast<int64>(InValue));
		}
		return TEXT("<invalid-enum>");
	}

	template<typename InEnumType>
	FText StaticEnumToText(InEnumType InValue)
	{
		if (UEnum* Enum = StaticEnum<InEnumType>())
		{
			return Enum->GetDisplayNameTextByValue(static_cast<int64>(InValue));
		}
		return NSLOCTEXT("TmvMediaUtils", "InvalidEnumText", "<invalid-enum>");
	}
	
	/** 
	 * Constructs a FourCC integer from four 8-bit character codes, commonly used for 
	 * identifying file formats, codec types, or pixel formats.
	 */
	inline constexpr uint32 MakeFourCC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	/**
	 * Returns the string made out of the 4 character codes from the given FourCC value. 
	 */
	inline FString FourCCToString(uint32 InFourCC)
	{
		return FString::Printf(TEXT("%c%c%c%c"),
			static_cast<TCHAR>((InFourCC >> 24) & 0xFF),
			static_cast<TCHAR>((InFourCC >> 16) & 0xFF),
			static_cast<TCHAR>((InFourCC >> 8) & 0xFF),
			static_cast<TCHAR>(InFourCC & 0xFF));
	}
}
