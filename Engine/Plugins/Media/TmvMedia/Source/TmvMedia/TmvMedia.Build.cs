// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class TmvMedia : ModuleRules
	{
		public TmvMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"Media",
					"RHI",
					"TmvMediaShaders",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ImageCore",
					"ImageWrapper",
					"Json",
					"JsonUtilities",
					"MediaAssets",
					"MediaUtils",
					"Projects",
					"Renderer",
					"RenderCore",
				});

			// Sparse output media-texture residency uses RHI-specific extension APIs
			// (D3D12: ID3D12DynamicRHI + UpdateTileMappings on reserved resources) to
			// commit/decommit per-mip physical backing. Each platform/RHI backend
			// lives under Private/Platforms/<Platform>/ and is automatically
			// excluded by UBT on non-matching targets, so this Build.cs only needs
			// to opt in to the matching RHI module on the matching platform.
			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
				PrivateDependencyModuleNames.Add("D3D12RHI");
			}

			// CQTest is a Developer module; only link it when developer/editor tools are being built
			// so that it is never pulled into shipping or game-client binaries.
			if (Target.bBuildEditor && Target.bBuildDeveloperTools)
			{
				PrivateDependencyModuleNames.Add("CQTest");
			}
		}
	}
}
