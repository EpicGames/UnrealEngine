// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaDirectoryPathCustomization.h"

#include "ContentBrowserDataDragDropOp.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Input/DragAndDrop.h"
#include "SDropTarget.h"
#include "Styling/AppStyle.h"
#include "Utils/TmvMediaPathUtils.h"
#include "Widgets/STmvMediaDirectoryPicker.h"
#include "Widgets/TmvMediaEditorTranscodeUtils.h"

#define LOCTEXT_NAMESPACE "FTmvMediaDirectoryPathCustomization"

/* IPropertyTypeCustomization interface
 *****************************************************************************/

TSharedRef<IPropertyTypeCustomization> FTmvMediaDirectoryPathCustomization::MakeInstance()
{
	return MakeShared<FTmvMediaDirectoryPathCustomization>();
}

void FTmvMediaDirectoryPathCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FTmvMediaDirectoryPathCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PathStringProperty = InPropertyHandle->GetChildHandle("Path");

	// Support same modifiers as FDirectoryPathStructCustomization::CustomizeHeader for content browser.
	bContentDir = InPropertyHandle->HasMetaData( TEXT("ContentDir") ) || InPropertyHandle->HasMetaData(TEXT("LongPackageName"));
	const bool bForceShowPluginContent = InPropertyHandle->HasMetaData(TEXT("ForceShowPluginContent"));
	const bool bForceShowEngineContent = InPropertyHandle->HasMetaData(TEXT("ForceShowEngineContent"));

	// Add Directory picker.
	InChildBuilder.AddCustomRow(LOCTEXT("DirectoryPathLabel", "Path"))
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(0.0f)
		.MinDesiredWidth(125.0f)
		[
			SNew(SDropTarget)	// Support drag and drop of external files.
			.OnAllowDrop(this, &FTmvMediaDirectoryPathCustomization::HandleVerifyDrag)
			.OnDropped(this, &FTmvMediaDirectoryPathCustomization::HandleDropEvent)
			[
				SNew(STmvMediaDirectoryPicker)
				.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
				.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.BrowseButtonToolTip(LOCTEXT("DirectoryButtonToolTipText", "Choose a directory from this computer"))
				.BrowseDirectory_Lambda([this]() -> FString
				{
					const FString DirectoryPath = GetDirectoryPath(/*bReturnEmptyOnMultiSelect*/ true);
					return !DirectoryPath.IsEmpty() ?
						UE::TmvMedia::PathUtils::ConvertSanitizedPathToFull(DirectoryPath) : FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
				})
				.BrowseTitle(LOCTEXT("BrowseTitle", "Directory picker..."))
				.DirectoryPath(this, &FTmvMediaDirectoryPathCustomization::GetDirectoryPath, /*bReturnEmptyOnMultiSelect*/ false)
				.OnDirectoryChanged(this, &FTmvMediaDirectoryPathCustomization::HandleDirectoryPickerDirectoryChanged)
				.IsEnabled(this, &FTmvMediaDirectoryPathCustomization::IsEnabled)
				.ContentDir(bContentDir)
				.ForceShowPluginContent(bForceShowPluginContent)
				.ForceShowEngineContent(bForceShowEngineContent)
			]
		];
}

FString FTmvMediaDirectoryPathCustomization::GetDirectoryPath(bool bReturnEmptyOnMultiSelect) const
{
	FString FilePath;
	if (PathStringProperty)
	{
		if (PathStringProperty->GetValue(FilePath) == FPropertyAccess::Result::MultipleValues)
		{
			return !bReturnEmptyOnMultiSelect ? NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values").ToString() : FString();
		}
	}
	return FilePath;
}

void FTmvMediaDirectoryPathCustomization::HandleDirectoryPickerDirectoryChanged(const FString& InPickedDirectory, bool bInContentDir)
{
	if (PathStringProperty)
	{
		using namespace UE::TmvMedia::PathUtils;
		PathStringProperty->SetValue(bInContentDir ? InPickedDirectory : GetSanitizedPath(InPickedDirectory));
	}
}

bool FTmvMediaDirectoryPathCustomization::HandleVerifyDrag(TSharedPtr<FDragDropOperation> InDragOperation)
{
	if (bContentDir)
	{
		if (InDragOperation && InDragOperation->IsOfType<FContentBrowserDataDragDropOp>())
		{
			TSharedPtr<FContentBrowserDataDragDropOp> ContentDragDropOp = StaticCastSharedPtr<FContentBrowserDataDragDropOp>(InDragOperation);
			return ContentDragDropOp &&
				(!ContentDragDropOp->GetDraggedFolders().IsEmpty() ||
				 !ContentDragDropOp->GetDraggedFiles().IsEmpty() ||
				 !ContentDragDropOp->GetDraggedItems().IsEmpty());
		}
		return false;
	}

	if (InDragOperation && InDragOperation->IsOfType<FExternalDragOperation>())
	{
		TSharedPtr<FExternalDragOperation> ExternalDragOperation = StaticCastSharedPtr<FExternalDragOperation>(InDragOperation);
		return ExternalDragOperation && ExternalDragOperation->HasFiles();
	}
	return false;
}

FReply FTmvMediaDirectoryPathCustomization::HandleDropEvent(const FGeometry& InGeometry, const FDragDropEvent& InDropEvent)
{
	const TSharedPtr<FDragDropOperation> DropOperation = InDropEvent.GetOperation();

	if (bContentDir)
	{
		if (DropOperation && DropOperation->IsOfType<FContentBrowserDataDragDropOp>())
		{
			TSharedPtr<FContentBrowserDataDragDropOp> ContentDragDropOp = StaticCastSharedPtr<FContentBrowserDataDragDropOp>(DropOperation);
			if (ContentDragDropOp)
			{
				FString ContentPath;
				if (!ContentDragDropOp->GetDraggedFolders().IsEmpty())
				{
					ContentPath = ContentDragDropOp->GetDraggedFolders()[0].GetInternalPath().ToString();
				}
				else if (!ContentDragDropOp->GetDraggedFiles().IsEmpty())
				{
					ContentPath = FPaths::GetPath(ContentDragDropOp->GetDraggedFiles()[0].GetInternalPath().ToString());
				}
				else if (!ContentDragDropOp->GetDraggedItems().IsEmpty())
				{
					const FContentBrowserItem& Item = ContentDragDropOp->GetDraggedItems()[0];
					ContentPath = Item.IsFolder()
						? Item.GetInternalPath().ToString()
						: FPaths::GetPath(Item.GetInternalPath().ToString());
				}

				if (!ContentPath.IsEmpty())
				{
					HandleDirectoryPickerDirectoryChanged(ContentPath, /*bInContentDir*/true);
					return FReply::Handled();
				}
			}
		}
		return FReply::Unhandled();
	}

	if (DropOperation && DropOperation->IsOfType<FExternalDragOperation>())
	{
		TSharedPtr<FExternalDragOperation> ExternalDragOperation = StaticCastSharedPtr<FExternalDragOperation>(DropOperation);
		if (ExternalDragOperation->HasFiles())
		{
			const FString SanitizedPath = UE::TmvMedia::PathUtils::GetSanitizedPath(ExternalDragOperation->GetFiles()[0]);
			HandleDirectoryPickerDirectoryChanged(UE::TmvMedia::PathUtils::GetDirectoryFullPath(SanitizedPath), /*bInContentDir*/false);
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

bool FTmvMediaDirectoryPathCustomization::IsEnabled() const
{
	return PathStringProperty ? PathStringProperty->IsEditable() : false;
}

#undef LOCTEXT_NAMESPACE
