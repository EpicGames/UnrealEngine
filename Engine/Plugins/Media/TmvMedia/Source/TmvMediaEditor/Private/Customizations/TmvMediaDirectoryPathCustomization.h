// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"

class FDragDropOperation;
class IPropertyHandle;

/**
 * Implements a customization for the FDirectoryPath class for Tmv transcoder.
 *
 * Details:
 * - The picked path is either made absolute if outside the project or relative to the project.
 * - Implements drag and drop of files from the explorer.
 */
class FTmvMediaDirectoryPathCustomization : public IPropertyTypeCustomization
{
public:
	/** Make an instance of this property type customization. */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

private:
	/**
	 * Gets the directory path from the property handle.
	 */
	FString GetDirectoryPath(bool bReturnEmptyOnMultiSelect) const;

	/**
	 * Called when a directory is selected.
	 */
	void HandleDirectoryPickerDirectoryChanged(const FString& InPickedDirectory, bool bInContentDir);

	/**
	 * Called when a drag operation enters the widget. 
	 */
	bool HandleVerifyDrag(TSharedPtr<FDragDropOperation> InDragOperation);

	/**
	 * Called on a drag drop event.
	 */
	FReply HandleDropEvent(const FGeometry& InGeometry, const FDragDropEvent& InDropEvent);

	/** Returns true if the widget is enabled (editable). */
	bool IsEnabled() const;

	/** Pointer to the string that will be set when changing the path */
	TSharedPtr<IPropertyHandle> PathStringProperty;

	/** Indicates that the path will be picked using the Slate-style directory picker inside the game Content dir. */
	bool bContentDir = false;
};

