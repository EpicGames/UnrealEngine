// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaEncoderOptionsEnumCustomization.h"

#include "DetailWidgetRow.h"
#include "Encoder/TmvMediaEncoderOptions.h"
#include "Engine/TextureDefines.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyRestriction.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "TmvMediaEncoderOptionsEnumCustomization"

namespace UE::TmvMediaEditor::EncoderOptionsEnumCustomization::Private
{
	/** True if the given property is one of the encoder-supported enum properties (DestinationColorSpace / DestinationEncoding). */
	bool IsSupportedProperty(const FProperty* InProperty)
	{
		if (!InProperty)
		{
			return false;
		}

		static const FName DestColorSpaceName = GET_MEMBER_NAME_CHECKED(FTmvMediaEncoderOptions, DestinationColorSpace);
		static const FName DestEncodingName = GET_MEMBER_NAME_CHECKED(FTmvMediaEncoderOptions, DestinationEncoding);

		const FName PropertyName = InProperty->GetFName();
		return PropertyName == DestColorSpaceName || PropertyName == DestEncodingName;
	}

	/** Returns the UEnum behind an enum-typed property (handles both FEnumProperty and FByteProperty). */
	const UEnum* GetEnumForProperty(const FProperty* InProperty)
	{
		if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(InProperty))
		{
			return EnumProperty->GetEnum();
		}
		if (const FByteProperty* ByteProperty = CastField<const FByteProperty>(InProperty))
		{
			return ByteProperty->GetIntPropertyEnum();
		}
		return nullptr;
	}
}

bool FTmvMediaEncoderOptionsEnumIdentifier::IsPropertyTypeCustomized(const IPropertyHandle& InHandle) const
{
	return UE::TmvMediaEditor::EncoderOptionsEnumCustomization::Private::IsSupportedProperty(InHandle.GetProperty());
}

TSharedRef<IPropertyTypeCustomization> FTmvMediaEncoderOptionsEnumCustomization::MakeInstance()
{
	return MakeShared<FTmvMediaEncoderOptionsEnumCustomization>();
}

void FTmvMediaEncoderOptionsEnumCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	ApplySupportedValuesRestriction(InPropertyHandle);

	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			PropertyCustomizationHelpers::MakePropertyComboBox(InPropertyHandle)
		];
}

void FTmvMediaEncoderOptionsEnumCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder,
	IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// Enums are leaf properties — no children to customize.
}

void FTmvMediaEncoderOptionsEnumCustomization::ApplySupportedValuesRestriction(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	using namespace UE::TmvMediaEditor::EncoderOptionsEnumCustomization::Private;

	const FProperty* Property = InPropertyHandle->GetProperty();
	const UEnum* Enum = GetEnumForProperty(Property);
	if (!Property || !Enum)
	{
		return;
	}

	const TSharedPtr<IPropertyHandle> ParentHandle = InPropertyHandle->GetParentHandle();
	if (!ParentHandle.IsValid())
	{
		return;
	}

	static const FName DestColorSpaceName = GET_MEMBER_NAME_CHECKED(FTmvMediaEncoderOptions, DestinationColorSpace);
	const bool bIsColorSpace = (Property->GetFName() == DestColorSpaceName);

	TArray<int32> SupportedValues;
	ParentHandle->EnumerateConstRawData([&SupportedValues, bIsColorSpace](const void* InRawData, const int32 /*InDataIndex*/, const int32 /*InNumDatas*/) -> bool
	{
		const FTmvMediaEncoderOptions* Options = static_cast<const FTmvMediaEncoderOptions*>(InRawData);
		if (!Options)
		{
			return true;
		}

		if (bIsColorSpace)
		{
			TArray<ETextureColorSpace> Supported;
			Options->GetSupportedDestinationColorSpaces(Supported);
			for (ETextureColorSpace Value : Supported)
			{
				SupportedValues.AddUnique(static_cast<int32>(Value));
			}
		}
		else
		{
			TArray<ETmvMediaEncoderEncoding> Supported;
			Options->GetSupportedDestinationEncodings(Supported);
			for (ETmvMediaEncoderEncoding Value : Supported)
			{
				SupportedValues.AddUnique(static_cast<int32>(Value));
			}
		}
		return false;	// one encoder is enough — selected jobs share an encoder type
	});

	if (SupportedValues.IsEmpty())
	{
		return;	// Encoder declares no subset — show every enum value.
	}

	const TSharedRef<FPropertyRestriction> Restriction = MakeShared<FPropertyRestriction>(
		LOCTEXT("UnsupportedByEncoder", "Not supported by the selected encoder."));

	// NumEnums() - 1 skips the implicit trailing _MAX entry.
	for (int32 EnumIndex = 0; EnumIndex < Enum->NumEnums() - 1; ++EnumIndex)
	{
		const int32 EnumValue = static_cast<int32>(Enum->GetValueByIndex(EnumIndex));
		if (!SupportedValues.Contains(EnumValue))
		{
			Restriction->AddHiddenValue(Enum->GetNameStringByIndex(EnumIndex));
		}
	}

	InPropertyHandle->AddRestriction(Restriction);
}

#undef LOCTEXT_NAMESPACE
