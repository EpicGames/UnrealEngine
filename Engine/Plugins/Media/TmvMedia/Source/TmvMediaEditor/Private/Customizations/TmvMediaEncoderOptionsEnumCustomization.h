// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"

/**
 * Identifies the DestinationColorSpace / DestinationEncoding properties of FTmvMediaEncoderOptions so that the
 * customization below only applies to those two properties and leaves any other use of the same enums alone.
 */
class FTmvMediaEncoderOptionsEnumIdentifier : public IPropertyTypeIdentifier
{
public:
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InHandle) const override;
};

/**
 * Property type customization for the DestinationColorSpace (ETextureColorSpace) and DestinationEncoding
 * (ETmvMediaEncoderEncoding) properties on FTmvMediaEncoderOptions. Adds an FPropertyRestriction that hides
 * enum values not returned by the encoder-specific GetSupportedDestination... virtuals, then defers to the
 * default property combo box so the widget matches every other enum dropdown in the panel.
 */
class FTmvMediaEncoderOptionsEnumCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

private:
	/** Adds an FPropertyRestriction hiding every enum value not in the encoder's supported subset. */
	void ApplySupportedValuesRestriction(const TSharedRef<IPropertyHandle>& InPropertyHandle);
};
