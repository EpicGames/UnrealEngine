// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaFilePathCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Input/DragAndDrop.h"
#include "SDropTarget.h"
#include "Styling/AppStyle.h"
#include "Utils/TmvMediaPathUtils.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "Widgets/TmvMediaEditorTranscodeUtils.h"

#define LOCTEXT_NAMESPACE "FTmvMediaFilePathCustomization"

/* IPropertyTypeCustomization interface
 *****************************************************************************/

TSharedRef<IPropertyTypeCustomization> FTmvMediaFilePathCustomization::MakeInstance()
{
	return MakeShared<FTmvMediaFilePathCustomization>();
}

void FTmvMediaFilePathCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FTmvMediaFilePathCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PathStringProperty = InPropertyHandle->GetChildHandle("FilePath");
	
	// Set filter.
	FString FileTypeFilter = TEXT("All files (*.*)|*.*");

	// Add file picker.
	InChildBuilder.AddCustomRow(LOCTEXT("FilePathLabel", "Filepath"))
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(0.0f)
		.MinDesiredWidth(125.0f)
		[
			SNew(SDropTarget)	// Support drag and drop of external files.
			.OnAllowDrop(this, &FTmvMediaFilePathCustomization::HandleVerifyDrag)
			.OnDropped(this, &FTmvMediaFilePathCustomization::HandleDropEvent)
			[
				SNew(SFilePathPicker)
				.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
				.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a file from this computer"))
				.BrowseDirectory_Lambda([this]()->FString
				{
					const FString FilePath = HandleFilePathPickerFilePath(/*bReturnEmptyOnMultiSelect*/ true);
					return !FilePath.IsEmpty() ?
						UE::TmvMedia::PathUtils::GetDirectoryFullPath(FilePath) : FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
				})
				.BrowseTitle(LOCTEXT("PropertyEditorTitle", "File picker..."))
				.FilePath(this, &FTmvMediaFilePathCustomization::HandleFilePathPickerFilePath, /*bReturnEmptyOnMultiSelect*/ false)
				.FileTypeFilter(FileTypeFilter)
				.OnPathPicked(this, &FTmvMediaFilePathCustomization::HandleFilePathPickerPathPicked)
			]
		];
}

FString FTmvMediaFilePathCustomization::HandleFilePathPickerFilePath(bool bReturnEmptyOnMultiSelect) const
{
	FString FilePath;
	if (PathStringProperty)
	{
		if (PathStringProperty->GetValue(FilePath) == FPropertyAccess::Result::MultipleValues)
		{
			return !bReturnEmptyOnMultiSelect ? NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values").ToString() : FString();
		}
	}
	return FilePath;
}

void FTmvMediaFilePathCustomization::HandleFilePathPickerPathPicked(const FString& InPickedPath)
{
	using namespace UE::TmvMedia::PathUtils;

	if (!PathStringProperty)
	{
		return;
	}

	PathStringProperty->SetValue(GetSanitizedPath(InPickedPath));
}

bool FTmvMediaFilePathCustomization::HandleVerifyDrag(TSharedPtr<FDragDropOperation> InDragOperation)
{
	if (InDragOperation && InDragOperation->IsOfType<FExternalDragOperation>())
	{
		TSharedPtr<FExternalDragOperation> ExternalDragOperation = StaticCastSharedPtr<FExternalDragOperation>(InDragOperation);
		return ExternalDragOperation && ExternalDragOperation->HasFiles();
	}
	return false;
}

FReply FTmvMediaFilePathCustomization::HandleDropEvent(const FGeometry& InGeometry, const FDragDropEvent& InDropEvent)
{
	const TSharedPtr<FDragDropOperation> DropOperation = InDropEvent.GetOperation();
	if (DropOperation && DropOperation->IsOfType<FExternalDragOperation>())
	{
		TSharedPtr<FExternalDragOperation> ExternalDragOperation = StaticCastSharedPtr<FExternalDragOperation>(DropOperation);
		if (ExternalDragOperation->HasFiles())
		{
			HandleFilePathPickerPathPicked(ExternalDragOperation->GetFiles()[0]);
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}
#undef LOCTEXT_NAMESPACE
