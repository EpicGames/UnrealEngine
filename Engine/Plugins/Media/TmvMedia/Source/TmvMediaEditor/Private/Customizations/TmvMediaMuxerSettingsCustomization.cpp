// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaMuxerSettingsCustomization.h"

#include "DetailWidgetRow.h"
#include "Encoder/ITmvMediaMuxerFactory.h"
#include "IDetailChildrenBuilder.h"
#include "ITmvMediaModule.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "TmvMediaMuxerSettingsCustomization"

TSharedRef<IPropertyTypeCustomization> FTmvMediaMuxerSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FTmvMediaMuxerSettingsCustomization());
}

void FTmvMediaMuxerSettingsCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	NameProperty = InPropertyHandle->GetChildHandle(TEXT("Name"));

	RefreshMuxerOptions();

	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.0f)
		[
			SNew(SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&MuxerDisplayNames)
			.OnSelectionChanged(this, &FTmvMediaMuxerSettingsCustomization::OnMuxerSelectionChanged)
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
			{
				return SNew(STextBlock).Text(FText::FromString(*InItem));
			})
			.Content()
			[
				SNew(STextBlock).Text(this, &FTmvMediaMuxerSettingsCustomization::GetSelectedMuxerDisplayName)
			]
		];
}

void FTmvMediaMuxerSettingsCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder,
	IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// No children — the Name field is fully handled in the header combo box.
}

void FTmvMediaMuxerSettingsCustomization::RefreshMuxerOptions()
{
	MuxerDisplayNames.Empty();
	DisplayNameToFactoryName.Empty();

	ITmvMediaModule* TmvModule = ITmvMediaModule::Get();
	if (!TmvModule)
	{
		return;
	}

	TArray<TWeakPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>> MuxerFactories;
	TmvModule->GetMuxerFactories(MuxerFactories);

	for (const TWeakPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe>& FactoryWeak : MuxerFactories)
	{
		if (TSharedPtr<ITmvMediaMuxerFactory, ESPMode::ThreadSafe> Factory = FactoryWeak.Pin())
		{
			const FString DisplayName = Factory->GetDisplayName().ToString();
			MuxerDisplayNames.Add(MakeShared<FString>(DisplayName));
			DisplayNameToFactoryName.Add(DisplayName, Factory->GetName());
		}
	}

	// If the current Name property is empty and there are options, select the first one.
	if (NameProperty.IsValid() && MuxerDisplayNames.Num() > 0)
	{
		FName CurrentName;
		NameProperty->GetValue(CurrentName);
		if (CurrentName.IsNone())
		{
			NameProperty->SetValue(DisplayNameToFactoryName[*MuxerDisplayNames[0]]);
		}
	}
}

void FTmvMediaMuxerSettingsCustomization::OnMuxerSelectionChanged(TSharedPtr<FString> InNewSelection, ESelectInfo::Type InSelectInfo)
{
	if (InNewSelection.IsValid() && NameProperty.IsValid())
	{
		if (const FName* FactoryName = DisplayNameToFactoryName.Find(*InNewSelection))
		{
			NameProperty->SetValue(*FactoryName);
		}
	}
}

FText FTmvMediaMuxerSettingsCustomization::GetSelectedMuxerDisplayName() const
{
	if (!NameProperty.IsValid())
	{
		return FText::GetEmpty();
	}

	FName CurrentName;
	NameProperty->GetValue(CurrentName);

	// Find the display name for the current factory name.
	for (const auto& [DisplayName, FactoryName] : DisplayNameToFactoryName)
	{
		if (FactoryName == CurrentName)
		{
			return FText::FromString(DisplayName);
		}
	}

	// Fallback: show the raw name if not found in registered factories.
	return FText::FromName(CurrentName);
}

#undef LOCTEXT_NAMESPACE
