// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;

/**
 * Implements a property type customization for FTmvMediaTranscodeMuxerSettings.
 * The Name field is displayed as a combo box populated from registered ITmvMediaMuxerFactory instances.
 */
class FTmvMediaMuxerSettingsCustomization : public IPropertyTypeCustomization
{
public:
	/** Make an instance of this property type customization. */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

private:
	/** Refresh the list of available muxer factories. */
	void RefreshMuxerOptions();

	/** Called when the combo box selection changes. */
	void OnMuxerSelectionChanged(TSharedPtr<FString> InNewSelection, ESelectInfo::Type InSelectInfo);

	/** Returns the currently selected muxer display name. */
	FText GetSelectedMuxerDisplayName() const;

	/** Handle to the Name property of FTmvMediaTranscodeMuxerSettings. */
	TSharedPtr<IPropertyHandle> NameProperty;

	/** Available muxer option strings for the combo box (display names). */
	TArray<TSharedPtr<FString>> MuxerDisplayNames;

	/** Mapping from display name to factory FName. */
	TMap<FString, FName> DisplayNameToFactoryName;
};
