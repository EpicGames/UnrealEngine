// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Modules/ModuleInterface.h"
#include "TmvMediaEditorLog.h"
#include "TmvMediaEditorStyle.h"
#include "UObject/NameTypes.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/STmvMediaTranscoder.h"
#include "Widgets/TmvMediaTranscodeListCommands.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "TmvMediaEditorModule"

DEFINE_LOG_CATEGORY(LogTmvMediaEditor);

/**
 * Implements the TmvMediaEditor module.
 */
class FTmvMediaEditorModule	: public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		RegisterTabSpawners();
		FTmvMediaTranscodeListCommands::Register();
	}

	virtual void ShutdownModule() override
	{
		UnregisterTabSpawners();
		FTmvMediaTranscodeListCommands::Unregister();
	}
	//~ End IModuleInterface

protected:
	void RegisterTabSpawners()
	{
		// Editor modules may load in commandlets/headless contexts where Slate isn't initialized.
		if (!FSlateApplication::IsInitialized())
		{
			return;
		}

		// Register the TMV Transcoder tab under the shared Media group.
		const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TmvMediaTranscoderTabName,
			FOnSpawnTab::CreateStatic(&FTmvMediaEditorModule::SpawnTmvTranscoderTab))
			.SetGroup(MenuStructure.GetLevelEditorMediaCategory())
			.SetDisplayName(LOCTEXT("TmvMediaTranscoderTabTitle_v2", "TMV Transcoder"))
			.SetTooltipText(LOCTEXT("TmvMediaTranscoderTooltipText_v2", "Open the TMV Transcoder tab."))
			.SetIcon(FSlateIcon(FTmvMediaEditorStyle::Get().GetStyleSetName(), "TmvMediaEditor.TranscoderTabIcon"));
	}

	void UnregisterTabSpawners()
	{
		if (FSlateApplication::IsInitialized())
		{
			FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TmvMediaTranscoderTabName);
		}
	}

	static TSharedRef<SDockTab> SpawnTmvTranscoderTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		// Create tab.
		TSharedPtr< STmvMediaTranscoder> TmvTranscoder;
		TSharedRef <SDockTab> Tab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SAssignNew(TmvTranscoder, STmvMediaTranscoder)
			];

		return Tab;
	}
	
	/**
	 * Call this to open the TMV Transcoder tab.
	 */
	static void OpenTmvTranscoderTab()
	{
		FTabId TabID(TmvMediaTranscoderTabName);
		FGlobalTabmanager::Get()->TryInvokeTab(TabID);
	}

private:
	/** Names for tabs. */
	static FLazyName TmvMediaTranscoderTabName;
};

FLazyName FTmvMediaEditorModule::TmvMediaTranscoderTabName(TEXT("TmvMediaTranscoder"));

IMPLEMENT_MODULE(FTmvMediaEditorModule, TmvMediaEditor);

#undef LOCTEXT_NAMESPACE
