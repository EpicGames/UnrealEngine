// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaEditorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FTmvMediaEditorStyle& FTmvMediaEditorStyle::Get()
{
	static FTmvMediaEditorStyle Instance;
	return Instance;
}

FTmvMediaEditorStyle::FTmvMediaEditorStyle()
	: FSlateStyleSet(TEXT("TmvMediaEditorStyle"))
{
	const FVector2f Icon16(16.f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());
	
	FSlateStyleSet::SetContentRoot(Plugin->GetContentDir() / TEXT("Editor"));

	Set("TmvMediaEditor.TranscoderTabIcon", new IMAGE_BRUSH_SVG("Icons/MediaTranscode_16", Icon16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FTmvMediaEditorStyle::~FTmvMediaEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
