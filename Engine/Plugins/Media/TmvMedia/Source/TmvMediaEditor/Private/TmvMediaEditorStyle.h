// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FTmvMediaEditorStyle final : public FSlateStyleSet
{
public:
	static FTmvMediaEditorStyle& Get();

	FTmvMediaEditorStyle();

	virtual ~FTmvMediaEditorStyle() override;
};
