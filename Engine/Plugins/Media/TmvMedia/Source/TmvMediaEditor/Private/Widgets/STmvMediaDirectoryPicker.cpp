// Copyright Epic Games, Inc. All Rights Reserved.

#include "STmvMediaDirectoryPicker.h"

#include "ContentBrowserModule.h"
#include "DesktopPlatformModule.h"
#include "IContentBrowserSingleton.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "STmvMediaDirectoryPicker"

void STmvMediaDirectoryPicker::Construct( const FArguments& InArgs )
{
	BrowseTitle = InArgs._BrowseTitle;
	BrowseDirectory = InArgs._BrowseDirectory;
	DirectoryPath = InArgs._DirectoryPath;
	OnDirectoryChanged = InArgs._OnDirectoryChanged;
	bContentDir = InArgs._ContentDir;
	bForceShowPluginContent = InArgs._ForceShowPluginContent;
	bForceShowEngineContent = InArgs._ForceShowEngineContent;
	
	TSharedPtr<SButton> OpenButton;
	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SAssignNew(DirectoryTextBox, SEditableTextBox)
			.Text(this, &STmvMediaDirectoryPicker::GetDirectoryTextBoxText)
			.Font(InArgs._Font)
			.SelectAllTextWhenFocused(true)
			.ClearKeyboardFocusOnCommit(false)
			.OnTextCommitted(this, &STmvMediaDirectoryPicker::OnDirectoryTextCommited)
			.SelectAllTextOnCommit(false)
			.IsReadOnly(InArgs._IsReadOnly)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SAssignNew(OpenButton, SButton)
			.ButtonStyle(InArgs._BrowseButtonStyle)
			.ToolTipText(InArgs._BrowseButtonToolTip)
			.OnClicked(this, &STmvMediaDirectoryPicker::OnBrowseButtonClicked)
			.ContentPadding(2.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			.IsFocusable(false)
			[
				SNew(SImage)
				.Image(InArgs._BrowseButtonImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];

	OpenButton->SetEnabled(InArgs._IsEnabled);
}

void STmvMediaDirectoryPicker::OnDirectoryTextCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	OnDirectoryChanged.ExecuteIfBound(InText.ToString(), bContentDir);
}

FText STmvMediaDirectoryPicker::GetDirectoryTextBoxText() const
{
	return FText::FromString(DirectoryPath.Get());
}

FReply STmvMediaDirectoryPicker::OnBrowseButtonClicked()
{
	const FString DefaultPath = BrowseDirectory.IsSet() ? BrowseDirectory.Get() : DirectoryPath.Get();
	if (bContentDir)
	{
		OpenContentPathPicker(DefaultPath);
	}
	else
	{
		FString OutDirectory;
		if ( OpenPlatformDirectoryPicker(OutDirectory, DefaultPath) )
		{
			OnDirectoryChanged.ExecuteIfBound(OutDirectory, /*bInContentDir*/ false);
		}
	}

	return FReply::Handled();
}

bool STmvMediaDirectoryPicker::OpenPlatformDirectoryPicker(FString& OutDirectory, const FString& InDefaultPath) const
{
	bool bFolderSelected = false;
	if ( IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get() )
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
			? ParentWindow->GetNativeWindow()->GetOSWindowHandle()
			: nullptr;

		FString OutFolderName;
		bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			ParentWindowHandle,
			BrowseTitle.Get().ToString(),
			InDefaultPath,
			OutFolderName
		);

		if ( bFolderSelected )
		{
			OutDirectory = OutFolderName;
		}
	}

	return bFolderSelected;
}

void STmvMediaDirectoryPicker::OpenContentPathPicker(const FString& InDefaultPath)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FPathPickerConfig PathPickerConfig;
	// PathPickerConfig.DefaultPath = InDefaultPath; // That seems to make it not work.
	PathPickerConfig.bAllowContextMenu = false;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &STmvMediaDirectoryPicker::OnContentPathPicked);
	PathPickerConfig.bForceShowEngineContent = bForceShowEngineContent;
	PathPickerConfig.bForceShowPluginContent = bForceShowPluginContent;

	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.AddWidget(SNew(SBox)
		.WidthOverride(300.0f)
		.HeightOverride(300.0f)
		[
			ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
		], FText());

	if (ContentPickerMenu.IsValid())
	{
		ContentPickerMenu->Dismiss();
	}
	
	ContentPickerMenu = FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		MenuBuilder.MakeWidget(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
}

void STmvMediaDirectoryPicker::OnContentPathPicked(const FString& InPickedPath)
{
	if (ContentPickerMenu.IsValid())
	{
		ContentPickerMenu->Dismiss();
		ContentPickerMenu.Reset();
	}

	OnDirectoryChanged.ExecuteIfBound(InPickedPath, /*bInContentDir*/ true);
}

#undef LOCTEXT_NAMESPACE
