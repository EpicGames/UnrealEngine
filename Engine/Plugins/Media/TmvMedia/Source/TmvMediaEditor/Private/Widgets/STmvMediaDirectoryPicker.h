// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Misc/Attribute.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IMenu;
class SEditableTextBox;

/**
 * A directory path box, with a button for picking a new path.
 * Reimplementation of SDirectoryPicker.
 * Differences:
 *  - Made the text box display a delegate to allow user class to intercept and sanitize the picked path.
 *  - Made it match the look, feel and functionality of SFilePathPicker (font, style, title, etc).
 */
class STmvMediaDirectoryPicker : public SCompoundWidget
{
public:
	/** Declares a delegate that is executed when a directory was picked. */
	DECLARE_DELEGATE_TwoParams(FOnDirectoryChanged, const FString& /*Directory*/, bool /*bInContentDir*/);

	SLATE_BEGIN_ARGS(STmvMediaDirectoryPicker)
		: _BrowseButtonToolTip(NSLOCTEXT("STmvMediaDirectoryPicker", "BrowseButtonToolTip", "Choose a directory from this computer"))
		, _Font()
		, _IsReadOnly(false)
		, _ContentDir(false)
		, _ForceShowPluginContent(false)
		, _ForceShowEngineContent(false)
	{ }

		/** Browse button image resource. */
		SLATE_ATTRIBUTE(const FSlateBrush*, BrowseButtonImage)

		/** Browse button visual style. */
		SLATE_STYLE_ARGUMENT(FButtonStyle, BrowseButtonStyle)

		/** Browse button tool tip text. */
		SLATE_ATTRIBUTE(FText, BrowseButtonToolTip)

		/** The directory to browse by default */
		SLATE_ATTRIBUTE(FString, BrowseDirectory)

		/** Title for the browse dialog window. */
		SLATE_ATTRIBUTE(FText, BrowseTitle)
		
		/** Holds the currently selected directory path. */
		SLATE_ATTRIBUTE(FString, DirectoryPath)

		/** Sets the font used for the path text box. */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** Whether the path text box can be modified by the user. */
		SLATE_ATTRIBUTE(bool, IsReadOnly)

		/** Indicates that the path will be picked using the Slate-style directory picker inside the game Content dir. */
		SLATE_ARGUMENT(bool, ContentDir)

		/** Indicates that the asset pickers should always show plugin content */
		SLATE_ARGUMENT(bool, ForceShowPluginContent)

		/** Indicates that the asset pickers should always show engine content */
		SLATE_ARGUMENT(bool, ForceShowEngineContent)

		/** Called when a path has been picked or modified. */
		SLATE_EVENT(FOnDirectoryChanged, OnDirectoryChanged)

	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs);

private:
	/** DirectoryTextBox - OnTextCommitted event handler. */
	void OnDirectoryTextCommited(const FText& InText, ETextCommit::Type InCommitType);

	/** DirectoryTextBox - Text attribute handler. */
	FText GetDirectoryTextBoxText() const;

	/** Browse Button - OnClicked event handler. */
	FReply OnBrowseButtonClicked();
	
	/** Utility function to open a directory picker dialog. */
	bool OpenPlatformDirectoryPicker(FString& OutDirectory, const FString& InDefaultPath) const;

	/** Utility function to open a content path picker drop down menu. */
	void OpenContentPathPicker(const FString& InDefaultPath);

	/** Content Picker Menu - Called when a content path is picked. */
	void OnContentPathPicked(const FString& InPickedPath);

private:
	/** Holds the directory path to browse by default. */
	TAttribute<FString> BrowseDirectory;

	/** Holds the title for the browse dialog window. */
	TAttribute<FText> BrowseTitle;
	
	/** Holds the currently selected directory path. */
	TAttribute<FString> DirectoryPath;

	/** Holds a delegate that is executed when a directory was picked. */
	FOnDirectoryChanged OnDirectoryChanged;

	/** Editable text box for the directory path. */
	TSharedPtr<SEditableTextBox> DirectoryTextBox;

	/** The content pick button popup menu*/
	TSharedPtr<IMenu> ContentPickerMenu;

	/** Indicates that the path will be picked using the Slate-style directory picker inside the game Content dir. */
	bool bContentDir = false;

	/** Indicates that the asset pickers should always show plugin content */
	bool bForceShowPluginContent = false;

	/** Indicates that the asset pickers should always show engine content */
	bool bForceShowEngineContent = false;
};