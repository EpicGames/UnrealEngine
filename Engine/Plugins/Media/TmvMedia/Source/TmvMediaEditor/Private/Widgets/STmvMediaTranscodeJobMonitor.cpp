// Copyright Epic Games, Inc. All Rights Reserved.

#include "STmvMediaTranscodeJobMonitor.h"

#include "Input/Reply.h"
#include "Misc/App.h"
#include "SlateOptMacros.h"
#include "TmvMediaEditorLog.h"
#include "Transcoder/ITmvMediaTranscodeJobManager.h"
#include "Transcoder/ITmvMediaTranscodeJobRunner.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "TmvMediaTranscodeJobMonitor"

namespace UE::TmvMediaEditor::JobMonitor
{
	/** Determines the interval between monitor refresh. Spurious states may be missed. */
	constexpr double JobRefreshInterval = 0.1;
}

/**
 * Job Monitor Item for the Job List View. 
 */
struct FTmvMediaTranscodeJobMonitorItem : public TSharedFromThis<FTmvMediaTranscodeJobMonitorItem>
{
	FTmvMediaTranscodeJobMonitorItem() = default;

	FTmvMediaTranscodeJobMonitorItem(UTmvMediaTranscodeJob* InJob)
	{
		Refresh(InJob);
	}

	/** Build the item list from the currently active jobs in the job manager. */
	static TArray<TSharedPtr<FTmvMediaTranscodeJobMonitorItem>> BuildListItems()
	{
		TArray<TSharedPtr<FTmvMediaTranscodeJobMonitorItem>> OutList;
		
		ITmvMediaTranscodeJobManager* TranscodeJobManager = ITmvMediaTranscodeJobManager::Get();
		if (!TranscodeJobManager)
		{
			return OutList;
		}
	
		TArray<TWeakObjectPtr<UTmvMediaTranscodeJob>> TranscodeJobs;
		TranscodeJobManager->GetTranscodeJobs(TranscodeJobs);
	
		// Loop over all transcode jobs.
		for (TWeakObjectPtr<UTmvMediaTranscodeJob> TranscodeJobWeak : TranscodeJobs)
		{
			if (TranscodeJobWeak.Get())
			{
				OutList.Add(MakeShared<FTmvMediaTranscodeJobMonitorItem>(TranscodeJobWeak.Get()));
			}
		}

		return OutList;
	}

	/** Refresh the list of items from the currently active jobs in the job manager. */
	static void RefreshList(TArray<TSharedPtr<FTmvMediaTranscodeJobMonitorItem>>& InOutItemList)
	{
		ITmvMediaTranscodeJobManager* TranscodeJobManager = ITmvMediaTranscodeJobManager::Get();
		if (!TranscodeJobManager)
		{
			return;
		}

		TArray<TWeakObjectPtr<UTmvMediaTranscodeJob>> TranscodeJobs;
		TranscodeJobManager->GetTranscodeJobs(TranscodeJobs);
		
		if (InOutItemList.Num() != TranscodeJobs.Num())
		{
			const int32 DiffNumItems = FMath::Abs(InOutItemList.Num() - TranscodeJobs.Num());
			if (InOutItemList.Num() > TranscodeJobs.Num())
			{
				for (int32 Index = 0; Index < DiffNumItems; ++Index)
				{
					InOutItemList.Pop();
				}
			}
			else
			{
				for (int32 Index = 0; Index < DiffNumItems; ++Index)
				{
					InOutItemList.Add(MakeShared<FTmvMediaTranscodeJobMonitorItem>());
				}
			}
		}
		
		// Go through the list and recache the display values.
		for (int32 ItemIndex = 0; ItemIndex < InOutItemList.Num(); ++ItemIndex)
		{
			// Recache the display values.
			InOutItemList[ItemIndex]->Refresh(TranscodeJobs[ItemIndex].Get());
		}
	}

	/**
	 * Called by the cancel button to know if a job can be cancelled.
	 * A job is cancellable whenever the runner still has it (active or pending), so the user
	 * can drop queued jobs and cancel jobs that have been told to stop but haven't drained yet.
	 * Falls back to IsRunning() for any job that exists in the manager but not in the runner.
	 */
	bool CanCancel() const
	{
		if (UTmvMediaTranscodeJob* TranscodeJob = TranscodeJobWeak.Get())
		{
			const ITmvMediaTranscodeJobRunner* Runner = ITmvMediaTranscodeJobRunner::Get();
			if (Runner && Runner->IsJobActiveOrPending(TranscodeJob->GetId()))
			{
				return true;
			}
			return TranscodeJob->IsRunning();
		}
		return false;
	}

	/** Called when cancel button for this job item is clicked. */
	FReply OnCancel()
	{
		if (UTmvMediaTranscodeJob* TranscodeJob = TranscodeJobWeak.Get())
		{
			const FGuid JobId = TranscodeJob->GetId();
			ITmvMediaTranscodeJobRunner* Runner = ITmvMediaTranscodeJobRunner::Get();
			if (Runner && Runner->IsJobActiveOrPending(JobId))
			{
				Runner->CancelJob(JobId);
			}
			else if (TranscodeJob->IsRunning())
			{
				// Fallback for jobs not managed by the runner.
				TranscodeJob->RequestStop(FApp::GetCurrentTime(), ETmvMediaTranscodeJobStopReason::Cancelled);
			}
		}

		return FReply::Handled();
	}

	/** Refresh the item from the given job. */
	void Refresh(UTmvMediaTranscodeJob* InJob)
	{
		if (!InJob)
		{
			InputUrl = CurrentStatus = CurrentProgress = ElapsedTime = FText::GetEmpty();
			TranscodeJobWeak.Reset();
			return;
		}

		TranscodeJobWeak = InJob;
		
		// Retrieve input url.
		InputUrl = FText::Format(LOCTEXT("Url", "{0}"), FText::FromString(InJob->Settings.GetInputPath()));

		FString StatusString;
		const bool bIsRunning = InJob->IsRunning();
		const bool bIsStopped = InJob->AreStagesStopped();
		const bool bHasErrors = InJob->bIsError;
		if (bIsRunning)
		{
			StatusString = TEXT("Running");
		}
		else
		{
			if (bIsStopped)
			{
				StatusString = TEXT("Stopped");
			}
			else
			{
				StatusString = TEXT("Stopping");
			}
		}
		if (bHasErrors)
		{
			StatusString += TEXT(" - Errors");
		}
		else if (InJob->IsCompleted())	// todo: need to know if it was cancelled.
		{
			if (InJob->IsCompleted()
				&& (InJob->StopReason == ETmvMediaTranscodeJobStopReason::None
				|| InJob->StopReason == ETmvMediaTranscodeJobStopReason::Completed))
			{
				StatusString += TEXT(" - Completed");
			}
			else
			{
				StatusString += TEXT(" - Cancelled");
			}
		}

		CurrentStatus = FText::Format(LOCTEXT("JobStatus", "{0}"), FText::FromString(StatusString));

		const FTmvMediaTranscodingJobStats Stats = InJob->GetJobStats();
		
		CurrentProgress = FText::Format(LOCTEXT("TranscodeProgress", "{0}/{1}"),
			FText::AsNumber(Stats.ProcessedFrame), FText::AsNumber(Stats.TotalFramesToProcess));

		double ElapsedTimeSec = bIsStopped ?  Stats.StopTime - Stats.StartTime : FAppTime::GetCurrentTime() - Stats.StartTime;

		FNumberFormattingOptions FormattingOptions;
		FormattingOptions.MaximumFractionalDigits = 1;	// only one digit is sufficient.
		ElapsedTime = FText::Format(LOCTEXT("TranscodeTime", "{0} sec"), FText::AsNumber(ElapsedTimeSec, &FormattingOptions));
	}

	/** Returns cached display text for input url. */
	FText GetInputUrl() const
	{
		return InputUrl;
	}

	/** Returns cached display text for current status. */
	FText GetCurrentStatus() const
	{
		return CurrentStatus;
	}

	/** Returns cached display text for job's elapsed time. */
	FText GetElapsedTime() const
	{
		return ElapsedTime;
	}

	/** Returns cached display text for the job's current progress. */
	FText GetCurrentProgress() const
	{
		return CurrentProgress;
	}

private:
	/** Keep a weak ref to the job for access in button callbacks. */
	TWeakObjectPtr<UTmvMediaTranscodeJob> TranscodeJobWeak;

	// Cached display text for each column, refreshed on timer (tick) or on job events.
	FText InputUrl;
	FText CurrentStatus;
	FText ElapsedTime;
	FText CurrentProgress;
};

/**
 * SListView Row Widget for a Job item.
 */
class STmvMediaTranscodeJobMonitorRow : public SMultiColumnTableRow<TSharedPtr<FTmvMediaTranscodeJobMonitorItem>>
{
public:
	SLATE_BEGIN_ARGS(STmvMediaTranscodeJobMonitorRow) {}
	SLATE_ARGUMENT(TSharedPtr<FTmvMediaTranscodeJobMonitorItem>, Item)
SLATE_END_ARGS()

	// Column names
	static const FName ColumnName_InputUrl;
	static const FName ColumnName_Status;
	static const FName ColumnName_ElapsedTime;
	static const FName ColumnName_Progress;

	TSharedPtr<FTmvMediaTranscodeJobMonitorItem> Item;
	
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		Item = InArgs._Item;
		FSuperRowType::FArguments SuperArgs = FSuperRowType::FArguments();
		FSuperRowType::Construct(SuperArgs, InOwnerTable);
	}

	//~ Begin SMultiColumnTableRow
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == ColumnName_InputUrl)
		{
			return SNew(SBox)
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Item.Get(), &FTmvMediaTranscodeJobMonitorItem::GetInputUrl)
			];
		}
		if (ColumnName == ColumnName_Status)
		{
			return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(Item.Get(), &FTmvMediaTranscodeJobMonitorItem::GetCurrentStatus)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.IsEnabled(Item.Get(), &FTmvMediaTranscodeJobMonitorItem::CanCancel)
				.OnClicked(Item.Get(), &FTmvMediaTranscodeJobMonitorItem::OnCancel)
				.Text(LOCTEXT("CancelProcessImages", "Cancel"))
				.ToolTipText(LOCTEXT("CancelProcessImagesButtonToolTip", "Cancel processing images."))
			];
		}
		if (ColumnName == ColumnName_ElapsedTime)
		{
			return SNew(SBox)
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Item.Get(), &FTmvMediaTranscodeJobMonitorItem::GetElapsedTime)
			];
		}
		if (ColumnName == ColumnName_Progress)
		{
			return SNew(SBox)
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Item.Get(), &FTmvMediaTranscodeJobMonitorItem::GetCurrentProgress)
			];
		}
		return SNullWidget::NullWidget;
	}
	//~ End SMultiColumnTableRow
};

const FName STmvMediaTranscodeJobMonitorRow::ColumnName_InputUrl = FName(TEXT("Input Url"));
const FName STmvMediaTranscodeJobMonitorRow::ColumnName_Status = FName(TEXT("Status"));
const FName STmvMediaTranscodeJobMonitorRow::ColumnName_ElapsedTime = FName(TEXT("Elapsed Time"));
const FName STmvMediaTranscodeJobMonitorRow::ColumnName_Progress = FName(TEXT("Progress"));

STmvMediaTranscodeJobMonitor::~STmvMediaTranscodeJobMonitor()
{
	if (ITmvMediaTranscodeJobManager* TranscodeJobManager = ITmvMediaTranscodeJobManager::Get())
	{
		TranscodeJobManager->GetOnTranscodeJobAdded().RemoveAll(this);
		TranscodeJobManager->GetOnTranscodeJobRemoved().RemoveAll(this);
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STmvMediaTranscodeJobMonitor::Construct(const FArguments& InArgs)
{
	JobList = FTmvMediaTranscodeJobMonitorItem::BuildListItems();

	ChildSlot
	[
		SNew(SVerticalBox)

		// Job list label.
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 5, 0, 0)
		[
			SNew(STextBlock)
				.Text(LOCTEXT("ActiveJobs", "Active Jobs"))
		]

		// Job list
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(JobListView, SListView<TSharedPtr<FTmvMediaTranscodeJobMonitorItem>>)
			.ListItemsSource(&JobList)
			.OnGenerateRow(this, &STmvMediaTranscodeJobMonitor::OnGenerateRow)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+SHeaderRow::Column(STmvMediaTranscodeJobMonitorRow::ColumnName_InputUrl)
				.FillWidth(0.6f)
				.DefaultLabel(LOCTEXT("JobListHeader_Input", "Input"))
				+SHeaderRow::Column(STmvMediaTranscodeJobMonitorRow::ColumnName_ElapsedTime)
				.FillWidth(0.1f)
				.DefaultLabel(LOCTEXT("JobListHeader_Time", "Time"))
				+SHeaderRow::Column(STmvMediaTranscodeJobMonitorRow::ColumnName_Progress)
				.FillWidth(0.1f)
				.DefaultLabel(LOCTEXT("JobListHeader_Progress", "Progress"))
				+ SHeaderRow::Column(STmvMediaTranscodeJobMonitorRow::ColumnName_Status)
				.FillWidth(0.2f)
				.DefaultLabel(LOCTEXT("JobListHeader_Status", "Status"))
			)
		]
	];

	// Get notified when transcode jobs are added or removed from the manager.
	if (ITmvMediaTranscodeJobManager* TranscodeJobManager = ITmvMediaTranscodeJobManager::Get())
	{
		TranscodeJobManager->GetOnTranscodeJobAdded().AddSP(this, &STmvMediaTranscodeJobMonitor::OnTranscodeJobAdded);
		TranscodeJobManager->GetOnTranscodeJobRemoved().AddSP(this, &STmvMediaTranscodeJobMonitor::OnTranscodeJobRemoved);
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<ITableRow> STmvMediaTranscodeJobMonitor::OnGenerateRow(TSharedPtr<FTmvMediaTranscodeJobMonitorItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const
{
	return SNew(STmvMediaTranscodeJobMonitorRow, InOwnerTable)
		.Item(InItem);
}

void STmvMediaTranscodeJobMonitor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// Throttled ui refresh 
	if (InCurrentTime - LastJobListRefreshTime >= UE::TmvMediaEditor::JobMonitor::JobRefreshInterval)
	{
		const int32 PrevNumJobs = JobList.Num();
		FTmvMediaTranscodeJobMonitorItem::RefreshList(JobList);

		// Fully rebuild the list if the number of job changed.
		if (PrevNumJobs != JobList.Num() && JobListView)
		{
			JobListView->RebuildList();
		}

		LastJobListRefreshTime = InCurrentTime;
	}
}

void STmvMediaTranscodeJobMonitor::RefreshTranscodeJobList()
{
	FTmvMediaTranscodeJobMonitorItem::RefreshList(JobList);
	if (JobListView)
	{
		JobListView->RebuildList();
	}
}

#undef LOCTEXT_NAMESPACE
