// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class STextBlock;
class SVerticalBox;
class UTmvMediaTranscodeJob;
struct FTmvMediaTranscodeJobMonitorItem;

/**
 * Monitors (and partially manages) active transcode jobs from the global job manager.
 */
class STmvMediaTranscodeJobMonitor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STmvMediaTranscodeJobMonitor){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

	virtual ~STmvMediaTranscodeJobMonitor() override;
private:
	/** SListView row generation handler. */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FTmvMediaTranscodeJobMonitorItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const;

	/** Refresh the list of transcode jobs from the job manager. */
	void RefreshTranscodeJobList();

	/** Called when a job is added to the job manager. */
	void OnTranscodeJobAdded(UTmvMediaTranscodeJob* InJob)
	{
		RefreshTranscodeJobList();
	}

	/** Called when a job is removed from the job manager. */
	void OnTranscodeJobRemoved(UTmvMediaTranscodeJob* InJob)
	{
		RefreshTranscodeJobList();
	}

	/** Job list View */
	TSharedPtr<SListView<TSharedPtr<FTmvMediaTranscodeJobMonitorItem>>> JobListView;

	/** Job list View items, reflect the active jobs in the job manager. */
	TArray<TSharedPtr<FTmvMediaTranscodeJobMonitorItem>> JobList;

	/** Keep track of the refresh time to throttle UI updates. */
	double LastJobListRefreshTime = 0.0;
};
