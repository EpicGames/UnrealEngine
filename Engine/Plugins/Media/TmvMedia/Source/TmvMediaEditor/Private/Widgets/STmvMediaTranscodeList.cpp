// Copyright Epic Games, Inc. All Rights Reserved.

#include "STmvMediaTranscodeList.h"

#include "Framework/Commands/GenericCommands.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "TmvMediaEditorLog.h"
#include "TmvMediaTranscodeListCommands.h"
#include "TmvMediaTranscodeListHandle.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Transcoder/ITmvMediaTranscodeJobManager.h"
#include "Transcoder/TmvMediaTranscodeList.h"
#include "Transcoder/TmvMediaTranscodeSerialization.h"
#include "Utils/TmvMediaSerializationUtils.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "STmvMediaTranscodeJobList"

/**
 * Job List Item for the Job List View widget.
 */
struct FTmvMediaTranscodeJobListItem : public TSharedFromThis<FTmvMediaTranscodeJobListItem>
{
	FTmvMediaTranscodeJobListItem() = default;

	FTmvMediaTranscodeJobListItem(int32 InIndex, UTmvMediaTranscodeList* InList)
		: ItemIndex(InIndex)
	{
		Refresh(InList);
	}

	/** Build the item view list backing the list view widget. */
	static TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> BuildListItems(UTmvMediaTranscodeList* InList)
	{
		TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> Items;
		if (InList)
		{
			Items.Reserve(InList->GetNumItems());			
			for (int32 ItemIndex = 0; ItemIndex < InList->GetNumItems(); ++ItemIndex)
			{
				Items.Add(MakeShared<FTmvMediaTranscodeJobListItem>(ItemIndex, InList));
			}
		}
		else
		{
			UE_LOGF(LogTmvMediaEditor, Error, "Unable to build list items: invalid transcode list.");
		}
		return Items;
	}

	/** Refresh item view list in place. */
	static void RefreshList(TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>>& InOutItemList, UTmvMediaTranscodeList* InList)
	{
		if (!InList)
		{
			UE_LOGF(LogTmvMediaEditor, Error, "Unable to refresh list items: invalid transcode list.");
			return;
		}
		
		if (InOutItemList.Num() != InList->GetNumItems())
		{
			const int32 DiffNumItems = FMath::Abs(InOutItemList.Num() - InList->GetNumItems());
			if (InOutItemList.Num() > InList->GetNumItems())
			{
				for (int32 Index = 0; Index < DiffNumItems; ++Index)
				{
					InOutItemList.Pop();
				}
			}
			else
			{
				for (int32 Index = 0; Index < DiffNumItems; ++Index)
				{
					InOutItemList.Add(MakeShared<FTmvMediaTranscodeJobListItem>());
				}
			}
		}
		
		// Go through the list and make sure it is up to date with the indices and recache the display values.
		for (int32 ItemIndex = 0; ItemIndex < InOutItemList.Num(); ++ItemIndex)
		{
			InOutItemList[ItemIndex]->ItemIndex = ItemIndex;
			// Recache the display values.
			InOutItemList[ItemIndex]->Refresh(InList);
		}
	}

	/** Refresh the cached display values for this item. */
	void Refresh(UTmvMediaTranscodeList* InList)
	{
		// Keep a reference to the list that is being displayed to propagate editing operations.
		TranscodeListWeak = InList;
		
		if (InList && InList->IsValidItemIndex(ItemIndex))
		{
			const FTmvMediaTranscodeListItem& Item = InList->GetItem(ItemIndex);
			ItemId = Item.Id;
			JobName = FText::FromString(Item.Name);
			InputUrl = FText::FromString(Item.Settings.GetInputPath());
			OutputPath = FText::FromString(Item.Settings.OutputPath.Path);
		}
		else
		{
			ItemId.Invalidate();
			JobName = InputUrl = OutputPath = FText::GetEmpty();
		}
	}

	/** Returns cached display text for job name. */
	FText GetJobName() const
	{
		return JobName;
	}

	/** Job Name (SInlineEditableTextBlock) handler for committed text. */
	void OnJobNameTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo)
	{
		JobName = InText;

		// Propagate this to the actual list.
		if (UTmvMediaTranscodeList* TranscodeList = TranscodeListWeak.Get())
		{
			if (FTmvMediaTranscodeListItem* JobItem = TranscodeList->GetItemMutable(ItemIndex))
			{
				FScopedTransaction Transaction(LOCTEXT("RenameJobTransaction", "Rename Selected Job"));
				TranscodeList->Modify();

				JobItem->Name = InText.ToString();
				TranscodeList->GetOnItemEvent().Broadcast(TranscodeList, {ETmvMediaTranscodeListItemEventType::ItemsModified, {ItemIndex}});
			}
		}
	}

	/** Job Name (SInlineEditableTextBlock) handler for text changed validation. */
	bool OnVerifyJobNameTextChanged(const FText& InText, FText& OutErrorMessage)
	{
		return true;
	}

	/** Returns cached display text for the input url. */
	FText GetInputUrl() const
	{
		return InputUrl;
	}

	/** Returns cached display text for the output path. */
	FText GetOutputPath() const
	{
		return OutputPath;
	}

	/** Returns the status widget switcher index. */
	int32 GetStatusIndex() const
	{
		if (const ITmvMediaTranscodeJobManager* TranscodeJobManager = ITmvMediaTranscodeJobManager::Get())
		{
			if (const UTmvMediaTranscodeJob* Job = TranscodeJobManager->GetTranscodeJob(ItemId))
			{
				return Job->IsRunning() ? 1 : 0;
			}
		}
		
		return 0;	// Shows status message (0) or progress bar (1). 
	}

	/** Returns the status message for the text block widget. */
	FText GetStatusMessage() const
	{
		static const FText IdleStatus = LOCTEXT("JobListItem_StatusMessage_Idle", "Idle");
		static const FText PendingStatus = LOCTEXT("JobListItem_StatusMessage_Pending", "Pending");

		// We would like to know if the job is queued for execution.
		if (const ITmvMediaTranscodeJobManager* TranscodeJobManager = ITmvMediaTranscodeJobManager::Get())
		{
			if (const UTmvMediaTranscodeJob* Job = TranscodeJobManager->GetTranscodeJob(ItemId))
			{
				return PendingStatus;
			}
		}
		
		return IdleStatus;
	}

	/** Returns the job's project percent for the progress bar widget. */
	TOptional<float> GetProgressPercent() const
	{
		if (const ITmvMediaTranscodeJobManager* TranscodeJobManager = ITmvMediaTranscodeJobManager::Get())
		{
			if (const UTmvMediaTranscodeJob* Job = TranscodeJobManager->GetTranscodeJob(ItemId))
			{
				const FTmvMediaTranscodingJobStats JobStats = Job->GetJobStats();
				
				if (JobStats.TotalFramesToProcess > 0)
				{
					return static_cast<float>(JobStats.ProcessedFrame) / static_cast<float>(JobStats.TotalFramesToProcess);
				}
			}
		}

		return 0;
	}

	/** Index of this item in the list. */
	int32 ItemIndex = INDEX_NONE;

	/**
	 * Keep a weak reference to the last list what was used for refresh.
	 * This is used to implement the editing operations that can be on the items directly
	 * in the list view widget.
	 */
	TWeakObjectPtr<UTmvMediaTranscodeList> TranscodeListWeak;

	/** Cached display text for JobName column. */
	FText JobName;

	/** Cached display text for InputUrl column. */
	FText InputUrl;

	/** Cached display text for OutputPath column. */
	FText OutputPath;

	/** Cached the item guid to retrieve the corresponding job in the job manager. */
	FGuid ItemId;

	/** Keep a pointer to the name edit box to be able to rename with F2 */
	TWeakPtr<SInlineEditableTextBlock> NameEditableBoxWeak;
};

/**
 * This drag drop operation for one transcode job item in the list view.
 * Using decorated drag and drop for clarity.
 */
class FTmvMediaTranscodeJobListItemDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FTmvMediaTranscodeJobListItemDragDropOp, FDecoratedDragDropOp)

	/** The template to create an instance */
	TSharedPtr<FTmvMediaTranscodeJobListItem> ListItem;

	/** Constructs the drag drop operation */
	static TSharedRef<FTmvMediaTranscodeJobListItemDragDropOp> New(const TSharedPtr<FTmvMediaTranscodeJobListItem>& InListItem, FText InDragText)
	{
		TSharedRef<FTmvMediaTranscodeJobListItemDragDropOp> Operation = MakeShared<FTmvMediaTranscodeJobListItemDragDropOp>();
		Operation->ListItem = InListItem;
		Operation->DefaultHoverText = InDragText;
		Operation->CurrentHoverText = InDragText;
		Operation->Construct();

		return Operation;
	}
};

/**
 * SListView Row Widget for a Job item.
 */
class STmvMediaTranscodeJobListRow : public SMultiColumnTableRow<TSharedPtr<FTmvMediaTranscodeJobListItem>>
{
public:
	SLATE_BEGIN_ARGS(STmvMediaTranscodeJobListRow) {}
		SLATE_ARGUMENT(TSharedPtr<FTmvMediaTranscodeJobListItem>, Item)
	SLATE_END_ARGS()

	// Colum names
	static const FName ColumnName_JobName;
	static const FName ColumnName_InputUrl;
	static const FName ColumnName_OutputPath;
	static const FName ColumnName_Status;
	
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		Item = InArgs._Item;
		// See SQueueJobListRow::Construct for an example of how to connect Drag events.
		FSuperRowType::Construct(
			FSuperRowType::FArguments()
			.OnDragDetected(this, &STmvMediaTranscodeJobListRow::OnDragDetected)
			.OnCanAcceptDrop(this, &STmvMediaTranscodeJobListRow::OnCanAcceptDrop)
			.OnAcceptDrop(this, &STmvMediaTranscodeJobListRow::OnAcceptDrop)
			, InOwnerTable);
	}
	
	FReply OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InPointerEvent)
	{
		if (Item.IsValid())
		{
			FText DefaultText = LOCTEXT("DefaultDragDropFormat", "Move 1 item(s)");
			return FReply::Handled().BeginDragDrop(FTmvMediaTranscodeJobListItemDragDropOp::New(Item, DefaultText));
		}
		return FReply::Unhandled();
	}

	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FTmvMediaTranscodeJobListItem> InItem)
	{
		TSharedPtr<FTmvMediaTranscodeJobListItemDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FTmvMediaTranscodeJobListItemDragDropOp>();
		if (DragDropOp.IsValid())
		{
			if (InDropZone == EItemDropZone::OntoItem)
			{
				DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			}
			else
			{
				DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Ok"));
			}
			return InDropZone;
		}
		return TOptional<EItemDropZone>();
	}

	FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FTmvMediaTranscodeJobListItem> InItem)
	{
		TSharedPtr<FTmvMediaTranscodeJobListItemDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FTmvMediaTranscodeJobListItemDragDropOp>();
		if (!DragDropOp.IsValid() || !DragDropOp->ListItem.IsValid() || !Item.IsValid())
		{
			return FReply::Unhandled();	
		}

		UTmvMediaTranscodeList* TranscodeList = Item->TranscodeListWeak.Get();

		if (!TranscodeList || TranscodeList != DragDropOp->ListItem->TranscodeListWeak.Get())
		{
			return FReply::Unhandled(); // Only support drag and drop from the same list for now.
		}
		
		FScopedTransaction Transaction(LOCTEXT("ReorderJob_Transaction", "Reorder Job"));
		TranscodeList->Modify();

		const int32 IndexOffset = (InDropZone == EItemDropZone::BelowItem) ? 1 : 0;
		return TranscodeList->ReorderItems(DragDropOp->ListItem->ItemIndex, Item->ItemIndex + IndexOffset)
			? FReply::Handled() : FReply::Unhandled();
	}

	//~ Begin SMultiColumnTableRow
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == ColumnName_JobName)
		{
			return SNew(SBox)
			.Padding(2.0f)
			[
				SAssignNew(Item->NameEditableBoxWeak, SInlineEditableTextBlock)
				.Text(Item.Get(), &FTmvMediaTranscodeJobListItem::GetJobName)
				.OnVerifyTextChanged(Item.Get(), &FTmvMediaTranscodeJobListItem::OnVerifyJobNameTextChanged)
				.OnTextCommitted(Item.Get(), &FTmvMediaTranscodeJobListItem::OnJobNameTextCommitted)
				.IsSelected(this, &STmvMediaTranscodeJobListRow::IsSelectedExclusively)
				.IsReadOnly(false)
			];
		}
		if (ColumnName == ColumnName_InputUrl)
		{
			return SNew(SBox)
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Item.Get(), &FTmvMediaTranscodeJobListItem::GetInputUrl)
			];
		}
		if (ColumnName == ColumnName_OutputPath)
		{
			return SNew(SBox)
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Item.Get(), &FTmvMediaTranscodeJobListItem::GetOutputPath)
			];
		}
		if (ColumnName == ColumnName_Status)
		{
			return SNew(SWidgetSwitcher)
			.WidgetIndex(Item.Get(), &FTmvMediaTranscodeJobListItem::GetStatusIndex)

			// Status Message Label
			+ SWidgetSwitcher::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(Item.Get(), &FTmvMediaTranscodeJobListItem::GetStatusMessage)
			]
			// Progress Bar
			+ SWidgetSwitcher::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SProgressBar)
				.Percent(Item.Get(), &FTmvMediaTranscodeJobListItem::GetProgressPercent)
			];
		}
		return SNullWidget::NullWidget;
	}
	//~ End SMultiColumnTableRow

	/** Shared reference to this item. */
	TSharedPtr<FTmvMediaTranscodeJobListItem> Item;
};

const FName STmvMediaTranscodeJobListRow::ColumnName_JobName = FName(TEXT("Job Name"));
const FName STmvMediaTranscodeJobListRow::ColumnName_InputUrl = FName(TEXT("Input Url"));
const FName STmvMediaTranscodeJobListRow::ColumnName_OutputPath = FName(TEXT("Output Path"));
const FName STmvMediaTranscodeJobListRow::ColumnName_Status = FName(TEXT("Status"));
// ----------------

STmvMediaTranscodeList::~STmvMediaTranscodeList()
{
	if (ListHandle)
	{
		if (UTmvMediaTranscodeList* List = ListHandle->Get())
		{
			List->GetOnItemEvent().RemoveAll(this);
		}
		ListHandle->OnGetCurrentSelectionDelegate.Unbind();
		ListHandle->OnGetNumSelectedDelegate.Unbind();
		ListHandle->GetOnTranscodeListChanged().RemoveAll(this);
	}
}

namespace UE::TmvMediaEditor::TranscodeList
{
	/** Helper function make a button that is wrapping a command. */
	TSharedRef<SWidget> MakeCommandButton(const TSharedPtr<FUICommandInfo>& InCommand, const TSharedPtr<FUICommandList>& InCommandList)
	{
		if (!InCommand || !InCommandList)
		{
			return SNullWidget::NullWidget;
		}
		
		TWeakPtr<FUICommandList> CommandListWeak = InCommandList;
		TWeakPtr<FUICommandInfo> CommandWeak = InCommand;
		
		auto IsEnabledFunction = [CommandWeak, CommandListWeak]() -> bool
		{
			const TSharedPtr<FUICommandInfo> Command = CommandWeak.Pin();
			const TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin();

			if (Command && CommandList)
			{
				return CommandList->CanExecuteAction(Command.ToSharedRef());
			}
			return false;
		};

		auto ClickedFunction = [CommandWeak, CommandListWeak]()
		{
			const TSharedPtr<FUICommandInfo> Command = CommandWeak.Pin();
			const TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin();

			if (Command && CommandList && CommandList->ExecuteAction(Command.ToSharedRef()))
			{
				return FReply::Handled();
			}
			return FReply::Unhandled();
		};

		return SNew(SButton)
			.IsEnabled_Lambda(IsEnabledFunction)
			.OnClicked_Lambda(ClickedFunction)
			.ForegroundColor(FSlateColor::UseStyle())
			.Text(InCommand->GetLabel())
			.ToolTipText(InCommand->GetDescription());
	}

	/**
	 * Utility class that will accumulate added item ids during an operation (from
	 * multiple Item Events) and select those items when the operation is completed.
	 * 
	 * This method works with operations that add items in multiple calls to the transcode list
	 * which generate an Item Event per item added, but we want to select all added items and not
	 * just the last one.
	 */
	class FAddedItemsScopedSelectionModifier
	{
	public:
		FAddedItemsScopedSelectionModifier(
			const TSharedPtr<FTmvMediaTranscodeListHandle>& InListHandle,
			const TSharedPtr<SListView<TSharedPtr<FTmvMediaTranscodeJobListItem>>>& InJobListView)
			: ListHandle(InListHandle)
			, JobListView(InJobListView)
		{
			if (UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr)
			{
				List->GetOnItemEvent().AddRaw(this, &FAddedItemsScopedSelectionModifier::OnTranscodeListItemEvent);
			}
		}

		~FAddedItemsScopedSelectionModifier()
		{
			if (UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr)
			{
				List->GetOnItemEvent().RemoveAll(this);	
			}

			if (!JobListView)
			{
				return;
			}

			JobListView->ClearSelection();

			if (AddedItemIds.IsEmpty())
			{
				return;
			}
			
			TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> NewSelectedItems;
			NewSelectedItems.Reserve(AddedItemIds.Num());

			const TArrayView<const TSharedPtr<FTmvMediaTranscodeJobListItem>>& Items = JobListView->GetItems();

			if (AddedItemIds.Num() > 1)
			{
				// Use a map to speed up the search in case of a larger list.
				TMap<FGuid, const TSharedPtr<FTmvMediaTranscodeJobListItem>> ItemMapByIds;

				ItemMapByIds.Reserve(Items.Num());
				for (const TSharedPtr<FTmvMediaTranscodeJobListItem>& Item : Items)
				{
					ItemMapByIds.Add(Item->ItemId, Item);
				}
				
				for (const FGuid& ItemId : AddedItemIds)
				{
					if (const TSharedPtr<FTmvMediaTranscodeJobListItem>* FoundItem = ItemMapByIds.Find(ItemId))
					{
						NewSelectedItems.Add(*FoundItem);
					}
				}
			}
			else
			{
				for (const FGuid& ItemId : AddedItemIds)
				{
					const TSharedPtr<FTmvMediaTranscodeJobListItem>* FoundItem =
						Items.FindByPredicate([ItemId](const TSharedPtr<FTmvMediaTranscodeJobListItem>& InItem)
						{
							return InItem->ItemId == ItemId;
						});

					if (FoundItem)
					{
						NewSelectedItems.Add(*FoundItem);
					}
				}
			}

			if (NewSelectedItems.Num())
			{
				JobListView->SetItemSelection(NewSelectedItems, /*bSelected*/ true);
			}
		}
		
	private:
		void OnTranscodeListItemEvent(const UTmvMediaTranscodeList* InList, const FTmvMediaTranscodeListItemEventArgs& InArgs)
		{
			if (InArgs.Type == ETmvMediaTranscodeListItemEventType::ItemsAdded && InList)
			{
				// Indices can change in case of multiple insertions, for safety, we keep track of ids instead.
				for (int32 ItemIndex : InArgs.ItemIndices)
				{
					const FTmvMediaTranscodeListItem& Item = InList->GetItem(ItemIndex);
					if (Item.Id.IsValid())
					{
						AddedItemIds.AddUnique(Item.Id);
					}
				}
			}
		}

		/** Accumulator of added item Ids.*/
		TArray<FGuid> AddedItemIds;

		/** Reference to the list handle. */
		TSharedPtr<FTmvMediaTranscodeListHandle> ListHandle;

		/** JobListView to modify selection. */
		TSharedPtr<SListView<TSharedPtr<FTmvMediaTranscodeJobListItem>>> JobListView;
	};
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STmvMediaTranscodeList::Construct(const FArguments& InArgs)
{
	ListHandle = InArgs._ListHandle;
	
	if (ListHandle)
	{
		JobList = FTmvMediaTranscodeJobListItem::BuildListItems(ListHandle->Get());

		if (UTmvMediaTranscodeList* List = ListHandle->Get())
		{
			List->GetOnItemEvent().AddSP(this, &STmvMediaTranscodeList::OnTranscodeListItemEvent);
		}
		
		ListHandle->OnGetCurrentSelectionDelegate.BindSPLambda(this, [this]()
		{
			TArray<int32> SelectedIndices;
			if (JobListView)
			{
				TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> SelectedItems = JobListView->GetSelectedItems();
				SelectedIndices.Reserve(SelectedItems.Num());
				for (const TSharedPtr<FTmvMediaTranscodeJobListItem>& SelectedItem : SelectedItems)
				{
					SelectedIndices.Add(SelectedItem->ItemIndex);
				}
			}
			return SelectedIndices;
		});
		
		ListHandle->OnGetNumSelectedDelegate.BindSPLambda(this, [this]()
		{
			return JobListView ? JobListView->GetNumItemsSelected() : 0;
		});
		
		ListHandle->GetOnTranscodeListChanged().AddSPLambda(this, [this](UTmvMediaTranscodeList* InPreviousList, UTmvMediaTranscodeList* InNewList)
		{
			if (InPreviousList)
			{
				InPreviousList->GetOnItemEvent().RemoveAll(this);
			}
			if (InNewList)
			{
				InNewList->GetOnItemEvent().AddSP(this, &STmvMediaTranscodeList::OnTranscodeListItemEvent);
			}
				
			FTmvMediaTranscodeJobListItem::RefreshList(JobList, InNewList);
			if (JobListView)
			{
				JobListView->RebuildList();
				JobListView->ClearSelection();
			}
		});
	}

	using namespace UE::TmvMediaEditor::TranscodeList;

	// Create a command list of this widget to add the transcode list specific commands.
	CommandList = MakeShared<FUICommandList>();
	BindCommands(CommandList);
	if (InArgs._CommandList)
	{
		CommandList->Append(InArgs._CommandList.ToSharedRef());
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
					.IsEnabled(this, &STmvMediaTranscodeList::CanAddJob)
					.OnClicked(this, &STmvMediaTranscodeList::OnAddJob)
					.ForegroundColor(FSlateColor::UseStyle())
					.Text(LOCTEXT("AddJobButtonLabel", "Add Job"))
					.ToolTipText(LOCTEXT("AddJobButtonToolTip", "Add New Job in the list"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
					.IsEnabled(this, &STmvMediaTranscodeList::CanDuplicateSelectedJobs)
					.OnClicked(this, &STmvMediaTranscodeList::OnDuplicateSelectedJobs)
					.ForegroundColor(FSlateColor::UseStyle())
					.Text(LOCTEXT("DuplicateJobButtonLabel", "Duplicate Job"))
					.ToolTipText(LOCTEXT("DuplicateJobButtonToolTip", "Duplicate selected Job"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
					.IsEnabled(this, &STmvMediaTranscodeList::CanRemoveSelectedJobs)
					.OnClicked(this, &STmvMediaTranscodeList::OnRemoveSelectedJobs)
					.ForegroundColor(FSlateColor::UseStyle())
					.Text(LOCTEXT("RemoveJobButtonLabel", "Remove Job(s)"))
					.ToolTipText(LOCTEXT("RemoveJobButtonToolTip", "Remove selected job(s) from the list."))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				MakeCommandButton(FTmvMediaTranscodeListCommands::Get().ImportJobItem, CommandList)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				MakeCommandButton(FTmvMediaTranscodeListCommands::Get().ExportJobItem, CommandList)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(JobListView, SListView<TSharedPtr<FTmvMediaTranscodeJobListItem>>)
			.ListItemsSource(&JobList)
			.SelectionMode(ESelectionMode::Multi)
			.OnSelectionChanged(this, &STmvMediaTranscodeList::OnSelectionChanged)
			.OnGenerateRow(this, &STmvMediaTranscodeList::OnGenerateRow)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+SHeaderRow::Column(STmvMediaTranscodeJobListRow::ColumnName_JobName)
				.FillWidth(0.15f)
				.DefaultLabel(LOCTEXT("JobListHeader_Job", "Job"))
				+SHeaderRow::Column(STmvMediaTranscodeJobListRow::ColumnName_InputUrl)
				.FillWidth(0.3f)
				.DefaultLabel(LOCTEXT("JobListHeader_Input", "Input"))
				+SHeaderRow::Column(STmvMediaTranscodeJobListRow::ColumnName_OutputPath)
				.FillWidth(0.3f)
				.DefaultLabel(LOCTEXT("JobListHeader_Output", "Output"))
				+ SHeaderRow::Column(STmvMediaTranscodeJobListRow::ColumnName_Status)
				.FixedWidth(80)
				.DefaultLabel(LOCTEXT("JobListHeader_Status", "Status"))
			)
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply STmvMediaTranscodeList::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);	
}

void STmvMediaTranscodeList::BindCommands(const TSharedPtr<FUICommandList>& InCommandList)
{
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	InCommandList->MapAction(GenericCommands.Rename,
		FExecuteAction::CreateSP(this, &STmvMediaTranscodeList::RenameSelectedJob),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscodeList::CanRenameSelectedJob));

	InCommandList->MapAction(GenericCommands.Delete,
		FExecuteAction::CreateSPLambda(this, [this](){ OnRemoveSelectedJobs(); }),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscodeList::CanRemoveSelectedJobs));

	InCommandList->MapAction(GenericCommands.Cut,
		FExecuteAction::CreateSP(this, &STmvMediaTranscodeList::CutSelectedJobs),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscodeList::CanCutSelectedJobs));

	InCommandList->MapAction(GenericCommands.Copy,
		FExecuteAction::CreateSP(this, &STmvMediaTranscodeList::CopySelectedJobs),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscodeList::CanCopySelectedJobs));

	InCommandList->MapAction(GenericCommands.Paste,
		FExecuteAction::CreateSP(this, &STmvMediaTranscodeList::Paste),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscodeList::CanPaste));

	InCommandList->MapAction(GenericCommands.Duplicate,
		FExecuteAction::CreateSPLambda(this, [this](){ OnDuplicateSelectedJobs(); }),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscodeList::CanDuplicateSelectedJobs));
}

TSharedRef<ITableRow> STmvMediaTranscodeList::OnGenerateRow(TSharedPtr<FTmvMediaTranscodeJobListItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const
{
	return SNew(STmvMediaTranscodeJobListRow, InOwnerTable)
		.Item(InItem);
}

void STmvMediaTranscodeList::OnSelectionChanged(TSharedPtr<FTmvMediaTranscodeJobListItem> InItem, ESelectInfo::Type InSelectInfo)
{
	if (UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr)
	{
		TArray<int32, TInlineAllocator<1>> SelectedItemIndices;

		if (JobListView.IsValid()) 
		{ 
			const TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> SelectedRows = JobListView->GetSelectedItems(); 
			SelectedItemIndices.Reserve(SelectedRows.Num());

			int32 PriorityItemIndex = INDEX_NONE;
			
			// Add "selected" item first to be sure it has priority for "single" selection work flows.
			if (InItem)
			{
				SelectedItemIndices.Add(InItem->ItemIndex);
				PriorityItemIndex = InItem->ItemIndex;
			}

			// Add the other selected item indices.
			for (const TSharedPtr<FTmvMediaTranscodeJobListItem>& Row : SelectedRows) 
			{ 
				if (Row.IsValid() && Row->ItemIndex != PriorityItemIndex)
				{ 
					SelectedItemIndices.Add(Row->ItemIndex);
				} 
			} 
		} 
		
		ListHandle->GetOnSelectionChanged().Broadcast(List, SelectedItemIndices);
	}
}

bool STmvMediaTranscodeList::IsSelectionValid() const
{
	return JobListView && JobListView->GetNumItemsSelected() > 0;
}

bool STmvMediaTranscodeList::CanAddJob() const
{
	return true;
}

FReply STmvMediaTranscodeList::OnAddJob()
{
	if (UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr)
	{
		FScopedTransaction Transaction(LOCTEXT("AddJobTransaction", "Add Job"));
		List->Modify();

		UE::TmvMediaEditor::TranscodeList::FAddedItemsScopedSelectionModifier SelectionModifier(ListHandle, JobListView);
		List->InsertItemAt(List->GetNumItems(), nullptr);
	}
	return FReply::Handled();
}

FReply STmvMediaTranscodeList::OnDuplicateSelectedJobs()
{
	if (UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr)
	{
		FScopedTransaction Transaction(LOCTEXT("DuplicateSelectionTransaction", "Duplicate Selected Job(s)"));
		List->Modify();

		UE::TmvMediaEditor::TranscodeList::FAddedItemsScopedSelectionModifier SelectionModifier(ListHandle, JobListView);
		TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> SelectedItems = JobListView->GetSelectedItems();
		for (const TSharedPtr<FTmvMediaTranscodeJobListItem>& SelectedItem : SelectedItems)
		{
			// todo: make this into a single operation? DuplicateItem(s)?
			List->InsertItemAt(List->GetNumItems(), &List->GetItem(SelectedItem->ItemIndex));
		}
	}
	return FReply::Handled();
}

FReply STmvMediaTranscodeList::OnRemoveSelectedJobs()
{
	if (UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr)
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveJobsTransaction", "Remove Selected Job(s)"));
		List->Modify();

		TArray<int32> ItemsToRemove;
		TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> SelectedItems = JobListView->GetSelectedItems();
		ItemsToRemove.Reserve(SelectedItems.Num());
		for (const TSharedPtr<FTmvMediaTranscodeJobListItem>& SelectedItem : SelectedItems)
		{
			ItemsToRemove.Add(SelectedItem->ItemIndex);
		}

		List->RemoveItems(ItemsToRemove);
	}
	return FReply::Handled();
}

bool STmvMediaTranscodeList::CanRenameSelectedJob() const
{
	return JobListView && JobListView->GetNumItemsSelected() == 1;
}

void STmvMediaTranscodeList::RenameSelectedJob()
{
	if (JobListView)
	{
		const TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> SelectedItems = JobListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			if (TSharedPtr<SInlineEditableTextBlock> NameEditableBox = SelectedItems[0]->NameEditableBoxWeak.Pin())
			{
				NameEditableBox->EnterEditingMode();
			}
		}
	}
}

void STmvMediaTranscodeList::CutSelectedJobs()
{
	CopySelectedJobs();
	OnRemoveSelectedJobs();
}

namespace UE::TmvMediaEditor::TranscodeList
{
	static FString JobClipboardPrefix = TEXT("TmvMediaTranscodeItems");
}

void STmvMediaTranscodeList::CopySelectedJobs()
{
	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	if (!List || !JobListView)
	{
		return;
	}

	TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> SelectedItems = JobListView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		UE_LOGF(LogTmvMediaEditor, Warning, "No items selected.");
		return;
	}
	
	TArray<FTmvMediaTranscodeListItem> Items;
	Items.Reserve(SelectedItems.Num());
	for(const TSharedPtr<FTmvMediaTranscodeJobListItem>& SelectedItem : SelectedItems)
	{
		if (SelectedItem)
		{
			Items.Add(List->GetItem(SelectedItem->ItemIndex));
		}
	}

	TArray<uint8> Memory;
	FMemoryWriter MemoryWriter(Memory);
	if (!UE::TmvMedia::TranscodeSerialization::SerializeTranscodeListItemsToJson(MemoryWriter, Items))
	{
		return;	// SerializeTranscodeListItemsToJson is logging errors already.
	}

	FString SerializedString = UE::TmvMedia::SerializationUtils::BytesToString(Memory);
	// Add Prefix to quickly identify whether current clipboard is from Transcode items or not
	SerializedString = *FString::Printf(TEXT("%s%s"), *UE::TmvMediaEditor::TranscodeList::JobClipboardPrefix, *SerializedString);
	FPlatformApplicationMisc::ClipboardCopy(*SerializedString);
}

bool STmvMediaTranscodeList::CanPaste() const
{
	if (CanAddJob())
	{
		FString PastedText;
		FPlatformApplicationMisc::ClipboardPaste(PastedText);
		return PastedText.StartsWith(UE::TmvMediaEditor::TranscodeList::JobClipboardPrefix);
	}
	return false;
}

void STmvMediaTranscodeList::Paste()
{
	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	if (!List)
	{
		return;
	}

	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	if (!PastedText.StartsWith(UE::TmvMediaEditor::TranscodeList::JobClipboardPrefix))
	{
		UE_LOGF(LogTmvMediaEditor, Error, "Paste Jobs: clipboard data is not the expected format.");
		return;
	}

	PastedText.RightChopInline(UE::TmvMediaEditor::TranscodeList::JobClipboardPrefix.Len());

	TArray<FTmvMediaTranscodeListItem> Items;
	FMemoryReaderView Reader(UE::TmvMedia::SerializationUtils::ValueToConstBytesView(PastedText));
	
	if (!UE::TmvMedia::TranscodeSerialization::DeserializeTranscodeListItemsFromJson(Reader, Items))
	{
		UE_LOGF(LogTmvMediaEditor, Error, "Paste Jobs: Failed to parse clipboard data.");
		return;
	}

	if (Items.IsEmpty())
	{
		UE_LOGF(LogTmvMediaEditor, Error, "Paste Jobs: Pasted item list is empty.");
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("PasteJobTransaction", "Paste Job"));
	List->Modify();

	UE::TmvMediaEditor::TranscodeList::FAddedItemsScopedSelectionModifier SelectionModifier(ListHandle, JobListView);
	for (const FTmvMediaTranscodeListItem& Item : Items)
	{
		List->InsertItemAt(List->GetNumItems(), &Item);
	}
}

void STmvMediaTranscodeList::OnTranscodeListItemEvent(const UTmvMediaTranscodeList* InList, const FTmvMediaTranscodeListItemEventArgs& InArgs)
{
	// Make sure this is our current list
	UTmvMediaTranscodeList* CurrentList = ListHandle ? ListHandle->Get() : nullptr;
	if (CurrentList && CurrentList == InList)
	{
		FTmvMediaTranscodeJobListItem::RefreshList(JobList, CurrentList);
		if (JobListView)
		{
			// For ItemsModified, skip RebuildList() — it destroys and recreates row widgets mid-click,
			// which clears the bChangedSelectionOnMouseDown flag on the clicked row. This causes the
			// subsequent mouse-up to skip signaling the selection change, so the parameters panel
			// never refreshes when switching jobs. Row attribute bindings pick up the updated cached
			// values on the next paint without requiring a full rebuild.
			if (InArgs.Type != ETmvMediaTranscodeListItemEventType::ItemsModified)
			{
				JobListView->RebuildList();
			}
			if (InArgs.Type == ETmvMediaTranscodeListItemEventType::ItemsRemoved || InArgs.Type == ETmvMediaTranscodeListItemEventType::ItemsReordered)
			{
				JobListView->ClearSelection();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE