// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FUICommandList;
class ITableRow;
class STableViewBase;
class UTmvMediaTranscodeList;
struct FTmvMediaTranscodeJobListItem;
struct FTmvMediaTranscodeListHandle;
struct FTmvMediaTranscodeListItemEventArgs;

/**
 * Tmv Media Transcoder Job List widget
 */
class STmvMediaTranscodeList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STmvMediaTranscodeList){}
		/** Handle to the transcode list object. */
		SLATE_ARGUMENT(TSharedPtr<FTmvMediaTranscodeListHandle>, ListHandle)
		/** Command list to use for toolbar and context menu. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	SLATE_END_ARGS()

	virtual ~STmvMediaTranscodeList() override;

	void Construct(const FArguments& InArgs);

	//~ Begin SWidget
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

private:
	/** Bind this widget's commands to the command list. */
	void BindCommands(const TSharedPtr<FUICommandList>& InCommandList);
	
	/** JobListView (SListView) row widget generation handler. */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FTmvMediaTranscodeJobListItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const;

	/** JobListView (SListView) selection change handler. */
	void OnSelectionChanged(TSharedPtr<FTmvMediaTranscodeJobListItem> InItem, ESelectInfo::Type InSelectInfo);

	/** Returns true if the job view is valid and current item selection is not empty. */
	bool IsSelectionValid() const;

	/** AddJob Button handlers */
	bool CanAddJob() const;
	FReply OnAddJob();
	
	/** Duplicate Job Button handlers */
	bool CanDuplicateSelectedJobs() const { return IsSelectionValid(); }
	FReply OnDuplicateSelectedJobs();

	/** Remove Job Button handlers */
	bool CanRemoveSelectedJobs() const { return IsSelectionValid(); }
	FReply OnRemoveSelectedJobs();

	/** Rename Jobs Action handlers */
	bool CanRenameSelectedJob() const;
	void RenameSelectedJob();

	/** Cut Jobs Action handlers */
	bool CanCutSelectedJobs() const { return IsSelectionValid(); }
	void CutSelectedJobs();

	/** Copy Jobs Action handlers */
	bool CanCopySelectedJobs() const { return IsSelectionValid(); }
	void CopySelectedJobs();

	/** Paste Jobs Action handlers */
	bool CanPaste() const;
	void Paste();
	
	/** Transcode List Item Event Handler. */
	void OnTranscodeListItemEvent(const UTmvMediaTranscodeList* InList, const FTmvMediaTranscodeListItemEventArgs& InArgs);
	
private:
	/** Transcode list being edited. */
	TSharedPtr<FTmvMediaTranscodeListHandle> ListHandle;

	/** Job List View widget. */
	TSharedPtr<SListView<TSharedPtr<FTmvMediaTranscodeJobListItem>>> JobListView;

	/** Job List View items (widget's backing array), reflect the transcode list elements. */
	TArray<TSharedPtr<FTmvMediaTranscodeJobListItem>> JobList;

	/** Shared Bound Command list for menus, toolbars and buttons. */
	TSharedPtr<FUICommandList> CommandList;
};
