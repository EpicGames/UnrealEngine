// Copyright Epic Games, Inc. All Rights Reserved.

#include "STmvMediaTranscoder.h"

#include "DesktopPlatformModule.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "STmvMediaTranscodeJobMonitor.h"
#include "STmvMediaTranscodeList.h"
#include "STmvMediaTranscoderJobControls.h"
#include "STmvMediaTranscoderJobDetails.h"
#include "SlateOptMacros.h"
#include "TmvMediaEditorLog.h"
#include "TmvMediaTranscodeListCommands.h"
#include "Transcoder/TmvMediaTranscodeList.h"
#include "Transcoder/TmvMediaTranscodeSerialization.h"
#include "Utils/TmvMediaPathUtils.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "STmvMediaTranscoder"

namespace UE::TmvMediaEditor::Transcoder
{
	static const FText UntitledDocumentName = LOCTEXT("UntitledDocumentName", "Untitled");
	
	static const FString TmvMediaConfigSection = TEXT("TmvMedia.Settings");
	static const FString TranscodeListConfigKey = TEXT("LastTranscodeListPath");
	static const FString LastJobItemConfigKey = TEXT("LastTranscodeJobItemPath");
	
	static const FString JobFileTypeFilter = TEXT("All files (*.json)|*.json");

	void AddNotification(const FText& InText, bool bSuccess)
	{
		FNotificationInfo Info(InText);
		Info.bFireAndForget = true;
		Info.FadeOutDuration = 0.5f;
		Info.ExpireDuration = 2.0f;

		if (TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info))
		{
			Notification->SetCompletionState(bSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
		}
	}
	
	bool CanOverwriteFile(const FString& InFilename)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Filename"), FText::FromString( InFilename ));
		FText OverwriteMessageFormatting = LOCTEXT("OverwriteMessageFormatting", "Overwrite existing file {Filename}?");
		FText DialogText = FText::Format( OverwriteMessageFormatting, Arguments );
		FText DialogTitle = LOCTEXT("OverwriteDialogTitle", "Warning");

		EAppReturnType::Type RetVal = FMessageDialog::Open( EAppMsgType::YesNo, DialogText, DialogTitle );
		return RetVal == EAppReturnType::Yes;
	}
}

STmvMediaTranscoder::~STmvMediaTranscoder() = default;

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STmvMediaTranscoder::Construct(const FArguments& InArgs)
{
	using namespace UE::TmvMediaEditor::Transcoder;
	using namespace UE::TmvMedia::PathUtils;

	CommandList = MakeShared<FUICommandList>();
	BindCommands(CommandList);

	// Get the last value for this setting.
	GConfig->GetString(*TmvMediaConfigSection, *LastJobItemConfigKey, LastJobItemPath, GEditorPerProjectIni);
	GConfig->GetString(*TmvMediaConfigSection, *TranscodeListConfigKey, TranscodeListPath, GEditorPerProjectIni);

	// Create a transient transcode list.
	TranscodeList.Reset(NewObject<UTmvMediaTranscodeList>(GetTransientPackage(), NAME_None));
	TranscodeListHandle = MakeShared<FTmvMediaTranscodeListHandle>();
	TranscodeListHandle->SetTranscodeList(TranscodeList.Get());

	// Try to load the last edited config, if any. This is for convenience when there is no asset workflow.
	const FString ResolvedTranscodeListPath = ConvertSanitizedPathToFull(TranscodeListPath);
	if (FPaths::FileExists(ResolvedTranscodeListPath))
	{
		if (!LoadJobList(ResolvedTranscodeListPath))
		{
			// If we failed to load the previous job list, clear the path to avoid stomping previous work.
			ResetJobListPath();
		}
	}

	FToolBarBuilder Toolbar(CommandList, FMultiBoxCustomization::None);
	{
		Toolbar.AddToolBarButton(FTmvMediaTranscodeListCommands::Get().CreateNewJobList);
		Toolbar.AddToolBarButton(FTmvMediaTranscodeListCommands::Get().SaveJobListAs);
		Toolbar.AddToolBarButton(FTmvMediaTranscodeListCommands::Get().SaveJobList);
		Toolbar.AddToolBarButton(FTmvMediaTranscodeListCommands::Get().OpenJobList);
		Toolbar.AddToolBarButton(FTmvMediaTranscodeListCommands::Get().LoadJobList);
	}

	ChildSlot
	[
		SNew(SScrollBox)

		// Tool/menu bar
		+ SScrollBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)	// fill all available horizontal space
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					Toolbar.MakeWidget()
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.FillWidth(1.0f)
				[
					SNew(SSpacer)
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					SNew(SBox)
					.VAlign(VAlign_Center)	// have text block vertically aligned (to match toolbar buttons)
					.HAlign(HAlign_Center)
					.Padding(4,0)
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							const FString CurrentPath = GetJobListPath();
							const FText CurrentPathText = !CurrentPath.IsEmpty() ? FText::FromString(CurrentPath) : UntitledDocumentName; 
							// todo: add '*' if the edited asset is dirty.
							return FText::Format(LOCTEXT("JobListPathBox", "{0}"), CurrentPathText);
						})
					]
				]
			]
		]
		// Add details view.
		+ SScrollBox::Slot()
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			+ SSplitter::Slot()
			.Value(0.7)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				[
					SNew(STmvMediaTranscodeList)
					.ListHandle(TranscodeListHandle)
					.CommandList(CommandList)
				]
				// Add Job Control buttons.
				+ SVerticalBox::Slot()
				.Padding(8.0f)
				.AutoHeight()
				.VAlign(VAlign_Top)	// Keep with the list above.
				.HAlign(HAlign_Right)
				[
					SAssignNew(JobControls, STmvMediaTranscoderJobControls, TranscodeListHandle)
				]
				+ SVerticalBox::Slot()
				.Padding(8.0f)
				.AutoHeight()
				.VAlign(VAlign_Top)	// Keep with the previous widget.
				.HAlign(HAlign_Fill)
				[
					SNew(STmvMediaTranscodeJobMonitor)
				]
			]
			+ SSplitter::Slot()
			.Value(0.3)
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(this, &STmvMediaTranscoder::GetDetailsViewWidgetIndex)
				+ SWidgetSwitcher::Slot()
				[
					SNew(STmvMediaTranscoderJobDetails, TranscodeListHandle)
				]
				+ SWidgetSwitcher::Slot()
				.Padding(2.0f, 24.0f, 2.0f, 2.0f)
				[
					SNew(SBox)
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NoJobSelected", "Select a job to view details."))
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply STmvMediaTranscoder::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

int32 STmvMediaTranscoder::GetDetailsViewWidgetIndex() const
{
	if (TranscodeListHandle && TranscodeListHandle->GetNumSelected() != 0)
	{
		return 0;
	}
	return 1;
}

void STmvMediaTranscoder::BindCommands(const TSharedPtr<FUICommandList>& InCommandBindings)
{
	const FTmvMediaTranscodeListCommands& ListCommands = FTmvMediaTranscodeListCommands::Get();

	InCommandBindings->MapAction(ListCommands.CreateNewJobList,
		FExecuteAction::CreateSP(this, &STmvMediaTranscoder::OnCreateNewJobList),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscoder::CanCreateNewJobList));

	InCommandBindings->MapAction(ListCommands.OpenJobList,
		FExecuteAction::CreateSP(this, &STmvMediaTranscoder::OnOpenJobList));

	InCommandBindings->MapAction(ListCommands.LoadJobList,
		FExecuteAction::CreateSP(this, &STmvMediaTranscoder::OnLoadJobList),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscoder::CanLoadJobList));

	InCommandBindings->MapAction(ListCommands.SaveJobList,
		FExecuteAction::CreateSP(this, &STmvMediaTranscoder::OnSaveJobList),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscoder::CanSaveJobList));

	InCommandBindings->MapAction(ListCommands.SaveJobListAs,
		FExecuteAction::CreateSP(this, &STmvMediaTranscoder::OnSaveJobListAs));

	InCommandBindings->MapAction(ListCommands.ImportJobItem,
		FExecuteAction::CreateSP(this, &STmvMediaTranscoder::OnImportJobItemFrom),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscoder::IsItemSelectionValid));

	InCommandBindings->MapAction(ListCommands.ExportJobItem,
		FExecuteAction::CreateSP(this, &STmvMediaTranscoder::OnExportJobItemAs),
		FCanExecuteAction::CreateSP(this, &STmvMediaTranscoder::IsItemSelectionValid));
}

bool STmvMediaTranscoder::CanCreateNewJobList() const
{
	if (JobControls)
	{
		return !JobControls->IsProcessing();
	}
	return false;
}

void STmvMediaTranscoder::OnCreateNewJobList()
{
	if (JobControls && JobControls->IsProcessing())
	{
		return;	
	}

	if (ensureMsgf(TranscodeListHandle, TEXT("Invalid transcode list handle")))
	{
		// Create a new transcode list object.
		TranscodeList.Reset(NewObject<UTmvMediaTranscodeList>(GetTransientPackage(), NAME_None));
		TranscodeListHandle->SetTranscodeList(TranscodeList.Get());
		ResetJobListPath();
	}
}

void STmvMediaTranscoder::OnOpenJobList()
{
	using namespace UE::TmvMediaEditor::Transcoder;

	// Get the saving location
	TArray<FString> Filenames;

	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("OpenJobListDialogTitle", "Open").ToString(),
			GetJobListBrowseDirectory(),
			TEXT(""),
			JobFileTypeFilter,
			EFileDialogFlags::None,
			Filenames
		);
	}

	if (Filenames.Num() > 0)
	{
		// Set the path and load the job list.
		SetJobListPath(Filenames[0]);
		OnLoadJobList();
	}
}

bool STmvMediaTranscoder::CanLoadJobList() const
{
	return !TranscodeListPath.IsEmpty();
}

void STmvMediaTranscoder::OnLoadJobList()
{
	static const FText LoadSuccess = LOCTEXT("LoadListNotificationOk", "Transcode List Loaded Successfully");
	static const FText LoadFailure = LOCTEXT("LoadListNotificationFail", "Loading Transcode List Failed");

	using namespace UE::TmvMediaEditor::Transcoder;
	using namespace UE::TmvMedia::PathUtils;
	bool bLoaded = LoadJobList(ConvertSanitizedPathToFull(TranscodeListPath));
	AddNotification(bLoaded ? LoadSuccess : LoadFailure, bLoaded);
}

bool STmvMediaTranscoder::CanSaveJobList() const
{
	return !TranscodeListPath.IsEmpty();
}

void STmvMediaTranscoder::OnSaveJobList()
{
	static const FText SaveSuccess = LOCTEXT("SaveListNotificationOk", "Transcode List Saved Successfully");
	static const FText SaveFailure = LOCTEXT("SaveListNotificationFail", "Saving Transcode List Failed");

	bool bSaved = false;
	if (TranscodeList.IsValid())
	{
		if (TranscodeListPath.IsEmpty())
		{
			UE_LOGF(LogTmvMediaEditor, Error, "Transcode list path is not specified.");
			return;
		}

		const FString ResolvedTranscodeListPath = UE::TmvMedia::PathUtils::ConvertSanitizedPathToFull(TranscodeListPath);
		IFileManager::Get().MakeDirectory(*FPaths::GetPath(ResolvedTranscodeListPath), /*Tree*/ true);		
		const TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*ResolvedTranscodeListPath));
		if (FileWriter)
		{
			using namespace UE::TmvMedia::TranscodeSerialization; 
			bSaved = SerializeTranscodeListToJson(*FileWriter, *TranscodeList.Get());
			if (!bSaved)
			{
				UE_LOGF(LogTmvMediaEditor, Error, "Failed to serialize to json.");	
			}
		}
		else
		{
			UE_LOGF(LogTmvMediaEditor, Error, "Failed to open \"%ls\" for writing.", *ResolvedTranscodeListPath);
		}
	}

	UE::TmvMediaEditor::Transcoder::AddNotification(bSaved ? SaveSuccess : SaveFailure, bSaved);
}

void STmvMediaTranscoder::OnSaveJobListAs()
{
	using namespace UE::TmvMediaEditor::Transcoder;

	// Get the saving location
	TArray<FString> OutFilenames;

	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("JobListSaveAsDialogTitle", "Save As").ToString(),
			GetJobListBrowseDirectory(),
			TEXT(""),
			JobFileTypeFilter,
			EFileDialogFlags::None,
			OutFilenames
		);
	}

	if (OutFilenames.Num() > 0)
	{
		FString OutFilename = OutFilenames[0];

		// Overwrite prompt
		if (FPaths::FileExists(OutFilename) && !CanOverwriteFile(OutFilename))
		{
			return;
		}

		// Set the path and save the job settings.
		SetJobListPath(OutFilename);
		OnSaveJobList();
	}
}

void STmvMediaTranscoder::OnImportJobItemFrom()
{
	using namespace UE::TmvMediaEditor::Transcoder;

	int32 SelectedItem = GetFirstSelectedItem();

	if (SelectedItem == INDEX_NONE)
	{
		UE_LOGF(LogTmvMediaEditor, Error, "Must select a job item.");
		AddNotification(LOCTEXT("ImportErrorNotification", "Must select a job item as target for importing."), /*Success*/false);
		return;
	}

	// Get the saving location
	TArray<FString> Filenames;

	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("ImportJobItemDialogTitle", "Import Job Item").ToString(),
			GetJobItemBrowseDirectory(),
			TEXT(""),
			JobFileTypeFilter,
			EFileDialogFlags::None,
			Filenames
		);
	}

	if (Filenames.Num() > 0)
	{
		if (LoadJobItem(SelectedItem, Filenames[0]))
		{
			SetLastJobItemPath(Filenames[0]);
		}
	}
}

void STmvMediaTranscoder::OnExportJobItemAs()
{
	using namespace UE::TmvMediaEditor::Transcoder;

	int32 SelectedItem = GetFirstSelectedItem();

	if (SelectedItem == INDEX_NONE)
	{
		UE_LOGF(LogTmvMediaEditor, Error, "Must select a job item.");
		AddNotification(LOCTEXT("ExportErrorNotification", "Must select a job item to export."), /*Success*/false);
		return;
	}
	
	// Get the saving location
	TArray<FString> OutFilenames;

	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("ExportJobItemAsDialogTitle", "Export Job Item As").ToString(),
			GetJobItemBrowseDirectory(),
			TEXT(""),
			JobFileTypeFilter,
			EFileDialogFlags::None,
			OutFilenames
		);
	}

	if (OutFilenames.Num() > 0)
	{
		FString OutFilename = OutFilenames[0];

		// Overwrite prompt
		if (FPaths::FileExists(OutFilename) && !CanOverwriteFile(OutFilename))
		{
			return;
		}

		if (SaveJobItem(SelectedItem, OutFilename))
		{
			SetLastJobItemPath(OutFilename);	// Keep track of the path for convenience.
		}
	}
}

FString STmvMediaTranscoder::GetJobListPath() const
{
	return TranscodeListPath;
}

void STmvMediaTranscoder::SetJobListPath(const FString& InPath)
{
	TranscodeListPath = UE::TmvMedia::PathUtils::GetSanitizedPath(InPath);

	// Persist the setting.
	using namespace UE::TmvMediaEditor::Transcoder;
	GConfig->SetString(*TmvMediaConfigSection, *TranscodeListConfigKey, *TranscodeListPath, GEditorPerProjectIni);
	GConfig->Flush(/*bRemoveFromCache*/ false, GEditorPerProjectIni);
}

void STmvMediaTranscoder::ResetJobListPath()
{
	TranscodeListPath.Reset();

	// Persist the setting.
	using namespace UE::TmvMediaEditor::Transcoder;
	GConfig->SetString(*TmvMediaConfigSection, *TranscodeListConfigKey, *TranscodeListPath, GEditorPerProjectIni);
	GConfig->Flush(/*bRemoveFromCache*/ false, GEditorPerProjectIni);
}

FString STmvMediaTranscoder::GetJobListBrowseDirectory() const
{
	if (!TranscodeListPath.IsEmpty())
	{
		using namespace UE::TmvMedia::PathUtils;
		return FPaths::GetPath(ConvertSanitizedPathToFull(TranscodeListPath));
	}

	return FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
}

bool STmvMediaTranscoder::LoadJobList(const FString& InPath)
{
	const TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InPath));
	if (FileReader)
	{
		using namespace UE::TmvMedia::TranscodeSerialization;
		TStrongObjectPtr<UTmvMediaTranscodeList> NewTranscodeList;
		NewTranscodeList.Reset(NewObject<UTmvMediaTranscodeList>(GetTransientPackage(), NAME_None));
		NewTranscodeList->SetFlags(RF_Public | RF_Transactional);
		
		if (DeserializeTranscodeListFromJson(*FileReader, *NewTranscodeList))
		{
			if (ensureMsgf(TranscodeListHandle, TEXT("Invalid transcode list handle")))
			{
				TranscodeList.Reset(NewTranscodeList.Get());
				TranscodeListHandle->SetTranscodeList(TranscodeList.Get());
				return true;
			}
		}
	}
	else
	{
		UE_LOGF(LogTmvMediaEditor, Error, "Failed to open files \"%ls\".", *InPath);
	}
	return false;
}

int32 STmvMediaTranscoder::GetFirstSelectedItem() const
{
	if (TranscodeListHandle)
	{
		TArray<int32> Selection = TranscodeListHandle->GetCurrentSelection();
		return !Selection.IsEmpty() ? Selection[0] : INDEX_NONE;
	}
	return INDEX_NONE;
}

FString STmvMediaTranscoder::GetLastJobItemPath() const
{
	return LastJobItemPath;
}

void STmvMediaTranscoder::SetLastJobItemPath(const FString& InPath)
{
	LastJobItemPath = UE::TmvMedia::PathUtils::GetSanitizedPath(InPath);

	// Persist the setting.
	using namespace UE::TmvMediaEditor::Transcoder;
	GConfig->SetString(*TmvMediaConfigSection, *LastJobItemConfigKey, *LastJobItemPath, GEditorPerProjectIni);
	GConfig->Flush(/*bRemoveFromCache*/ false, GEditorPerProjectIni);
}

FString STmvMediaTranscoder::GetJobItemBrowseDirectory() const
{
	if (!LastJobItemPath.IsEmpty())
	{
		using namespace UE::TmvMedia::PathUtils;
		return FPaths::GetPath(ConvertSanitizedPathToFull(LastJobItemPath));	
	}

	return GetJobListBrowseDirectory();
}


bool STmvMediaTranscoder::LoadJobItem(int32 InItemIndex, const FString& InJobItemPath)
{
	static const FText LoadSuccess = LOCTEXT("LoadNotificationOk", "Job Item Loaded Successfully");
	static const FText LoadFailure = LOCTEXT("LoadNotificationFail", "Loading Job Item Failed");

	bool bLoaded = false;
	if (TranscodeList && TranscodeList->IsValidItemIndex(InItemIndex))
	{
		const TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InJobItemPath));
		if (FileReader)
		{
			using namespace UE::TmvMedia::TranscodeSerialization;
			FTmvMediaTranscodeJobSettings NewJobSettings;
			TInstancedStruct<FTmvMediaEncoderOptions> NewEncoderOptions;	// Deserializer will initialize.

			if (DeserializeTranscodeJobSettingsFromJson(*FileReader, &NewJobSettings, NewEncoderOptions))
			{
				if (FTmvMediaTranscodeListItem* Item = TranscodeList->GetItemMutable(InItemIndex))
				{
					Item->Settings = NewJobSettings;
					Item->EncoderOptions = NewEncoderOptions;
					bLoaded = true;
					TranscodeList->GetOnItemEvent().Broadcast(TranscodeList.Get(), {ETmvMediaTranscodeListItemEventType::ItemsModified, {InItemIndex}});
				}
			}
		}
	}

	UE::TmvMediaEditor::Transcoder::AddNotification(bLoaded ? LoadSuccess : LoadFailure, bLoaded);	
	return bLoaded;
}

bool STmvMediaTranscoder::SaveJobItem(int32 InItemIndex, const FString& InJobItemPath)
{
	static const FText SaveSuccess = LOCTEXT("SaveNotificationOk", "Config Saved Successfully");
	static const FText SaveFailure = LOCTEXT("SaveNotificationFail", "Saving Config Failed");

	bool bSaved = false;
	if (TranscodeList && TranscodeList->IsValidItemIndex(InItemIndex))
	{
		const FString JobItemFullPath = UE::TmvMedia::PathUtils::ConvertSanitizedPathToFull(InJobItemPath);
		IFileManager::Get().MakeDirectory(*FPaths::GetPath(JobItemFullPath), /*Tree*/ true);
		
		const TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*JobItemFullPath));
		if (FileWriter)
		{
			using namespace UE::TmvMedia::TranscodeSerialization;
			const FTmvMediaTranscodeListItem& JobItem = TranscodeList->GetItem(InItemIndex);
			bSaved = SerializeTranscodeJobSettingsToJson(*FileWriter, &JobItem.Settings, JobItem.EncoderOptions);
			if (!bSaved)
			{
				UE_LOGF(LogTmvMediaEditor, Error, "Failed to serialize to json.");
			}
		}
		else
		{
			UE_LOGF(LogTmvMediaEditor, Error, "Failed to open \"%ls\" for writing.", *JobItemFullPath);
		}
	}

	UE::TmvMediaEditor::Transcoder::AddNotification(bSaved ? SaveSuccess : SaveFailure, bSaved);
	return bSaved;
}

#undef LOCTEXT_NAMESPACE
