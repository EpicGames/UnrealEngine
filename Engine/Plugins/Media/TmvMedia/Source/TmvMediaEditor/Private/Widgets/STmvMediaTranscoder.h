// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TmvMediaTranscodeListHandle.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class STmvMediaTranscoderJobControls;
class UTmvMediaTranscodeList;

/**
 * Tmv Media Transcoder Panel
 *
 * This widget is the main panel widget.
 */
class STmvMediaTranscoder : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STmvMediaTranscoder){}
	SLATE_END_ARGS()

	virtual ~STmvMediaTranscoder() override;

	void Construct(const FArguments& InArgs);

	//~ Begin SWidget
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

private:
	/** Details View Widget switcher handler. Returns which widget index to view. */
	int32 GetDetailsViewWidgetIndex() const;
	
	/**
	 * Allows the editors to bind commands.
	 *
	 * @param InCommandBindings The existing command bindings to map to.
	*/
	void BindCommands(const TSharedPtr<FUICommandList>& InCommandBindings);

	// Job List Command handlers
	bool CanCreateNewJobList() const;
	void OnCreateNewJobList();
	void OnOpenJobList();
	bool CanLoadJobList() const;
	void OnLoadJobList();
	bool CanSaveJobList() const;
	void OnSaveJobList();
	void OnSaveJobListAs();
	void OnImportJobItemFrom();
	void OnExportJobItemAs();

	/** Returns the current job list file path. */
	FString GetJobListPath() const;

	/** Sets the current job list file path. This will persist in the transcoder config settings. */
	void SetJobListPath(const FString& InPath);
	
	/** Reset the current job list file path. This will persist in the transcoder config settings. */
	void ResetJobListPath();

	/** Returns the default browse directory for the job list file dialog(s). */
	FString GetJobListBrowseDirectory() const;

	/** Load a job list from the given file path. Only support json for now. */
	bool LoadJobList(const FString& InPath);

	/** Utility function to retrieve the first selected item. */
	int32 GetFirstSelectedItem() const;

	/** Determines if the selection is valid. */
	bool IsItemSelectionValid() const
	{
		return GetFirstSelectedItem() != INDEX_NONE;
	}

	/** Returns the file path of the last exported job item. */
	FString GetLastJobItemPath() const;

	/** Set the file path of the last exported job item. This will persist in the transcoder config settings. */
	void SetLastJobItemPath(const FString& InPath);

	/** Returns the default browse directory for the job item file dialog(s). */
	FString GetJobItemBrowseDirectory() const;

	/** Load (import) a job item in the current list from the given file path (support json format only for now). */
	bool LoadJobItem(int32 InItemIndex, const FString& InJobItemPath);

	/** Save (export) a job item from the current list to the given file path (support json format only for now). */
	bool SaveJobItem(int32 InItemIndex, const FString& InJobItemPath);

	/** File path to save or load the job settings to or from */
	FString TranscodeListPath;

	/** Keep track of the last (imported or exported) job item path. */
	FString LastJobItemPath;

	/** Currently loaded transcode list. */
	TStrongObjectPtr<UTmvMediaTranscodeList> TranscodeList;

	/** Backing handle for all editing operations on the transcode list. (this should be the "asset editor" object normally) */
	TSharedPtr<FTmvMediaTranscodeListHandle> TranscodeListHandle;

	/** Job Controls (start, stop) toolbar widget. */
	TSharedPtr<STmvMediaTranscoderJobControls> JobControls;

	/** Shared Bound Command list for menus, toolbars and buttons. */
	TSharedPtr<FUICommandList> CommandList;
};
