// Copyright Epic Games, Inc. All Rights Reserved.

#include "STmvMediaTranscoderJobControls.h"

#include "Framework/Notifications/NotificationManager.h"
#include "SlateOptMacros.h"
#include "TmvMediaEditorLog.h"
#include "TmvMediaTranscodeListHandle.h"
#include "TmvMediaTranscodeNotification.h"
#include "TmvMediaTranscodeTaskMakeMediaSource.h"
#include "Transcoder/ITmvMediaTranscodeJobManager.h"
#include "Transcoder/ITmvMediaTranscodeJobRunner.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeJobBuilder.h"
#include "Transcoder/TmvMediaTranscodeList.h"
#include "Utils/TmvMediaMessageContext.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "STmvMediaTranscoderJobControls"

namespace UE::TmvMediaEditor::Transcode
{
	UTmvMediaTranscodeJob* MakeJob(const FTmvMediaTranscodeListItem* InJobItem, FTmvMediaMessageContext* OutMessageContext)
	{
		// Editor-only extension tasks get attached via the post-build hook.
		return FTmvMediaTranscodeJobBuilder(*InJobItem)
			.OnPostBuild(FOnTranscodeJobBuilt::CreateStatic(&UE::TmvMediaEditor::TranscodeTask::AddMakeOrUpdateMediaSourceTask))
			.Build(OutMessageContext);
	}

	/** Show a transient error toaster (fire-and-forget). Used to surface job-builder failures inline with the action. */
	static void ShowBuildErrorNotification(const FText& InErrorText)
	{
		FNotificationInfo Info(InErrorText);
		Info.bFireAndForget = true;
		Info.FadeOutDuration = 0.5f;
		// Match FTmvMediaTranscodeNotification's failure dwell time so users have time to read the message.
		Info.ExpireDuration = 5.0f;
		if (TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info))
		{
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
}

STmvMediaTranscoderJobControls::~STmvMediaTranscoderJobControls()
{
	if (ITmvMediaTranscodeJobRunner* Runner = ITmvMediaTranscodeJobRunner::Get())
	{
		Runner->GetOnJobFinished().RemoveAll(this);
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STmvMediaTranscoderJobControls::Construct(const FArguments& InArgs, const TSharedPtr<FTmvMediaTranscodeListHandle>& InListHandle)
{
	ListHandle = InListHandle;

	if (ITmvMediaTranscodeJobRunner* Runner = ITmvMediaTranscodeJobRunner::Get())
	{
		Runner->GetOnJobFinished().AddSP(this, &STmvMediaTranscoderJobControls::OnRunnerJobFinished);
	}

	ChildSlot
	[
		// Add process images button.
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
					.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
					.IsEnabled(this, &STmvMediaTranscoderJobControls::CanStartJobs)
					.OnClicked(this, &STmvMediaTranscoderJobControls::OnStartJobs)
					.ForegroundColor(FSlateColor::UseStyle())
					.Text(LOCTEXT("StartJobButtonLabel", "Start Selected Job(s)"))
					.ToolTipText(this, &STmvMediaTranscoderJobControls::GetStartJobsToolTip)
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
					.IsEnabled(this, &STmvMediaTranscoderJobControls::CanCancelJobs)
					.OnClicked(this, &STmvMediaTranscoderJobControls::OnCancelJobs)
					.Text(LOCTEXT("CancelJobButtonLabel", "Cancel"))
					.ToolTipText(LOCTEXT("CancelJobButtonToolTip", "Cancel all transcoding jobs."))
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool STmvMediaTranscoderJobControls::IsProcessing() const
{
	const ITmvMediaTranscodeJobRunner* Runner = ITmvMediaTranscodeJobRunner::Get();
	if (!Runner)
	{
		return false;
	}

	for (const FGuid& JobId : SubmittedJobIds)
	{
		if (Runner->IsJobActiveOrPending(JobId))
		{
			return true;
		}
	}
	return false;
}

bool STmvMediaTranscoderJobControls::IsJobProcessingOrPending(const FGuid& InJobId) const
{
	if (!SubmittedJobIds.Contains(InJobId))
	{
		return false;
	}
	const ITmvMediaTranscodeJobRunner* Runner = ITmvMediaTranscodeJobRunner::Get();
	return Runner && Runner->IsJobActiveOrPending(InJobId);
}

FText STmvMediaTranscoderJobControls::GetStartJobsToolTip() const
{
	if (CanStartJobs())
	{
		return LOCTEXT("StartJobButtonToolTip", "Start transcoding selected jobs.");
	}

	// If the jobs can't be started, find why and drop a hint to the user in the tool tip.
	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	if (!List)
	{
		return LOCTEXT("StartJobButtonToolTip_InvalidListHandle", "Error: Invalid List Handle.");
	}

	FTextBuilder Reasons;
	Reasons.Indent();	// Indent our list of reasons.

	const TArray<int32> CurrentSelection = ListHandle->GetCurrentSelection();

	// Check that we don't have any items already started or that is not configured to run.
	for (const int32 SelectedItem : CurrentSelection)
	{
		if (List->IsValidItemIndex(SelectedItem))
		{
			const FTmvMediaTranscodeListItem& JobItem = List->GetItem(SelectedItem);
			if (!CanStartJobItem(JobItem))
			{
				if (IsJobProcessingOrPending(JobItem.Id))
				{
					Reasons.AppendLineFormat(LOCTEXT("JobFailReason_IsProcessing", "-Item \"{0}\": Is already processing or pending."), FText::FromString(JobItem.Name));
				}
				else
				{
					if (!JobItem.Settings.IsInputPathSet())
					{
						Reasons.AppendLineFormat(LOCTEXT("JobFailReason_NoInputPath", "-Item \"{0}\": Input Path Not Set."), FText::FromString(JobItem.Name));
					}
					if (!JobItem.Settings.IsOutputPathSet())
					{
						Reasons.AppendLineFormat(LOCTEXT("JobFailReason_NoOutputPath", "-Item \"{0}\": Output Path Not Set."), FText::FromString(JobItem.Name));
					}
					if (JobItem.Settings.bMakeOutputAsset && JobItem.Settings.OutputAssetDirectory.Path.IsEmpty())
					{
						Reasons.AppendLineFormat(LOCTEXT("JobFailReason_NoOutputAssetDirectory", "-Item \"{0}\": Output Asset Directory Not Set."), FText::FromString(JobItem.Name));
					}
				}

				// Protection for too many items.
				if (Reasons.GetNumLines() > 20)
				{
					Reasons.AppendLine(LOCTEXT("JobFailReason_Ellipsis", "..."));
					break;
				}
			}
		}
	}

	if (CurrentSelection.IsEmpty() || Reasons.GetNumLines() == 0)
	{
		Reasons.AppendLine(LOCTEXT("StartJobButtonToolTip_NoSelectedJobs", "-Job selection is empty."));
	}

	return FText::Format(LOCTEXT("StartJobButtonToolTip_JobFailReasons", "Cannot start transcoding selected jobs:\n{0}"), Reasons.ToText());
}

bool STmvMediaTranscoderJobControls::CanStartJobItem(const FTmvMediaTranscodeListItem& InJobItem) const
{
	return UTmvMediaTranscodeList::ValidateJobItem(InJobItem)
		&& !IsJobProcessingOrPending(InJobItem.Id);
}

bool STmvMediaTranscoderJobControls::CanStartJobs() const
{
	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	if (!List)
	{
		return false;
	}

	bool bHasStartableJobs = false;

	// Check that we don't have any items already started or that is not configured to run.
	for (const int32 SelectedItem : ListHandle->GetCurrentSelection())
	{
		if (List->IsValidItemIndex(SelectedItem))
		{
			if (!CanStartJobItem(List->GetItem(SelectedItem)))
			{
				return false;
			}

			bHasStartableJobs = true;
		}
	}

	return bHasStartableJobs;
}

FReply STmvMediaTranscoderJobControls::OnStartJobs()
{
	ITmvMediaTranscodeJobRunner* Runner = ITmvMediaTranscodeJobRunner::Get();
	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	if (!Runner || !List)
	{
		return FReply::Handled();
	}

	// Attach a fresh notification handler only at the moment each job is about to start, so
	// queued jobs don't show an empty toast until they actually run.
	FTmvMediaTranscodeJobRunOptions Options;
	Options.OnPreStart = FOnTranscodeJobStarting::CreateLambda([](UTmvMediaTranscodeJob* InJob)
	{
		if (InJob)
		{
			InJob->SetNotificationHandler(MakeShared<FTmvMediaTranscodeNotificationSafe>());
		}
	});

	const TArray<int32> SelectedItems = ListHandle->GetCurrentSelection();
	for (int32 SelectedItem : SelectedItems)
	{
		if (!List->IsValidItemIndex(SelectedItem))
		{
			continue;
		}

		const FTmvMediaTranscodeListItem& JobItem = List->GetItem(SelectedItem);
		if (!CanStartJobItem(JobItem))
		{
			continue;
		}

		FTmvMediaMessageContext BuildErrors;
		UTmvMediaTranscodeJob* NewJob = UE::TmvMediaEditor::Transcode::MakeJob(&JobItem, &BuildErrors);

		// Surface build errors as a toaster notification.
		if (!BuildErrors.Messages.IsEmpty())
		{
			UE::TmvMediaEditor::Transcode::ShowBuildErrorNotification(BuildErrors.ToText());
		}

		if (!NewJob)
		{
			continue;
		}

		SubmittedJobIds.Add(NewJob->GetId());
		Runner->EnqueueJob(NewJob, Options);
	}

	return FReply::Handled();
}

bool STmvMediaTranscoderJobControls::CanCancelJobs() const
{
	return IsProcessing();
}

FReply STmvMediaTranscoderJobControls::OnCancelJobs()
{
	ITmvMediaTranscodeJobRunner* Runner = ITmvMediaTranscodeJobRunner::Get();
	if (!Runner)
	{
		return FReply::Handled();
	}

	// Snapshot since OnJobFinished prunes SubmittedJobIds during the cancel loop.
	const TArray<FGuid> ToCancel = SubmittedJobIds.Array();

	// Drain pending entries first: cancelling the active job promotes the next pending via
	// OnPreStart, which flashes an empty CS_Pending toast before we can cancel it.
	const ITmvMediaTranscodeJobManager* Manager = ITmvMediaTranscodeJobManager::Get();
	FGuid ActiveId;
	for (const FGuid& JobId : ToCancel)
	{
		const UTmvMediaTranscodeJob* Job = Manager ? Manager->GetTranscodeJob(JobId) : nullptr;
		if (Job && Job->IsRunning())
		{
			ActiveId = JobId;
		}
		else
		{
			Runner->CancelJob(JobId);
		}
	}
	if (ActiveId.IsValid())
	{
		Runner->CancelJob(ActiveId);
	}

	return FReply::Handled();
}

void STmvMediaTranscoderJobControls::OnRunnerJobFinished(UTmvMediaTranscodeJob* InJob)
{
	if (InJob)
	{
		SubmittedJobIds.Remove(InJob->GetId());
	}
}

#undef LOCTEXT_NAMESPACE
