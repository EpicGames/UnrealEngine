// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Input/Reply.h"
#include "Misc/Guid.h"
#include "Widgets/SCompoundWidget.h"

class UTmvMediaTranscodeJob;
struct FTmvMediaTranscodeListHandle;
struct FTmvMediaTranscodeListItem;

/**
 * Tmv Transcoder job controls widget
 */
class STmvMediaTranscoderJobControls : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STmvMediaTranscoderJobControls){}
	SLATE_END_ARGS()

	virtual ~STmvMediaTranscoderJobControls() override;

	void Construct(const FArguments& InArgs, const TSharedPtr<FTmvMediaTranscodeListHandle>& InListHandle);

	/** Returns true if at least one job submitted by this widget is still active or pending. */
	bool IsProcessing() const;

private:
	/** Returns true if the given job is one we submitted and the runner still has it active or pending. */
	bool IsJobProcessingOrPending(const FGuid& InJobId) const;

	/** Returns the Start Jobs button tool tip. */
	FText GetStartJobsToolTip() const;

	/** Returns true if the given job item can be started. */
	bool CanStartJobItem(const FTmvMediaTranscodeListItem& InJobItem) const;

	/** Returns true if the currently selected set of jobs can be started. */
	bool CanStartJobs() const;

	/** Called when we click on the start jobs button. */
	FReply OnStartJobs();

	/** Returns true if the current set of running or pending jobs can be cancelled. */
	bool CanCancelJobs() const;

	/** Called when we click on the cancel button. */
	FReply OnCancelJobs();

	void OnRunnerJobFinished(UTmvMediaTranscodeJob* InJob);

	/** Reference to the transcode list */
	TSharedPtr<FTmvMediaTranscodeListHandle> ListHandle;

	/**
	 * Ids of jobs this widget enqueued in the global runner. Used to scope can-start /
	 * cancel queries to jobs we own, so we ignore (and don't cancel) jobs from other callers.
	 */
	TSet<FGuid> SubmittedJobIds;
};
