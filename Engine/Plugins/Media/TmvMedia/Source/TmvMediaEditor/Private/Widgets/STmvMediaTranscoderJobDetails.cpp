// Copyright Epic Games, Inc. All Rights Reserved.

#include "STmvMediaTranscoderJobDetails.h"

#include "Customizations/TmvMediaDirectoryPathCustomization.h"
#include "Customizations/TmvMediaEncoderOptionsEnumCustomization.h"
#include "Customizations/TmvMediaFilePathCustomization.h"
#include "Customizations/TmvMediaMuxerSettingsCustomization.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Encoder/TmvMediaEncoderOptions.h"
#include "IStructureDataProvider.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "TmvMediaEditorLog.h"
#include "TmvMediaEditorTranscodeUtils.h"
#include "TmvMediaTranscodeListHandle.h"
#include "Transcoder/ITmvMediaTranscodeJobManager.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeList.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "STmvMediaTranscoderJobDetails"

namespace UE::TmvMediaEditor::TranscoderJobDetails
{
	/**
	 * Utility function to gather the StructOnScope wrappers of the given Job Items sub-structure.
	 * @param InListHandle Transcode list handle
	 * @param InSelectedItemIndices Job Item indices to gather from.
	 * @param InMakeStructOnScopeForItemFunc Function to make the StructOnScope wrapper from the job item.
	 */
	TArray<TSharedPtr<FStructOnScope>> GatherStructOnScopes(
		const TSharedPtr<FTmvMediaTranscodeListHandle>& InListHandle,
		const TConstArrayView<int32> InSelectedItemIndices,
		TFunctionRef<TSharedPtr<FStructOnScope>(FTmvMediaTranscodeListItem*)> InMakeStructOnScopeForItemFunc)
	{
		TArray<TSharedPtr<FStructOnScope>> OutStructOnScopes;

		UTmvMediaTranscodeList* List = InListHandle ? InListHandle->Get() : nullptr;
		if (!List)
		{
			return OutStructOnScopes;
		}

		for(const int32 JobItemIndex : InSelectedItemIndices)
		{
			if (FTmvMediaTranscodeListItem* JobItem = List->GetItemMutable(JobItemIndex))
			{
				if (TSharedPtr<FStructOnScope> StructOnScope = InMakeStructOnScopeForItemFunc(JobItem))
				{
					OutStructOnScopes.Add(MoveTemp(StructOnScope));	
				}
			}
		}
		return OutStructOnScopes;
	}
}

STmvMediaTranscoderJobDetails::~STmvMediaTranscoderJobDetails()
{
	if (ListHandle)
	{
		if (UTmvMediaTranscodeList* List = ListHandle->Get())
		{
			List->GetOnItemEvent().RemoveAll(this);
		}
		ListHandle->GetOnTranscodeListChanged().RemoveAll(this);
		ListHandle->GetOnSelectionChanged().RemoveAll(this);
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STmvMediaTranscoderJobDetails::Construct(const FArguments& InArgs, const TSharedPtr<FTmvMediaTranscodeListHandle>& InListHandle)
{
	ListHandle = InListHandle;
	CurrentSelection.Reset(); // nothing selected on construction.

	if (ListHandle)
	{
		if (UTmvMediaTranscodeList* List = ListHandle->Get())
		{
			List->GetOnItemEvent().AddSP(this, &STmvMediaTranscoderJobDetails::OnTranscodeListItemEvent);
		}
		ListHandle->GetOnSelectionChanged().AddSP(this, &STmvMediaTranscoderJobDetails::OnJobItemSelectionChanged);
		ListHandle->GetOnTranscodeListChanged().AddSP(this, &STmvMediaTranscoderJobDetails::OnTranscodeListChanged);
	}
	
	// Set up widgets.
	TSharedPtr<SBox> JobDetailsViewBox;
	TSharedPtr<SBox> EncoderDetailsViewBox;

	SetupEncoderOptionsSelection();
	
	ChildSlot
	[
		SNew(SVerticalBox)
		.IsEnabled(this, &STmvMediaTranscoderJobDetails::IsDetailsEnabled)

		// Add details view.
		+ SVerticalBox::Slot()
		.AutoHeight()
			[
				SAssignNew(JobDetailsViewBox, SBox)
			]
		+ SVerticalBox::Slot()
		.AutoHeight()
			[
				// Encoder selector
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 3.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TmvMediaEncoderSelectionLabel", "Encoder"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(15.0f, 0.0f, 0.0f, 0.0f)
				[
					SAssignNew(EncoderOptionsComboBox, SComboBox<const UScriptStruct*>)
					.OptionsSource(&EncoderOptionsStructs)
					.InitiallySelectedItem(GetEncoderOptionsItem())
					.OnGenerateWidget(this, &STmvMediaTranscoderJobDetails::MakeEncoderOptionsItemWidget)
					.OnSelectionChanged(this, &STmvMediaTranscoderJobDetails::OnEncoderOptionsSelectionChanged)
					.ContentPadding(2.f)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &STmvMediaTranscoderJobDetails::GetEncoderOptionsContent)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			]
		+ SVerticalBox::Slot()
		.AutoHeight()
			[
				SAssignNew(EncoderDetailsViewBox, SBox)
			]
	];

	RefreshJobSettingsDetailsView();
	if (JobDetailsView && JobDetailsViewBox)
	{
		JobDetailsViewBox->SetContent(JobDetailsView->GetWidget().ToSharedRef());
	}
	
	RefreshEncoderOptionsDetailsView();
	if (EncoderDetailsView && EncoderDetailsViewBox)
	{
		EncoderDetailsViewBox->SetContent(EncoderDetailsView->GetWidget().ToSharedRef());
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool STmvMediaTranscoderJobDetails::IsDetailsEnabled() const
{
	// The whole details panel is disabled if any of the selected jobs are either queued or running.
	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	const ITmvMediaTranscodeJobManager* TranscodeJobManager = ITmvMediaTranscodeJobManager::Get();
	if (List && !CurrentSelection.IsEmpty() && TranscodeJobManager)
	{
		for (const int32 SelectedJobItemIndex : CurrentSelection)
		{
			const FTmvMediaTranscodeListItem& Item = List->GetItem(SelectedJobItemIndex);
			if (TranscodeJobManager->GetTranscodeJob(Item.Id))
			{
				// If a job is already queued, it won't be editable anymore (settings are copied in the queued job).
				return false;
			}
		}
	}
	return true;
}

void STmvMediaTranscoderJobDetails::OnJobItemSelectionChanged(const UTmvMediaTranscodeList* InList, TConstArrayView<int32> InSelectedItems)
{
	CurrentSelection = InSelectedItems;
	RefreshJobSettingsDetailsView();
	RefreshEncoderOptionsDetailsView();
	RefreshEncoderOptionsCombo();
}

void STmvMediaTranscoderJobDetails::OnTranscodeListItemEvent(const UTmvMediaTranscodeList* InList, const FTmvMediaTranscodeListItemEventArgs& InArgs)
{
	if (InArgs.Type == ETmvMediaTranscodeListItemEventType::ItemsModified && bIgnoreModifiedItemEvents)
	{
		return;
	}
	
	// Note: when items are added, it may reallocate the array and the memory pointed to by
	// the detail views will become dangling. Views must be refreshed.
	RefreshJobSettingsDetailsView();
	RefreshEncoderOptionsDetailsView();
	RefreshEncoderOptionsCombo();
}

void STmvMediaTranscoderJobDetails::OnTranscodeListChanged(UTmvMediaTranscodeList* InPreviousList, UTmvMediaTranscodeList* InNewList)
{
	if (InPreviousList)
	{
		InPreviousList->GetOnItemEvent().RemoveAll(this);
	}
	if (InNewList)
	{
		InNewList->GetOnItemEvent().AddSP(this, &STmvMediaTranscoderJobDetails::OnTranscodeListItemEvent);
	}

	RefreshJobSettingsDetailsView();
	RefreshEncoderOptionsDetailsView();
	RefreshEncoderOptionsCombo();
}

void STmvMediaTranscoderJobDetails::OnFinishedChangingProperties(const FPropertyChangedEvent& InChangedEvent)
{
	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	if (List && !CurrentSelection.IsEmpty())
	{
		TGuardValue TickGuard(bIgnoreModifiedItemEvents, true);	// Ignore events coming from self.
		FTmvMediaTranscodeListItemEventArgs Args;
		Args.Type = ETmvMediaTranscodeListItemEventType::ItemsModified;
		Args.ItemIndices = CurrentSelection;
		List->GetOnItemEvent().Broadcast(List, Args);
	}

	// End Transaction
	CurrentTransaction.Reset();
}

void STmvMediaTranscoderJobDetails::NotifyPreChange(FEditPropertyChain* InPropertyAboutToChange)
{
	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	if (List)
	{
		// Begin Transaction, if not already in one
		if (!CurrentTransaction.IsValid())
		{
			CurrentTransaction = MakeShared<FScopedTransaction>(LOCTEXT("EditTranscodeJobs", "Edit Transcode Jobs"));
		}
		List->Modify();
	}
}

void STmvMediaTranscoderJobDetails::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	// Note: notifying the list is currently done in OnFinishedChangingProperties. We don't have a per property notify hook for now.
}

void STmvMediaTranscoderJobDetails::RefreshJobSettingsDetailsView()
{
	using namespace UE::TmvMediaEditor::TranscoderJobDetails;
	const TSharedRef<FStructOnScopeStructureDataProvider> StructProvider = MakeShared<FStructOnScopeStructureDataProvider>();
	StructProvider->SetStructData(GatherStructOnScopes(ListHandle, CurrentSelection, [](FTmvMediaTranscodeListItem* InJobItem)
	{
		return MakeShared<FStructOnScope>(FTmvMediaTranscodeJobSettings::StaticStruct(), reinterpret_cast<uint8*>(&InJobItem->Settings));
	}));
	
	if (!JobDetailsView)
	{
		JobDetailsView = UE::TmvMediaEditor::Transcode::CreateStructureDetailView(StructProvider, this);
		if (JobDetailsView && JobDetailsView->GetDetailsView())
		{
			// Add our filepath and directory path customizations that supports drag and drop and path resolving.
			JobDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(FName(TEXT("FilePath")),
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(FTmvMediaFilePathCustomization::MakeInstance));
			JobDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(FName(TEXT("DirectoryPath")),
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(FTmvMediaDirectoryPathCustomization::MakeInstance));
			JobDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(FName(TEXT("TmvMediaTranscodeMuxerSettings")),
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(FTmvMediaMuxerSettingsCustomization::MakeInstance));

			JobDetailsView->GetDetailsView()->OnFinishedChangingProperties().AddSP(this, &STmvMediaTranscoderJobDetails::OnFinishedChangingProperties);
		}

		if (!JobDetailsView)
		{
			UE_LOGF(LogTmvMediaEditor, Error, "Failed to create job options detail view.");
		}
	}
	else
	{
		JobDetailsView->SetStructureProvider(StructProvider);
	}
}

void STmvMediaTranscoderJobDetails::RefreshEncoderOptionsDetailsView()
{
	using namespace UE::TmvMediaEditor::TranscoderJobDetails;
	const TSharedRef<FStructOnScopeStructureDataProvider> StructProvider = MakeShared<FStructOnScopeStructureDataProvider>();
	StructProvider->SetStructData(GatherStructOnScopes(ListHandle, CurrentSelection, [](FTmvMediaTranscodeListItem* InJobItem)
	{
		return (InJobItem && InJobItem->EncoderOptions.IsValid()) ?
			MakeShared<FStructOnScope>(InJobItem->EncoderOptions.GetScriptStruct(), InJobItem->EncoderOptions.GetMutableMemory()) : TSharedPtr<FStructOnScope>();
	}));
	
	if (!EncoderDetailsView)
	{
		EncoderDetailsView = UE::TmvMediaEditor::Transcode::CreateStructureDetailView(StructProvider, this);
		if (EncoderDetailsView && EncoderDetailsView->GetDetailsView())
		{
			// Filter the DestinationColorSpace / DestinationEncoding dropdowns to the values the active encoder
			// can actually produce (queried via FTmvMediaEncoderOptions virtuals). The identifier scopes the
			// customization to those two property names only, so other enum uses in the view are unaffected.
			const TSharedRef<FTmvMediaEncoderOptionsEnumIdentifier> Identifier = MakeShared<FTmvMediaEncoderOptionsEnumIdentifier>();
			EncoderDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(
				TEXT("ETextureColorSpace"),
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTmvMediaEncoderOptionsEnumCustomization::MakeInstance),
				Identifier);
			EncoderDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(
				TEXT("ETmvMediaEncoderEncoding"),
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTmvMediaEncoderOptionsEnumCustomization::MakeInstance),
				Identifier);
		}
		else
		{
			UE_LOGF(LogTmvMediaEditor, Error, "Failed to create encoder options detail view.");
		}
	}
	else
	{
		EncoderDetailsView->SetStructureProvider(StructProvider);
	}
}

void STmvMediaTranscoderJobDetails::RefreshEncoderOptionsCombo()
{
	if (EncoderOptionsComboBox)
	{
		if (const UScriptStruct* SelectedItem = GetEncoderOptionsItem())
		{
			EncoderOptionsComboBox->SetSelectedItem(SelectedItem);
		}
	}
}

void STmvMediaTranscoderJobDetails::SetupEncoderOptionsSelection()
{
	EncoderOptionsStructs = UE::TmvMediaEditor::Transcode::GetAllDerivedStruct(FTmvMediaEncoderOptions::StaticStruct());
}

const UScriptStruct* STmvMediaTranscoderJobDetails::GetEncoderOptionsItem() const
{
	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	if (!List)
	{
		return nullptr;
	}

	const UScriptStruct* SelectedStruct = nullptr;
	
	for (int32 SelectedJobIndex : CurrentSelection)
	{
		if (FTmvMediaTranscodeListItem* SelectedJobItem = List->GetItemMutable(SelectedJobIndex))
		{
			if (!SelectedStruct)
			{
				SelectedStruct = SelectedJobItem->EncoderOptions.GetScriptStruct();
			}
			else if (SelectedStruct != SelectedJobItem->EncoderOptions.GetScriptStruct())
			{
				return nullptr;	// Multiple values
			}
		}
	}

	return SelectedStruct;
}

FText STmvMediaTranscoderJobDetails::GetEncoderOptionsContent() const
{
	const UScriptStruct* SelectedItem = GetEncoderOptionsItem();
	return SelectedItem ? SelectedItem->GetDisplayNameText() : NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
}

TSharedRef<SWidget> STmvMediaTranscoderJobDetails::MakeEncoderOptionsItemWidget(const UScriptStruct* InItem)
{
	return SNew(STextBlock)
		.Text(InItem ? InItem->GetDisplayNameText() : FText::GetEmpty())
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void STmvMediaTranscoderJobDetails::OnEncoderOptionsSelectionChanged(const UScriptStruct* InNewSelection, ESelectInfo::Type InSelectInfo)
{
	// Ignore if already transacting.
	if (!InNewSelection || GIsTransacting)
	{
		return;
	}

	UTmvMediaTranscodeList* List = ListHandle ? ListHandle->Get() : nullptr;
	if (!List)
	{
		UE_LOGF(LogTmvMediaEditor, Error, "Invalid Transcode Job List.");
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SetEncoderOptions", "Set Encoder Options"));
	List->Modify();
	
	int32 NumModifiedJobs = 0;

	for (int32 SelectedJobIndex : CurrentSelection)
	{
		FTmvMediaTranscodeListItem* SelectedJobItem = List->GetItemMutable(SelectedJobIndex);	
		if (!SelectedJobItem)
		{
			continue;
		}

		if (SelectedJobItem->EncoderOptions.GetScriptStruct() != InNewSelection)
		{
			if (EncoderDetailsView && NumModifiedJobs == 0)
			{
				EncoderDetailsView->SetStructureProvider(MakeShared<FStructOnScopeStructureDataProvider>());	// Clear to avoid dangling.
			}
			SelectedJobItem->EncoderOptions.InitializeAsScriptStruct(InNewSelection);
			++NumModifiedJobs;
		}
	}

	if (NumModifiedJobs)
	{
		RefreshEncoderOptionsDetailsView();
	}
	else
	{
		Transaction.Cancel();
	}
}

#undef LOCTEXT_NAMESPACE
