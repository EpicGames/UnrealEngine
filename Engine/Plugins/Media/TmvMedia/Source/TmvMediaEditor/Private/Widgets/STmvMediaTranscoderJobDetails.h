// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

class FScopedTransaction;
class IStructureDetailsView;
class UTmvMediaTranscodeList;
struct FTmvMediaTranscodeListHandle;
struct FTmvMediaTranscodeListItemEventArgs;

/**
 * Transcoder Job Settings and Encoder Options Detail views for the Tmv transcoder list editor. 
 */
class STmvMediaTranscoderJobDetails : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(STmvMediaTranscoderJobDetails){}
	SLATE_END_ARGS()

	virtual ~STmvMediaTranscoderJobDetails() override;
	
	void Construct(const FArguments& InArgs, const TSharedPtr<FTmvMediaTranscodeListHandle>& InListHandle);

private:
	/** Returns true if the detail panel should be enabled for editing. */
	bool IsDetailsEnabled() const;

	/** Handles transcode list selection changed to update the details view. */
	void OnJobItemSelectionChanged(const UTmvMediaTranscodeList* InList, TConstArrayView<int32> InSelectedItems);

	/** Handles transcode list event (add, remove, modified, etc) to update the details view. */
	void OnTranscodeListItemEvent(const UTmvMediaTranscodeList* InList, const FTmvMediaTranscodeListItemEventArgs& InArgs);

	/** Handles transcode list changed (i.e. a new list is loaded) to update the details view. */
	void OnTranscodeListChanged(UTmvMediaTranscodeList* InPreviousList, UTmvMediaTranscodeList* InNewList);

	/** Details View event handling. Propagates the detail view event. */
	void OnFinishedChangingProperties(const FPropertyChangedEvent& InChangedEvent);

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FEditPropertyChain* InPropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged) override;
	//~ End FNotifyHook

	/** Refresh (create or update) the Job Setting Details View. */
	void RefreshJobSettingsDetailsView();

	/** Refresh (create or update) the Encoder Options Details View. */
	void RefreshEncoderOptionsDetailsView();

	/** Encoder Options Combo Box - Make sure the encoder option combo box is reflecting the current job item. */
	void RefreshEncoderOptionsCombo();

	/** Encoder Options Combo Box - Populate Encoder Options Structs list for Combo Box. */
	void SetupEncoderOptionsSelection();

	/**
	 * Encoder Options Combo Box - Returns the selected "item" from the backing list.
	 * @return the currently selected struct or null if multiple different values.
	 */
	const UScriptStruct* GetEncoderOptionsItem() const;

	/** Encoder Options Combo Box - Returns the text to display in the combo box content. */
	FText GetEncoderOptionsContent() const;

	/** Encoder Options Combo Box - Returns the widget (a text box) for the given item. */
	TSharedRef<SWidget> MakeEncoderOptionsItemWidget(const UScriptStruct* InItem);

	/** Encoder Options Combo Box - Handles updating the job item and details view when selection change. */  
	void OnEncoderOptionsSelectionChanged(const UScriptStruct* InNewSelection, ESelectInfo::Type InSelectInfo);

	/** Handle to the currently opened Transcode List. */
	TSharedPtr<FTmvMediaTranscodeListHandle> ListHandle;

	/** Currently selected Job Item Indices (in the job list) that is reflected by this detail view. */
	TArray<int32> CurrentSelection;

	/** Current property editing transaction (from any of the details view). */
	TSharedPtr<FScopedTransaction> CurrentTransaction;
	
	/** Flag indicating the modified items events are coming from the detail panel itself and should be ignored. */
	bool bIgnoreModifiedItemEvents = false;
	
	/** Holds the "Job Settings" details view. */
	TSharedPtr<IStructureDetailsView> JobDetailsView;
	
	/** Holds the "Encoder Options" details view. */
	TSharedPtr<IStructureDetailsView> EncoderDetailsView;

	/** Reference to Encoder Options ComboBox widget to propagate "refresh" events. */
	TSharedPtr<SComboBox<const UScriptStruct*>> EncoderOptionsComboBox;

	/** Cached list of the child ScriptStruct of EncoderOptions struct to the encoder options combo box. */
	TArray<const UScriptStruct*> EncoderOptionsStructs;
};
