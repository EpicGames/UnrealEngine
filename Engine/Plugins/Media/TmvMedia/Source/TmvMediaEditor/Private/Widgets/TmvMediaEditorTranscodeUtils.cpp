// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaEditorTranscodeUtils.h"

#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "UObject/UObjectIterator.h"

namespace UE::TmvMediaEditor::Transcode
{
	TArray<const UScriptStruct*> GetAllDerivedStruct(const UScriptStruct* InBaseStruct)
	{
		TArray<const UScriptStruct*> OutStructs;

		if (InBaseStruct == nullptr)
		{
			return OutStructs;
		}

		OutStructs.Reserve(2);

		for (const UScriptStruct* CurrentStruct : TObjectRange<UScriptStruct>())
		{
			if (CurrentStruct != InBaseStruct && CurrentStruct->IsChildOf(InBaseStruct))
			{
				OutStructs.Add(CurrentStruct);
			}
		}
		return OutStructs;
	}

	TSharedPtr<IStructureDetailsView> CreateStructureDetailView(const TSharedPtr<IStructureDataProvider>& InStructProvider, FNotifyHook* InNotifyHook)
	{
		FStructureDetailsViewArgs StructureViewArgs;
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;

		FDetailsViewArgs ViewArgs;
		ViewArgs.bAllowSearch = false;
		ViewArgs.bHideSelectionTip = false;
		ViewArgs.bShowObjectLabel = false;
		ViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		ViewArgs.NotifyHook = InNotifyHook;

		FPropertyEditorModule& PropertyEditor = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		return PropertyEditor.CreateStructureProviderDetailView(ViewArgs, StructureViewArgs, InStructProvider);
	}
}