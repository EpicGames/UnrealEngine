// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class FNotifyHook;
class IStructureDataProvider;
class IStructureDetailsView;
class UScriptStruct;

namespace UE::TmvMediaEditor::Transcode
{
	/** Enumerate all the derived struct. */
	TArray<const UScriptStruct*> GetAllDerivedStruct(const UScriptStruct* InBaseStruct);

	/** Helper function to create a struct detail view for transcode job details panel. */
	TSharedPtr<IStructureDetailsView> CreateStructureDetailView(const TSharedPtr<IStructureDataProvider>& InStructProvider, FNotifyHook* InNotifyHook);
}
