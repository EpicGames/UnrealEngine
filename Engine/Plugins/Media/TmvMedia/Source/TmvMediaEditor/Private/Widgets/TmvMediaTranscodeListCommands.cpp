// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaTranscodeListCommands.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "TmvMediaTranscodeListCommands"

FTmvMediaTranscodeListCommands::FTmvMediaTranscodeListCommands()
	: TCommands<FTmvMediaTranscodeListCommands>(TEXT("TmvMediaTranscodeList")
		, LOCTEXT("TmvMediaTranscodeListCommands", "Tmv Media Transcode List")
		, NAME_None
		, FAppStyle::GetAppStyleSetName())
{
	
}

void FTmvMediaTranscodeListCommands::RegisterCommands()
{
	UI_COMMAND(CreateNewJobList
		, "New"
		, "Create a new job list"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(OpenJobList
		, "Open..."
		, "Open Job list from file"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(LoadJobList
		, "Reload"
		, "Load Job list from current file"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SaveJobList
		, "Save"
		, "Save Job list to file"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control, EKeys::S));
	
	UI_COMMAND(SaveJobListAs
		, "Save As..."
		, "Save Job List to a new file"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ImportJobItem
		, "Import Item..."
		, "Load the selected item from file"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ExportJobItem
		, "Export Item..."
		, "Save the selected item to file"
		, EUserInterfaceActionType::Button
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE