// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

/**
* Set of Transcode List Editor Commands.
*/
class FTmvMediaTranscodeListCommands : public TCommands<FTmvMediaTranscodeListCommands>
{
public:	
	FTmvMediaTranscodeListCommands();

	/** Initialize commands */
	virtual void RegisterCommands() override;

	/** Create a new Transcode Job List.  */
	TSharedPtr<FUICommandInfo> CreateNewJobList;

	/** Open a Transcode Job List from a new file or asset. */
	TSharedPtr<FUICommandInfo> OpenJobList;

	/** Load Transcode Job List from current file. */
	TSharedPtr<FUICommandInfo> LoadJobList;

	/** Save Transcode Job List to current file. */
	TSharedPtr<FUICommandInfo> SaveJobList;

	/** Save Transcode Job Lis to a new file (from browser). */
	TSharedPtr<FUICommandInfo> SaveJobListAs;

	/** Import/Load selected item from specified file (from browser). */
	TSharedPtr<FUICommandInfo> ImportJobItem;

	/** Export/Save selected item to specified existing or new file. */
	TSharedPtr<FUICommandInfo> ExportJobItem;
};