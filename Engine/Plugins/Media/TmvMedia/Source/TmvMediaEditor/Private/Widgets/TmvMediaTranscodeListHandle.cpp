// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaTranscodeListHandle.h"

void FTmvMediaTranscodeListHandle::SetTranscodeList(UTmvMediaTranscodeList* InTranscodeList)
{
	UTmvMediaTranscodeList* PreviousList = TranscodeListWeak.Get();
	TranscodeListWeak = InTranscodeList;
	OnListChanged.Broadcast(PreviousList, InTranscodeList);
}

