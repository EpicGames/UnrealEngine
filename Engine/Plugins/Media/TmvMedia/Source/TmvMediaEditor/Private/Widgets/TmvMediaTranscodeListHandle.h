// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Transcoder/TmvMediaTranscodeList.h"
#include "UObject/WeakObjectPtrTemplates.h"

/** 
 * Keeps an indirect reference to the edited transcode list so it can be changed, 
 * on reload for instance, and avoid reconstructing all widgets.
 * 
 * Note: In a MVC pattern, this object would be the "controller", but that word is already used
 * for other concepts in UE so it is avoided. Using "handle" for now, might change later. As this is
 * likely to be refactored into an asset editor, the post fix "editor" is typically used for what this class is.
 */
struct FTmvMediaTranscodeListHandle : TSharedFromThis<FTmvMediaTranscodeListHandle>
{
	/** Returns true if this handle has a valid reference to the edited transcode list. */
	bool IsValid() const
	{
		return TranscodeListWeak.IsValid();
	}
	
	/** Returns the edited transcode list. */
	UTmvMediaTranscodeList* Get() const
	{
		return TranscodeListWeak.Get();
	}

	/** Returns a strong reference to the edited transcode list. */
	TStrongObjectPtr<UTmvMediaTranscodeList> Pin() const
	{
		return TranscodeListWeak.Pin();
	}

	/** Assigned a new transcode list object to this handle. This will broadcast OnListChanged event for widgets to update. */
	void SetTranscodeList(UTmvMediaTranscodeList* InTranscodeList);

	/** Multicast Delegate called when the transcode list object is changed. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnListChanged, UTmvMediaTranscodeList* /*PreviousList*/, UTmvMediaTranscodeList* /*NewList*/);
	FOnListChanged& GetOnTranscodeListChanged()
	{
		return OnListChanged;
	}

	/** Multicast Delegate called when the JobItem Selection changes. */ 
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSelectionChanged, const UTmvMediaTranscodeList*, TConstArrayView<int32> /*SelectedItems*/);
	FOnSelectionChanged& GetOnSelectionChanged()
	{
		return OnSelectionChanged;
	}

	/** Delegate implemented by the job list widget to query the current selection. */
	DECLARE_DELEGATE_RetVal(TArray<int32>, FOnGetCurrentSelection)
	FOnGetCurrentSelection OnGetCurrentSelectionDelegate;

	/** Returns currently selected job items in the list view. */
	TArray<int32> GetCurrentSelection() const
	{
		return OnGetCurrentSelectionDelegate.IsBound() ? OnGetCurrentSelectionDelegate.Execute() : TArray<int32>();
	}

	/** Delegate implemented by the job list widget to query the current number of selected items. */
	DECLARE_DELEGATE_RetVal(int32, FOnGetNumSelected)
	FOnGetNumSelected OnGetNumSelectedDelegate;

	/** Returns number of currently selected job items in the list view. */ 
	int32 GetNumSelected() const
	{
		return OnGetNumSelectedDelegate.IsBound() ? OnGetNumSelectedDelegate.Execute() : 0;
	}
	
private:
	/** This handle keeps a weak pointer to the edited transcode list. The owner should be the top level parent widget. */
	TWeakObjectPtr<UTmvMediaTranscodeList> TranscodeListWeak;
	
	/** OnListChanged Delegate. */ 
	FOnListChanged OnListChanged;
	
	/** OnSelectionChanged Delegate. */
	FOnSelectionChanged OnSelectionChanged;
};
