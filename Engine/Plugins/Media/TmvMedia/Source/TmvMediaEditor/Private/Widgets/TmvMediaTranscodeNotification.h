// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Widgets/Notifications/SNotificationList.h"

/** 
 * Implementation of a transcode notification using SNotificationItem.
 * This class is not meant to be used directly, but rather, wrapped by
 * another object that will make sure the functions are called from the main thread.
 */
class FTmvMediaTranscodeNotification : public ITmvMediaTranscodeNotification
{
public:
	FTmvMediaTranscodeNotification()
	{
		FNotificationInfo Info(FText::GetEmpty());
		Info.bFireAndForget = false;

		NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
	}

	virtual ~FTmvMediaTranscodeNotification() override
	{
		// Skip if slate is already torn down.
		if (FSlateApplication::IsInitialized())
		{
			CloseImpl(/*bInSuccess*/ false);
		}
	}

	//~ Begin ITmvMediaTranscodeNotification
	virtual void SetText(const FText& InText) override
	{
		if (NotificationItem)
		{
			NotificationItem->SetText(InText);
		}
	}

	virtual void Close(bool bInSuccess) override
	{
		CloseImpl(bInSuccess);
	}
	//~ End ITmvMediaTranscodeNotification

private:
	/** Closes the notification. */
	void CloseImpl(bool bInSuccess)
	{
		if (NotificationItem)
		{
			NotificationItem->SetEnabled(false);
			NotificationItem->SetCompletionState(bInSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
			if (!bInSuccess)
			{
				// Leave more time to read error messages.
				NotificationItem->SetExpireDuration(2.5f);
			}
			NotificationItem->ExpireAndFadeout();
			NotificationItem.Reset();
		}
	}

	TSharedPtr<SNotificationItem> NotificationItem;
};

/** 
* Transcode Notification implementation that takes a shared ptr to another implementation
* and makes sure that the functions are called in the main thread.
*/
class FTmvMediaTranscodeNotificationWrapper : public ITmvMediaTranscodeNotification
{
public:
	FTmvMediaTranscodeNotificationWrapper(const TSharedPtr<ITmvMediaTranscodeNotification>& InInternal)
		: Internal(InInternal)
	{}

	virtual ~FTmvMediaTranscodeNotificationWrapper() override
	{
		// Ensure the internal object is released in the main thread.
		if (!IsInGameThread() && Internal)
		{
			Async(EAsyncExecution::TaskGraphMainThread, [ObjetToRelease = MoveTemp(Internal)] () mutable
			{
				ObjetToRelease.Reset();
			});
		}
	}

	//~ Begin ITmvMediaTranscodeNotification
	virtual void SetText(const FText& InText) override
	{
		if (Internal)
		{
			ExecuteOnGameThread([InternalWeak = Internal.ToWeakPtr(), InText]()
			{
				if (const TSharedPtr<ITmvMediaTranscodeNotification> Internal = InternalWeak.Pin())
				{
					Internal->SetText(InText);
				}
			});
		}
	}

	virtual void Close(bool bInSuccess) override
	{
		if (Internal)
		{
			ExecuteOnGameThread([InternalWeak = Internal.ToWeakPtr(), bInSuccess]()
			{
				if (const TSharedPtr<ITmvMediaTranscodeNotification> Internal = InternalWeak.Pin())
				{
					Internal->Close(bInSuccess);
				}
			});
		}
	}
	//~ End ITmvMediaTranscodeNotification

private:
	/** Call a callable on the game thread. */
	template <typename CallableType>
	void ExecuteOnGameThread(CallableType&& Callable)
	{
		if (IsInGameThread())
		{
			Callable();
		}
		else
		{
			Async(EAsyncExecution::TaskGraphMainThread, Forward<CallableType>(Callable));
		}
	}

	/** Internal object that implements the functions that must be called from the main thread. */
	TSharedPtr<ITmvMediaTranscodeNotification> Internal;
};

class FTmvMediaTranscodeNotificationSafe : public FTmvMediaTranscodeNotificationWrapper
{
public:
	FTmvMediaTranscodeNotificationSafe()
		: FTmvMediaTranscodeNotificationWrapper(MakeShared<FTmvMediaTranscodeNotification>())
	{}
};