// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaTranscodeTaskMakeMediaSource.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Encoder/TmvMediaEncoderOptions.h"
#include "HAL/FileManager.h"
#include "MediaSource.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "ObjectTools.h"
#include "TmvMediaEditorLog.h"
#include "Transcoder/TmvMediaFrameEncoder.h"
#include "Transcoder/TmvMediaFrameProducer.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeMuxer.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "Utils/TmvMediaPathUtils.h"

namespace UE::TmvMediaEditor::TranscodeTask
{
	/**
	 * Creates a MediaSource asset at InFullAssetPath pointing to InMediaPath.
	 *
	 * Uses UMediaSource::SpawnMediaSourceForString to create the concrete media source type
	 * (typically UImgMediaSource or UFileMediaSource) based on the file extension of the output files.
	 */
	UMediaSource* CreateMediaSourceAsset(const FString& InFullAssetPath, const FString& InMediaPath, const FString& InFileExtension, UPackage* InPackage)
	{
		FString ResolvedFilePath = InMediaPath;

		if (FPaths::DirectoryExists(InMediaPath))
		{
			// Find any file in the output directory, filtered on the given extension if available, to pass to SpawnMediaSourceForString.
			TArray<FString> FoundFiles;

			IFileManager::Get().FindFiles(FoundFiles, *InMediaPath, !InFileExtension.IsEmpty() ? *InFileExtension : nullptr);

			// Do a minimum of clean up to get a valid file.
			FoundFiles.RemoveAll([](const FString& InFileName) { return InFileName.StartsWith(TEXT(".")); });
			FoundFiles.Sort();

			if (FoundFiles.IsEmpty())
			{
				UE_LOGF(LogTmvMediaEditor, Warning,
					"MakeOrUpdateMediaSourceAsset: No output files found in '%ls'; skipping media source creation.",
					*InMediaPath);
				return nullptr;
			}

			ResolvedFilePath = InMediaPath / FoundFiles[0];
		}

		// Make sure the resolved file path exists
		if (!FPaths::FileExists(ResolvedFilePath))
		{
			UE_LOGF(LogTmvMediaEditor, Warning,
				"MakeOrUpdateMediaSourceAsset: '%ls' does not exist; skipping media source creation.",
				*ResolvedFilePath);
			return nullptr;
		}

		UMediaSource* MediaSource = UMediaSource::SpawnMediaSourceForString(ResolvedFilePath, InPackage);
		if (!MediaSource)
		{
			UE_LOGF(LogTmvMediaEditor, Warning,
				"MakeOrUpdateMediaSourceAsset: SpawnMediaSourceForString returned null for '%ls'.",
				*ResolvedFilePath);
			return nullptr;
		}

		const FString AssetName = FPackageName::GetShortName(InFullAssetPath);
		MediaSource->Rename(*AssetName, InPackage);
		MediaSource->SetFlags(RF_Public | RF_Standalone);
		MediaSource->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(MediaSource);

		return MediaSource;
	}

	/** Possible results of the set property operations. */ 
	enum class EPropertyResult : int8
	{
		/** Update operation failed: missing parameter, wrong type, etc. */
		Failed = -1,
		/** Update operation done successfully. */
		Done = 0,
		/** Value was already up to date and nothing needed to be done. */
		UpToDate = 1
	};

	/**
	 * Use the blueprint function from the known media source classes to set the media path.
	 * Supporting UImgMediaSource "SetSequencePath" and UFileMediaSource "SetFilePath". 
	 * @param InMediaSource Media Source to configure 
	 * @param InPath Desired path to be set as the media source's media file or sequence path.
	 * @return Either "Failed" if the known functions are not found or have the wrong signature, or "Done" otherwise.
	 */
	EPropertyResult SetMediaSourceMediaPath(UMediaSource* InMediaSource, const FString& InPath)
	{
		// Try to find UImgMediaSource "SetSequencePath".
		UFunction* Func = InMediaSource->FindFunction(TEXT("SetSequencePath"));

		// Try to find UFileMediaSource "SetFilePath", which as the same signature.
		if (!Func)
		{
			Func = InMediaSource->FindFunction(TEXT("SetFilePath"));
		}

		if (!Func)
		{
			return EPropertyResult::Failed;
		}

		// Allocate a buffer matching the function's actual param layout so we don't
		// hard-code assumptions about the UHT-generated params struct.
		TArray<uint8> ParamBuffer;
		ParamBuffer.AddZeroed(Func->ParmsSize);

		// Initialize all params and locate the Path parameter by name + type.
		bool bFoundPath = false;
		for (TFieldIterator<FProperty> It(Func); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->InitializeValue_InContainer(ParamBuffer.GetData());

			if (!bFoundPath && It->GetFName() == TEXT("Path"))
			{
				if (FStrProperty* StrProp = CastField<FStrProperty>(*It))
				{
					StrProp->SetPropertyValue_InContainer(ParamBuffer.GetData(), InPath);
					bFoundPath = true;
				}
			}
		}

		if (!bFoundPath)
		{
			UE_LOGF(LogTmvMediaEditor, Warning,
				"SetMediaSourceSequencePath: \"%ls\" signature has changed; \"Path\" param not found on \"%ls\".",
				*Func->GetName(), *InMediaSource->GetClass()->GetName());
		}
		else
		{
			InMediaSource->ProcessEvent(Func, ParamBuffer.GetData());
			InMediaSource->MarkPackageDirty();
		}

		// Destroy param values — important for FString to release heap memory.
		for (TFieldIterator<FProperty> It(Func); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->DestroyValue_InContainer(ParamBuffer.GetData());
		}
		
		return bFoundPath ? EPropertyResult::Done : EPropertyResult::Failed;
	}

	/**
	 * Utility function to set the "FrameRateOverride" property of the media source.
	 * 
	 * @param InMediaSource Media Source to configure 
	 * @param InFrameRate Desired frame rate to set in the media source.
	 * @return	Either "Done" if the frame rate was set, "UpToDate" if the frame rate was already the desired value, 
	 *			or "Failed" if the property was not found.
	 */
	EPropertyResult SetMediaSourceFrameRate(UMediaSource* InMediaSource, const FFrameRate& InFrameRate)
	{
		// Update the framerate in the img media source.
		if (FStructProperty* const FrameRateProperty = FindFProperty<FStructProperty>(InMediaSource->GetClass(), TEXT("FrameRateOverride")))
		{
			if (void* PropertyValuePtr = FrameRateProperty->ContainerPtrToValuePtr<FFrameRate>(InMediaSource, 0))
			{
				// Read the value
				FFrameRate PropertyValue;
				FrameRateProperty->CopyCompleteValue(&PropertyValue, PropertyValuePtr);
				
				if (PropertyValue != InFrameRate)
				{
					FrameRateProperty->CopyCompleteValue(PropertyValuePtr, &InFrameRate);

					// notifies editor/content browser
					FPropertyChangedEvent ChangeEvent(FrameRateProperty);
					InMediaSource->PostEditChangeProperty(ChangeEvent);

					InMediaSource->MarkPackageDirty();
					return EPropertyResult::Done;
				}
				return EPropertyResult::UpToDate;
			}
		}
		return EPropertyResult::Failed;
	}

	/** Save the given media source to file. */
	bool SaveMediaSourceAsset(UPackage* InPackage, UMediaSource* InMediaSource, const FString& InFullAssetPath)
	{
		const FString PackageFilename = FPackageName::LongPackageNameToFilename(InFullAssetPath, FPackageName::GetAssetPackageExtension());
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Public | RF_Standalone;
		return UPackage::SavePackage(InPackage, InMediaSource, *PackageFilename, SaveArgs);
	}

	const FString& GetValidString(const FString& InString, const FString& InDefault)
	{
		return InString.IsEmpty() ? InDefault : InString;
	}

	/** Make a safe asset name from the job settings. */
	FString MakeMediaSourceFullAssetName(const UTmvMediaTranscodeJob* InTranscodeJob)
	{
		// Using the job name as the media source name. If it is empty, we use a default name.
		const FString MediaSourceName = GetValidString(InTranscodeJob->JobName, TEXT("TranscodeOutput"));
		const FString MediaSourcePath = GetValidString(InTranscodeJob->Settings.OutputAssetDirectory.Path, TEXT("/Game"));
		return MediaSourcePath / MediaSourceName;
	}

	/**
	 * Creates or replaces a MediaSource asset at InAssetPath pointing to InSequencePath.
	 */
	void MakeOrUpdateMediaSourceAsset(UTmvMediaTranscodeJob* InTranscodeJob)
	{
		// Only perform the task if the job succeeded without errors and was not canceled.
		const bool bJobSucceeded = !InTranscodeJob->bIsError && InTranscodeJob->StopReason == ETmvMediaTranscodeJobStopReason::Completed;

		if (bJobSucceeded && InTranscodeJob->Settings.bMakeOutputAsset && !InTranscodeJob->Settings.OutputAssetDirectory.Path.IsEmpty())
		{
			// This is the fallback setting, but should be overriden by the media file's frame rate.
			FFrameRate FrameRate = InTranscodeJob->Settings.GetInputFramerate();

			// Fetch the frame rate from the frame producer, which is going to be what the media file was.
			if (UTmvMediaFrameProducer* FrameProducer = InTranscodeJob->GetStage<UTmvMediaFrameProducer>())
			{
				const FTmvMediaFrameProducerTrackInfo VideoTrackInfo = FrameProducer->GetVideoTrackInfo();
				FrameRate = VideoTrackInfo.FrameRate;
			}

			// Using the job name as the media source name.
			const FString FullMediaSourcePath = MakeMediaSourceFullAssetName(InTranscodeJob);
			const FString MediaSourceName = FPackageName::GetShortName(FullMediaSourcePath);
			const bool bIsContainerOutput = InTranscodeJob->Settings.OutputFormat == ETmvMediaTranscodeOutputFormat::Container;

			// For containers, the media path is the output file (e.g. OutputPath/BaseName.tmv).
			// For file sequences, the media path is the output directory.
			FString FullMediaPath;
			if (bIsContainerOutput)
			{
				FullMediaPath = UTmvMediaTranscodeMuxer::GetContainerOutputFilePath(InTranscodeJob);
			}
			else
			{
				FullMediaPath = InTranscodeJob->Settings.GetAbsoluteOutputPath();
			}

			UPackage* MediaSourcePackage = CreatePackage(*FullMediaSourcePath);

			if (!MediaSourcePackage)
			{
				UE_LOGF(LogTmvMediaEditor, Error, "Unable to create media source \"%ls\" ... ", *FullMediaSourcePath);
				return;
			}
			
			// We may have to save this package, so it has to be fully loaded.
			MediaSourcePackage->FullyLoad();

			// See if we have an existing media source.
			UMediaSource* MediaSource = Cast<UMediaSource>(FindObject<UObject>(MediaSourcePackage, *MediaSourceName));

			// Accumulator for existing media source(s) that might be discarded if it can't be updated.
			TArray<UObject*> DiscardedMediaSourcesToConsolidate;
			
			bool bNeedSaveMediaSource = false;

			// Existing media source
			if (MediaSource)
			{
				using namespace UE::TmvMedia::PathUtils;
				// Check if the sequence path is already correct.
				FString ExistingMediaPath = GetMediaSourceMediaFullPath(MediaSource);
				
				// Try to set the path
				if (!FPaths::IsSamePath(ExistingMediaPath, FullMediaPath))
				{
					bNeedSaveMediaSource = true; // At least the path will be changed.

					// Try to update the sequence path.
					const EPropertyResult SetPathResult = SetMediaSourceMediaPath(MediaSource, FullMediaPath);

					// If we couldn't update it, we will have to discard this object and create a new one.
					if (SetPathResult == EPropertyResult::Failed || !FPaths::IsSamePath(GetMediaSourceMediaFullPath(MediaSource), FullMediaPath))
					{
						// Fallback method:
						// We will discard the current media source and create a new one with the correct path and class.
						// However, we will need to consolidate in memory objects to update references.
						MediaSource->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
						DiscardedMediaSourcesToConsolidate.Add(MediaSource);
						MediaSource = nullptr;
					}
				}
			}

			// Create new media source.
			if (!MediaSource)
			{
				// Retrieve the produced file extension.
				// For containers, use the container extension (e.g. "tmv").
				// For file sequences, use the per-frame codec extension (e.g. "apv1").
				FString FileExtension;
				if (bIsContainerOutput)
				{
					FileExtension = FPaths::GetExtension(FullMediaPath);
				}
				else if (const UTmvMediaFrameEncoder* EncoderStage = InTranscodeJob->GetStage<UTmvMediaFrameEncoder>())
				{
					if (const FTmvMediaEncoderOptions* EncoderOptions = EncoderStage->GetEncoderOptions())
					{
						FileExtension = EncoderOptions->GetFileSequenceExtension();
					}
				}

				MediaSource = CreateMediaSourceAsset(FullMediaSourcePath, FullMediaPath, FileExtension, MediaSourcePackage);
				if (!MediaSource)
				{
					return; // CreateMediaSourceAsset already logged an error message.
				}
				bNeedSaveMediaSource = true;

				// Consolidate in memory objects to update references, etc.
				if (!DiscardedMediaSourcesToConsolidate.IsEmpty())
				{
					UE_LOGF(LogTmvMediaEditor, Log, "Consolidating references to new media source \"%ls\" ... ", *FullMediaSourcePath);
					ObjectTools::ConsolidateObjects(MediaSource, DiscardedMediaSourcesToConsolidate, /*bShowDeleteConfirmation*/ false);
				}
			}

			if (MediaSource)
			{
				if (SetMediaSourceFrameRate(MediaSource, FrameRate) == EPropertyResult::Done)
				{
					bNeedSaveMediaSource = true; // Frame rate was modified, will need saving.
				}

				// Only save the asset if it was created of modified to avoid invoking source control if not needed.
				if (bNeedSaveMediaSource)
				{
					if (!SaveMediaSourceAsset(MediaSourcePackage, MediaSource, FullMediaSourcePath))
					{
						UE_LOGF(LogTmvMediaEditor, Error, "Failed to save media source \"%ls\"", *FullMediaSourcePath);
					}
				}
			}
		}
	}

	void AddMakeOrUpdateMediaSourceTask(UTmvMediaTranscodeJob* InTranscodeJob)
	{
		InTranscodeJob->GetOnTranscodeJobFinished().AddStatic(&MakeOrUpdateMediaSourceAsset);
	}
}
