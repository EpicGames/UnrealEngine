// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class UTmvMediaTranscodeJob;

namespace UE::TmvMediaEditor::TranscodeTask
{
	/**
	 * Adds a task that will create/update a MediaSource asset for the output sequence when the job is finished.
	 * @param InTranscodeJob Transcode job to add the task to.
	 */
	void AddMakeOrUpdateMediaSourceTask(UTmvMediaTranscodeJob* InTranscodeJob);
}
