// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class TmvMediaEditor : ModuleRules
	{
		public TmvMediaEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ApplicationCore",
					"AssetRegistry",
					"ContentBrowserData",
					"Core",
					"CoreUObject",
					"DesktopWidgets",
					"EditorWidgets",
					"InputCore",
					"MediaAssets",
					"Projects",
					"PropertyEditor",
					"Slate",
					"SlateCore",
					"TmvMedia",
					"UnrealEd",
					"WorkspaceMenuStructure",
				});
		}
	}
}
