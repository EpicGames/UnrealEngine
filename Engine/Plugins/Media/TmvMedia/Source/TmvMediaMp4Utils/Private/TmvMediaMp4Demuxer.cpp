// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaMp4Demuxer.h"
#include "MP4Boxes.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "TmvMediaMp4UtilsLog.h"

namespace UE::TmvMedia
{

FTmvMediaMp4Demuxer::~FTmvMediaMp4Demuxer()
{
	Close();
}

ETmvMediaContainerResult FTmvMediaMp4Demuxer::OpenFile(const FString& InFilePath)
{
	Close();

	TSharedPtr<MP4Utilities::FMP4FileDataReader, ESPMode::ThreadSafe> FileReader = MP4Utilities::FMP4FileDataReader::Create();
	if (!FileReader.IsValid())
	{
		LastError = TEXT("Failed to create MP4 file data reader.");
		return ETmvMediaContainerResult::Fail;
	}

	if (!FileReader->Open(InFilePath))
	{
		LastError = FString::Printf(TEXT("Failed to open file: %s"), *InFilePath);
		return ETmvMediaContainerResult::Fail;
	}

	DataReader = FileReader;

	// Locate and read root boxes from the file.
	MP4Utilities::FMP4BoxLocator Locator;
	// Read these root-level boxes: ftyp, moov, moof, sidx. Stop scanning after mdat in standard files.
	const TArray<uint32> ReadBoxes = {
		MP4Utilities::MakeBoxAtom('f','t','y','p'),
		MP4Utilities::MakeBoxAtom('m','o','o','v'),
		MP4Utilities::MakeBoxAtom('m','o','o','f'),
		MP4Utilities::MakeBoxAtom('s','i','d','x'),
	};

	if (!Locator.LocateAndReadRootBoxes(
		RootBoxes,
		DataReader,
		TArray<uint32>(),   // InFirstBoxes (no order constraint)
		TArray<uint32>(),   // InStopAfterBoxes
		false,              // bStopWithMDAT
		ReadBoxes,          // InReadDataOfBoxes
		MP4Utilities::IMP4DataReaderBase::FCancellationCheckDelegate()))
	{
		LastError = FString::Printf(TEXT("Failed to locate root boxes: %s"), *Locator.GetLastError());
		return ETmvMediaContainerResult::Fail;
	}

	return ParseContainer();
}

ETmvMediaContainerResult FTmvMediaMp4Demuxer::OpenBuffer(TConstArrayView<uint8> InData)
{
	Close();

	if (!MP4Utilities::FMP4BoxLocator::LocateAndReadRootBoxesFromBuffer(RootBoxes, InData))
	{
		LastError = TEXT("Failed to parse root boxes from buffer.");
		return ETmvMediaContainerResult::Fail;
	}

	// Create a buffer data reader for sample data access.
	DataReader = MP4Utilities::FMP4BufferDataReader::Create(InData);

	return ParseContainer();
}

ETmvMediaContainerResult FTmvMediaMp4Demuxer::ParseContainer()
{
	// Find the moov root box.
	TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe> MoovBoxData;
	TArray<TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>> MoofBoxDatas;

	for (const TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>& Box : RootBoxes)
	{
		if (Box->Type == MP4Utilities::MakeBoxAtom('m','o','o','v'))
		{
			MoovBoxData = Box;
		}
		else if (Box->Type == MP4Utilities::MakeBoxAtom('m','o','o','f'))
		{
			MoofBoxDatas.Add(Box);
		}
	}

	if (!MoovBoxData.IsValid())
	{
		LastError = TEXT("No 'moov' box found in container.");
		return ETmvMediaContainerResult::Fail;
	}

	// Parse the moov box tree.
	MP4Boxes::FMP4BoxTreeParser MoovParser;
	if (!MoovParser.ParseBoxTree(MoovBoxData))
	{
		LastError = TEXT("Failed to parse 'moov' box tree.");
		return ETmvMediaContainerResult::Fail;
	}

	TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe> MoovTree = MoovParser.GetBoxTree();
	if (!MoovTree.IsValid())
	{
		LastError = TEXT("Empty 'moov' box tree.");
		return ETmvMediaContainerResult::Fail;
	}

	// Get movie header for overall duration.
	TSharedPtr<MP4Boxes::FMP4BoxMVHD, ESPMode::ThreadSafe> MvhdBox =
		MoovTree->FindBoxRecursive<MP4Boxes::FMP4BoxMVHD>(MP4Utilities::MakeBoxAtom('m','v','h','d'));
	if (MvhdBox.IsValid())
	{
		MovieDuration = MvhdBox->GetDuration();
	}

	// Parse moof boxes for fragmented files.
	TArray<TSharedPtr<MP4Boxes::FMP4BoxBase>> AllRootBoxes;
	AllRootBoxes.Add(MoovTree);
	for (const TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>& MoofData : MoofBoxDatas)
	{
		MP4Boxes::FMP4BoxTreeParser MoofParser;
		if (MoofParser.ParseBoxTree(MoofData))
		{
			AllRootBoxes.Add(MoofParser.GetBoxTree());
		}
	}

	// Find mvex box for fragmented files.
	TSharedPtr<MP4Boxes::FMP4BoxMVEX, ESPMode::ThreadSafe> MvexBox =
		MoovTree->FindBoxRecursive<MP4Boxes::FMP4BoxMVEX>(MP4Utilities::MakeBoxAtom('m','v','e','x'));

	// Find all trak boxes within moov.
	TArray<TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>> TrakBoxes;
	MoovTree->GetAllBoxInstances<MP4Boxes::FMP4BoxTRAK>(TrakBoxes, MP4Utilities::MakeBoxAtom('t','r','a','k'));

	int32 TrackIndex = 0;
	for (const TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>& TrakBox : TrakBoxes)
	{
		// Build fragment info only for fragmented files (mvex + moof present).
		TSharedPtr<MP4Boxes::FMP4Track::FFragmentInfo, ESPMode::ThreadSafe> FragmentInfo;
		if (MvexBox.IsValid())
		{
			FragmentInfo = MakeShared<MP4Boxes::FMP4Track::FFragmentInfo, ESPMode::ThreadSafe>();
			FragmentInfo->Prepare(TrakBox, MvexBox, AllRootBoxes);
		}

		// Create the track.
		TSharedPtr<MP4Boxes::FMP4Track, ESPMode::ThreadSafe> Track =
			MP4Boxes::FMP4Track::Create(TrakBox, FragmentInfo);

		if (!Track->Prepare(MovieDuration, MovieDuration))
		{
			UE_LOGF(LogTmvMediaMp4Utils, Warning, "FTmvMediaMp4Demuxer: Failed to prepare track %d: %ls", TrackIndex, *Track->GetLastError());
			++TrackIndex;
			continue;
		}

		FTrackState State;
		State.Track = Track;
		State.Iterator = Track->CreateIterator();

		// Build cached track info.
		FTmvMediaDemuxerTrackInfo& Info = State.CachedInfo;
		Info.TrackIndex = TrackStates.Num();

		// Determine track type from handler and STSD.
		// Skip timecode tracks - they are parsed separately for metadata.
		TSharedPtr<MP4Boxes::FMP4BoxHDLR, ESPMode::ThreadSafe> HdlrBox =
			TrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxHDLR>(MP4Utilities::MakeBoxAtom('h','d','l','r'));
		if (HdlrBox.IsValid())
		{
			uint32 HandlerType = HdlrBox->GetHandlerType();
			if (HandlerType == MP4Utilities::MakeBoxAtom('t','m','c','d'))
			{
				++TrackIndex;
				continue;
			}
			else if (HandlerType == MP4Utilities::MakeBoxAtom('v','i','d','e'))
			{
				Info.TrackType = ETmvMediaTrackType::Video;
			}
			else if (HandlerType == MP4Utilities::MakeBoxAtom('s','o','u','n'))
			{
				Info.TrackType = ETmvMediaTrackType::Audio;
			}
		}

		// Get timescale and duration from mdhd.
		TSharedPtr<MP4Boxes::FMP4BoxMDHD, ESPMode::ThreadSafe> MdhdBox =
			TrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxMDHD>(MP4Utilities::MakeBoxAtom('m','d','h','d'));
		if (MdhdBox.IsValid())
		{
			Info.Timescale = MdhdBox->GetTimescale();
			MP4Utilities::FFractionalTime Duration = MdhdBox->GetDuration();
			Info.Duration = Duration.GetNumerator();
			Info.LanguageCode = MdhdBox->GetLanguageCode639_2T();
		}

		Info.NumSamples = Track->GetNumberOfSamples();

		// Get constant sample duration from stts if all samples share the same delta.
		TSharedPtr<MP4Boxes::FMP4BoxSTTS, ESPMode::ThreadSafe> SttsBox =
			TrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxSTTS>(MP4Utilities::MakeBoxAtom('s','t','t','s'));
		if (SttsBox.IsValid())
		{
			const auto& Entries = SttsBox->GetEntries();
			if (Entries.Num() == 1)
			{
				Info.ConstantSampleDuration = Entries[0].sample_delta;
			}
		}

		// Get sample entry format and type-specific info from stsd.
		TSharedPtr<MP4Boxes::FMP4BoxSTSD, ESPMode::ThreadSafe> StsdBox =
			TrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxSTSD>(MP4Utilities::MakeBoxAtom('s','t','s','d'));
		if (StsdBox.IsValid())
		{
			// GetSampleType() must be called first - it triggers parsing of STSD children
			// into typed sample entry objects (FMP4BoxVisualSampleEntry, etc.).
			MP4Boxes::FMP4BoxSampleEntry::ESampleType SampleType = StsdBox->GetSampleType();

			if (SampleType == MP4Boxes::FMP4BoxSampleEntry::ESampleType::Video)
			{
				TArray<TSharedPtr<MP4Boxes::FMP4BoxVisualSampleEntry, ESPMode::ThreadSafe>> VisualEntries;
				StsdBox->GetSampleDescriptions(VisualEntries);
				if (VisualEntries.Num() > 0)
				{
					Info.SampleEntryFormat = VisualEntries[0]->GetType();
					Info.DisplayWidth = VisualEntries[0]->GetWidth();
					Info.DisplayHeight = VisualEntries[0]->GetHeight();

					for (const TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>& ChildBox : VisualEntries[0]->GetChildren())
					{
						Info.CodecSpecificBoxes.Add(ChildBox->GetBoxDataRAW());
					}
				}
			}
			else if (SampleType == MP4Boxes::FMP4BoxSampleEntry::ESampleType::Audio)
			{
				TArray<TSharedPtr<MP4Boxes::FMP4BoxAudioSampleEntry, ESPMode::ThreadSafe>> AudioEntries;
				StsdBox->GetSampleDescriptions(AudioEntries);
				if (AudioEntries.Num() > 0)
				{
					Info.SampleEntryFormat = AudioEntries[0]->GetType();
					Info.SamplingRate = AudioEntries[0]->GetSampleRate();
					Info.NumberOfChannels = static_cast<uint16>(AudioEntries[0]->GetChannelCount());

					for (const TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>& ChildBox : AudioEntries[0]->GetChildren())
					{
						Info.CodecSpecificBoxes.Add(ChildBox->GetBoxDataRAW());
					}
				}
			}
		}

		// Get track name from metadata.
		const MP4Boxes::FMP4TrackMetadataCommon& Metadata = Track->GetCommonMetadata();
		Info.TrackName = Metadata.Name;
		if (!Metadata.LanguageTag.IsEmpty())
		{
			Info.LanguageCode = Metadata.LanguageTag;
		}

		// Get dimensions from tkhd as fallback for video tracks.
		if (Info.TrackType == ETmvMediaTrackType::Video && (Info.DisplayWidth == 0 || Info.DisplayHeight == 0))
		{
			TSharedPtr<MP4Boxes::FMP4BoxTKHD, ESPMode::ThreadSafe> TkhdBox =
				TrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxTKHD>(MP4Utilities::MakeBoxAtom('t','k','h','d'));
			if (TkhdBox.IsValid())
			{
				Info.DisplayWidth = TkhdBox->GetWidth();
				Info.DisplayHeight = TkhdBox->GetHeight();
			}
		}

		TrackStates.Add(MoveTemp(State));
		++TrackIndex;
	}

	if (TrackStates.IsEmpty())
	{
		LastError = TEXT("No valid tracks found in container.");
		return ETmvMediaContainerResult::Fail;
	}

	// Extract start timecode from tmcd tracks or udta metadata.
	ParseStartTimecode(MoovTree);

	return ETmvMediaContainerResult::Success;
}

int32 FTmvMediaMp4Demuxer::GetTrackCount() const
{
	return TrackStates.Num();
}

ETmvMediaContainerResult FTmvMediaMp4Demuxer::GetTrackInfo(int32 InTrackIndex, FTmvMediaDemuxerTrackInfo& OutTrackInfo) const
{
	if (InTrackIndex < 0 || InTrackIndex >= TrackStates.Num())
	{
		return ETmvMediaContainerResult::Fail;
	}

	OutTrackInfo = TrackStates[InTrackIndex].CachedInfo;
	return ETmvMediaContainerResult::Success;
}

ETmvMediaContainerResult FTmvMediaMp4Demuxer::ReadSample(int32 InTrackIndex, FTmvMediaDemuxerSample& OutSample)
{
	if (InTrackIndex < 0 || InTrackIndex >= TrackStates.Num())
	{
		return ETmvMediaContainerResult::Fail;
	}

	FTrackState& State = TrackStates[InTrackIndex];
	if (!State.Iterator.IsValid() || !State.Iterator->IsValid())
	{
		return ETmvMediaContainerResult::EndOfStream;
	}

	ETmvMediaContainerResult Result = PopulateSampleFromIterator(State.Iterator, OutSample, /*bReadData=*/ true);

	// Always advance the iterator regardless of success or failure.
	// On failure, skipping the problematic sample prevents the caller from getting
	// stuck in an infinite retry loop on the same broken sample.
	if (!State.Iterator->Next())
	{
		// Iterator exhausted, next call will return EndOfStream.
	}

	return Result;
}

ETmvMediaContainerResult FTmvMediaMp4Demuxer::ReadSampleInfo(int32 InTrackIndex, FTmvMediaDemuxerSample& OutSample)
{
	if (InTrackIndex < 0 || InTrackIndex >= TrackStates.Num())
	{
		return ETmvMediaContainerResult::Fail;
	}

	FTrackState& State = TrackStates[InTrackIndex];
	if (!State.Iterator.IsValid() || !State.Iterator->IsValid())
	{
		return ETmvMediaContainerResult::EndOfStream;
	}

	ETmvMediaContainerResult Result = PopulateSampleFromIterator(State.Iterator, OutSample, /*bReadData=*/ false);

	// Always advance the iterator regardless of success or failure.
	if (!State.Iterator->Next())
	{
		// Iterator exhausted, next call will return EndOfStream.
	}

	return Result;
}

ETmvMediaContainerResult FTmvMediaMp4Demuxer::PopulateSampleFromIterator(
	const TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe>& InIterator,
	FTmvMediaDemuxerSample& OutSample,
	bool bReadData)
{
	OutSample.SampleNumber = InIterator->GetSampleNumber();
	OutSample.DTS = InIterator->GetDTS().GetNumerator();
	OutSample.PTS = InIterator->GetPTS().GetNumerator();
	OutSample.Duration = static_cast<uint32>(InIterator->GetDuration().GetNumerator());
	OutSample.bIsKeyframe = InIterator->IsSyncOrRAPSample();
	OutSample.FileOffset = InIterator->GetSampleFileOffset();
	OutSample.SampleSize = InIterator->GetSampleSize();

	if (bReadData && DataReader.IsValid() && OutSample.SampleSize > 0)
	{
		OutSample.Data.SetNumUninitialized(OutSample.SampleSize);
		int64 BytesRead = DataReader->ReadData(
			OutSample.Data.GetData(),
			OutSample.SampleSize,
			OutSample.FileOffset,
			MP4Utilities::IMP4DataReaderBase::FCancellationCheckDelegate());

		if (BytesRead < 0)
		{
			LastError = FString::Printf(TEXT("Failed to read sample data at offset %lld, size %lld: %s"),
				OutSample.FileOffset, OutSample.SampleSize, *DataReader->GetLastError());
			OutSample.Data.Empty();
			return ETmvMediaContainerResult::Fail;
		}

		if (BytesRead < OutSample.SampleSize)
		{
			LastError = FString::Printf(TEXT("Partial read at offset %lld: expected %lld bytes, got %lld"),
				OutSample.FileOffset, OutSample.SampleSize, BytesRead);
			OutSample.Data.Empty();
			return ETmvMediaContainerResult::Fail;
		}
	}
	else
	{
		OutSample.Data.Empty();
	}

	return ETmvMediaContainerResult::Success;
}

ETmvMediaContainerResult FTmvMediaMp4Demuxer::Seek(int32 InTrackIndex, FTimespan InTime, FTimespan InLaterTimeThreshold)
{
	if (InTrackIndex < 0 || InTrackIndex >= TrackStates.Num())
	{
		return ETmvMediaContainerResult::Fail;
	}

	FTrackState& State = TrackStates[InTrackIndex];
	if (!State.Track.IsValid())
	{
		return ETmvMediaContainerResult::Fail;
	}

	State.Iterator = State.Track->CreateIteratorAtKeyframe(InTime, InLaterTimeThreshold);
	if (!State.Iterator.IsValid() || !State.Iterator->IsValid())
	{
		LastError = TEXT("Failed to seek to keyframe.");
		return ETmvMediaContainerResult::Fail;
	}

	return ETmvMediaContainerResult::Success;
}

ETmvMediaContainerResult FTmvMediaMp4Demuxer::SeekToSample(int32 InTrackIndex, uint32 InSampleNumber)
{
	if (InTrackIndex < 0 || InTrackIndex >= TrackStates.Num())
	{
		return ETmvMediaContainerResult::Fail;
	}

	FTrackState& State = TrackStates[InTrackIndex];
	if (!State.Track.IsValid())
	{
		return ETmvMediaContainerResult::Fail;
	}

	State.Iterator = State.Track->CreateIterator(InSampleNumber);
	if (!State.Iterator.IsValid() || !State.Iterator->IsValid())
	{
		LastError = FString::Printf(TEXT("Failed to seek to sample %u."), InSampleNumber);
		return ETmvMediaContainerResult::Fail;
	}

	return ETmvMediaContainerResult::Success;
}

void FTmvMediaMp4Demuxer::Close()
{
	TrackStates.Empty();
	RootBoxes.Empty();
	DataReader.Reset();
	MovieDuration = MP4Utilities::FFractionalTime();
	StartTimecodeString.Reset();
	StartTimecodeRateString.Reset();
	LastError.Empty();
}

TOptional<FString> FTmvMediaMp4Demuxer::GetStartTimecode() const
{
	return StartTimecodeString;
}

TOptional<FString> FTmvMediaMp4Demuxer::GetStartTimecodeRate() const
{
	return StartTimecodeRateString;
}

void FTmvMediaMp4Demuxer::ParseStartTimecode(const TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>& InMoovTree)
{
	if (!InMoovTree.IsValid() || !DataReader.IsValid())
	{
		return;
	}

	// Path A: Look for a tmcd (timecode) track.
	TArray<TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>> TrakBoxes;
	InMoovTree->GetAllBoxInstances<MP4Boxes::FMP4BoxTRAK>(TrakBoxes, MP4Utilities::MakeBoxAtom('t','r','a','k'));

	for (const TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>& TrakBox : TrakBoxes)
	{
		TSharedPtr<MP4Boxes::FMP4BoxHDLR, ESPMode::ThreadSafe> HdlrBox =
			TrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxHDLR>(MP4Utilities::MakeBoxAtom('h','d','l','r'));
		if (!HdlrBox.IsValid() || HdlrBox->GetHandlerType() != MP4Utilities::MakeBoxAtom('t','m','c','d'))
		{
			continue;
		}

		// Found a tmcd track. Get the sample description.
		TSharedPtr<MP4Boxes::FMP4BoxSTSD, ESPMode::ThreadSafe> StsdBox =
			TrakBox->FindBoxRecursive<MP4Boxes::FMP4BoxSTSD>(MP4Utilities::MakeBoxAtom('s','t','s','d'));
		if (!StsdBox.IsValid() || StsdBox->GetSampleType() != MP4Boxes::FMP4BoxSampleEntry::ESampleType::QTFFTimecode)
		{
			continue;
		}

		TArray<TSharedPtr<MP4Boxes::FMP4BoxQTFFTimecodeSampleEntry, ESPMode::ThreadSafe>> TcEntries;
		StsdBox->GetSampleDescriptions(TcEntries);
		if (TcEntries.IsEmpty())
		{
			continue;
		}

		const TSharedPtr<MP4Boxes::FMP4BoxQTFFTimecodeSampleEntry, ESPMode::ThreadSafe>& TcEntry = TcEntries[0];
		const uint32 TcFlags = TcEntry->GetFlags();
		const uint32 TcTimescale = TcEntry->GetTimescale();
		const uint32 TcFrameDuration = TcEntry->GetFrameDuration();
		const uint32 TcNumFrames = TcEntry->GetNumberOfFrames();
		const bool bDropFrame = (TcFlags & MP4Boxes::FMP4BoxQTFFTimecodeSampleEntry::EFlags::DropFrame) != 0;
		const bool bMax24Hour = (TcFlags & MP4Boxes::FMP4BoxQTFFTimecodeSampleEntry::EFlags::Max24Hour) != 0;

		// Build frame rate from the timecode sample description.
		const FFrameRate FrameRate(TcTimescale, TcFrameDuration);

		// Read the first sample (4-byte big-endian frame counter) from the tmcd track.
		MP4Boxes::FMP4Track::FFragmentInfo* NoFragments = nullptr;
		TSharedPtr<MP4Boxes::FMP4Track, ESPMode::ThreadSafe> TcTrack =
			MP4Boxes::FMP4Track::Create(TrakBox, TSharedPtr<MP4Boxes::FMP4Track::FFragmentInfo, ESPMode::ThreadSafe>());
		if (!TcTrack.IsValid() || !TcTrack->Prepare(MovieDuration, MovieDuration))
		{
			continue;
		}

		TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe> It = TcTrack->CreateIterator();
		if (!It.IsValid() || !It->IsValid())
		{
			continue;
		}

		const int64 SampleSize = It->GetSampleSize();
		const int64 SampleFileOffset = It->GetSampleFileOffset();
		if (SampleSize != 4)
		{
			continue;
		}

		uint32 TimecodeBuffer = 0;
		const int64 NumRead = DataReader->ReadData(&TimecodeBuffer, SampleSize, SampleFileOffset,
			MP4Utilities::IMP4DataReaderBase::FCancellationCheckDelegate());
		if (NumRead != SampleSize)
		{
			continue;
		}

		const uint32 FrameCounter = MP4Utilities::GetFromBigEndian(TimecodeBuffer);

		// Convert frame counter to FTimecode.
		const double SecondsPerFrame = FrameRate.AsInterval();
		const FTimecode Timecode = FTimecode::FromFrameNumber(FFrameNumber(static_cast<int32>(FrameCounter)), FrameRate, bDropFrame);

		StartTimecodeString.Emplace(Timecode.ToString());
		StartTimecodeRateString.Emplace(FString::Printf(TEXT("%u/%u"), TcTimescale, TcFrameDuration));
		return;
	}

	// Path B: Fallback to udta metadata ('(c)TIM', '(c)TSC', '(c)TSZ' boxes in moov/udta).
	TSharedPtr<MP4Boxes::FMP4BoxUDTA, ESPMode::ThreadSafe> UdtaBox =
		InMoovTree->FindBoxRecursive<MP4Boxes::FMP4BoxUDTA>(MP4Utilities::MakeBoxAtom('u','d','t','a'), 0);
	if (!UdtaBox.IsValid())
	{
		return;
	}

	auto CTIMBox = UdtaBox->FindBoxRecursive<MP4Boxes::FMP4BoxBase>(MP4Utilities::MakeBoxAtom(0xA9U, 'T', 'I', 'M'), 0);
	auto CTSCBox = UdtaBox->FindBoxRecursive<MP4Boxes::FMP4BoxBase>(MP4Utilities::MakeBoxAtom(0xA9U, 'T', 'S', 'C'), 0);
	auto CTSZBox = UdtaBox->FindBoxRecursive<MP4Boxes::FMP4BoxBase>(MP4Utilities::MakeBoxAtom(0xA9U, 'T', 'S', 'Z'), 0);
	if (!CTIMBox.IsValid() || !CTSCBox.IsValid() || !CTSZBox.IsValid())
	{
		return;
	}

	// Parse Apple-style text data: 2-byte length + 2-byte language + UTF-8 string.
	auto GetAppleTextValue = [](const TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>& InBox) -> FString
	{
		MP4Utilities::FMP4AtomReader Reader(InBox->GetBoxData());
		FString Result;
		uint16 TextLength = 0;
		uint16 Language = 0;
		if (Reader.Read(TextLength) && Reader.Read(Language))
		{
			int32 BytesToRead = FMath::Min(static_cast<int32>(TextLength), static_cast<int32>(Reader.GetNumBytesRemaining()));
			if (BytesToRead > 0)
			{
				TArray<uint8> TextData;
				TextData.SetNumUninitialized(BytesToRead);
				if (Reader.ReadBytes(TextData.GetData(), BytesToRead))
				{
					FUTF8ToTCHAR Converter(reinterpret_cast<const ANSICHAR*>(TextData.GetData()), BytesToRead);
					Result = FString(Converter.Length(), Converter.Get());
				}
			}
		}
		return Result;
	};

	const FString TimecodeStr = GetAppleTextValue(CTIMBox);
	const FString TimescaleStr = GetAppleTextValue(CTSCBox);
	const FString FrameDurStr = GetAppleTextValue(CTSZBox);

	if (!TimecodeStr.IsEmpty() && !TimescaleStr.IsEmpty() && !FrameDurStr.IsEmpty())
	{
		StartTimecodeString.Emplace(TimecodeStr);
		StartTimecodeRateString.Emplace(FString::Printf(TEXT("%s/%s"), *TimescaleStr, *FrameDurStr));
	}
}

FString FTmvMediaMp4Demuxer::GetLastError() const
{
	return LastError;
}

} // namespace UE::TmvMedia
