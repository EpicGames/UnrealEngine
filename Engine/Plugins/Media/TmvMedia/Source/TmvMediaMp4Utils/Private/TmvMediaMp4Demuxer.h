// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Decoder/ITmvMediaDemuxer.h"
#include "MP4BoxBase.h"
#include "MP4Track.h"
#include "MP4DataReader.h"
#include "MP4Utilities.h"

namespace UE::TmvMedia
{

/**
 * MP4 container demuxer implementation.
 * Wraps MP4Boxes/MP4Utilities for box tree parsing and sample iteration.
 */
class FTmvMediaMp4Demuxer : public ITmvMediaDemuxer
{
public:
	FTmvMediaMp4Demuxer() = default;
	virtual ~FTmvMediaMp4Demuxer() override;

	// ITmvMediaDemuxer interface
	virtual ETmvMediaContainerResult OpenFile(const FString& InFilePath) override;
	virtual ETmvMediaContainerResult OpenBuffer(TConstArrayView<uint8> InData) override;
	virtual int32 GetTrackCount() const override;
	virtual ETmvMediaContainerResult GetTrackInfo(int32 InTrackIndex, FTmvMediaDemuxerTrackInfo& OutTrackInfo) const override;
	virtual ETmvMediaContainerResult ReadSample(int32 InTrackIndex, FTmvMediaDemuxerSample& OutSample) override;
	virtual ETmvMediaContainerResult ReadSampleInfo(int32 InTrackIndex, FTmvMediaDemuxerSample& OutSample) override;
	virtual ETmvMediaContainerResult Seek(int32 InTrackIndex, FTimespan InTime, FTimespan InLaterTimeThreshold) override;
	virtual ETmvMediaContainerResult SeekToSample(int32 InTrackIndex, uint32 InSampleNumber) override;
	virtual void Close() override;
	virtual FString GetLastError() const override;
	virtual TOptional<FString> GetStartTimecode() const override;
	virtual TOptional<FString> GetStartTimecodeRate() const override;

private:
	/** Parse the box tree from loaded root boxes and build tracks. */
	ETmvMediaContainerResult ParseContainer();

	/** Extract start timecode from tmcd tracks or udta metadata. */
	void ParseStartTimecode(const TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>& InMoovTree);

	/** Populate an FTmvMediaDemuxerSample from the current iterator position. Optionally reads sample data. */
	ETmvMediaContainerResult PopulateSampleFromIterator(
		const TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe>& InIterator,
		FTmvMediaDemuxerSample& OutSample,
		bool bReadData);

	/** Per-track state. */
	struct FTrackState
	{
		TSharedPtr<MP4Boxes::FMP4Track, ESPMode::ThreadSafe> Track;
		TSharedPtr<MP4Boxes::FMP4Track::FIterator, ESPMode::ThreadSafe> Iterator;
		FTmvMediaDemuxerTrackInfo CachedInfo;
	};

	/** Data reader for file or buffer access. */
	TSharedPtr<MP4Utilities::IMP4DataReaderBase, ESPMode::ThreadSafe> DataReader;

	/** Parsed root boxes. */
	TArray<TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>> RootBoxes;

	/** Track states. */
	TArray<FTrackState> TrackStates;

	/** Movie duration from mvhd. */
	MP4Utilities::FFractionalTime MovieDuration;

	/** Parsed start timecode string (e.g., "01:00:00:00"), if available. */
	TOptional<FString> StartTimecodeString;

	/** Parsed start timecode frame rate string (e.g., "30000/1001"), if available. */
	TOptional<FString> StartTimecodeRateString;

	/** Last error message. */
	FString LastError;
};

} // namespace UE::TmvMedia
