// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaMp4DemuxerFactory.h"

#include "Internationalization/Internationalization.h"
#include "TmvMediaMp4Demuxer.h"

FName FTmvMediaMp4DemuxerFactory::GetName() const
{
	static const FName Name(TEXT("Tmv"));
	return Name;
}

FText FTmvMediaMp4DemuxerFactory::GetDisplayName() const
{
	return NSLOCTEXT("TmvMediaMp4Utils", "DemuxerDisplayName", "Tiled-Mipmap Video");
}

TArray<FString> FTmvMediaMp4DemuxerFactory::GetSupportedContainerFormats() const
{
	return { TEXT("tmv") };
}

TSharedPtr<ITmvMediaDemuxer, ESPMode::ThreadSafe> FTmvMediaMp4DemuxerFactory::CreateDemuxer()
{
	return MakeShared<UE::TmvMedia::FTmvMediaMp4Demuxer, ESPMode::ThreadSafe>();
}
