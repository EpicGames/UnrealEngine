// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Decoder/ITmvMediaDemuxerFactory.h"

class FTmvMediaMp4DemuxerFactory : public ITmvMediaDemuxerFactory
{
public:
	virtual ~FTmvMediaMp4DemuxerFactory() override = default;

	// ITmvMediaDemuxerFactory interface
	virtual FName GetName() const override;
	virtual FText GetDisplayName() const override;
	virtual TArray<FString> GetSupportedContainerFormats() const override;
	virtual TSharedPtr<ITmvMediaDemuxer, ESPMode::ThreadSafe> CreateDemuxer() override;
};
