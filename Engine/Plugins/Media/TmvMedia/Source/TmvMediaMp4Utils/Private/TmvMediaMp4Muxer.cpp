// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaMp4Muxer.h"
#include "MP4Utilities.h"
#include "TmvMediaMp4UtilsLog.h"

namespace UE::TmvMedia
{

FTmvMediaMp4Muxer::~FTmvMediaMp4Muxer()
{
	if (Mp4Muxer.IsValid())
	{
		Mp4Muxer->StopAndClose();
		Mp4Muxer.Reset();
	}
}

ETmvMediaContainerResult FTmvMediaMp4Muxer::Configure(const FTmvMediaMuxerConfig& InConfig)
{
	Mp4Muxer = IMP4RawMuxer::Create();
	if (!Mp4Muxer.IsValid())
	{
		UE_LOGF(LogTmvMediaMp4Utils, Error, "FTmvMediaMp4Muxer::Configure Failed to create Mp4RawMuxer");
		return ETmvMediaContainerResult::Fail;
	}

	IMP4RawMuxer::FConfiguration Mp4Config;
	Mp4Config.OutputFilename = InConfig.OutputFilename;
	Mp4Config.TemporaryFilename = InConfig.TemporaryFilename;
	Mp4Config.InterleaveDuration = InConfig.InterleaveDuration;

	switch (InConfig.OutputMode)
	{
	case FTmvMediaMuxerConfig::EOutputMode::Standard:
		Mp4Config.MuxMode = IMP4RawMuxer::FConfiguration::EMuxMode::Standard;
		break;
	case FTmvMediaMuxerConfig::EOutputMode::WebOptimized:
		Mp4Config.MuxMode = IMP4RawMuxer::FConfiguration::EMuxMode::WebOptimized;
		break;
	case FTmvMediaMuxerConfig::EOutputMode::Fragmented:
		Mp4Config.MuxMode = IMP4RawMuxer::FConfiguration::EMuxMode::Fragmented;
		break;
	default:
		Mp4Config.MuxMode = IMP4RawMuxer::FConfiguration::EMuxMode::Standard;
		break;
	}

	if (!Mp4Muxer->Configure(Mp4Config))
	{
		LastError = Mp4Muxer->GetLastError();
		UE_LOGF(LogTmvMediaMp4Utils, Error, "FTmvMediaMp4Muxer::Configure failed: %ls", *LastError);
		return ETmvMediaContainerResult::Fail;
	}

	return ETmvMediaContainerResult::Success;
}

int32 FTmvMediaMp4Muxer::AddTrack(const FTmvMediaMuxerTrackConfig& InTrackConfig)
{
	if (!Mp4Muxer.IsValid())
	{
		LastError = TEXT("Muxer not configured. Call Configure() first.");
		return -1;
	}

	IMP4RawMuxer::FTrackSpec TrackSpec;

	// Set track type properties.
	switch (InTrackConfig.TrackType)
	{
	case ETmvMediaTrackType::Video:
	{
		IMP4RawMuxer::FTrackSpec::FVideo Video;
		Video.DisplayWidth = InTrackConfig.DisplayWidth;
		Video.DisplayHeight = InTrackConfig.DisplayHeight;
		Video.CompressorName = InTrackConfig.CompressorName;
		TrackSpec.Properties.Emplace<IMP4RawMuxer::FTrackSpec::FVideo>(MoveTemp(Video));
		break;
	}
	case ETmvMediaTrackType::Audio:
	{
		IMP4RawMuxer::FTrackSpec::FAudio Audio;
		Audio.SamplingRate = InTrackConfig.SamplingRate;
		Audio.NumberOfChannels = InTrackConfig.NumberOfChannels;
		TrackSpec.Properties.Emplace<IMP4RawMuxer::FTrackSpec::FAudio>(MoveTemp(Audio));
		break;
	}
	case ETmvMediaTrackType::Unspecified:
		LastError = TEXT("\"Unspecified\" track type");
		return -1;
	}

	TrackSpec.SampleEntryFormat = InTrackConfig.SampleEntryFormat;
	TrackSpec.bIsAllKeyframes = InTrackConfig.bIsAllKeyframes;
	TrackSpec.Timescale = InTrackConfig.Timescale;
	TrackSpec.ConstantSampleDuration = InTrackConfig.ConstantSampleDuration;
	TrackSpec.SampleEntryBoxes = InTrackConfig.CodecSpecificBoxes;
	TrackSpec.SubSampleFlags = InTrackConfig.SubSampleFlags;
	TrackSpec.Name = InTrackConfig.TrackName;

	int32 TrackIndex = Mp4Muxer->AddTrack(TrackSpec);
	if (TrackIndex < 0)
	{
		LastError = Mp4Muxer->GetLastError();
		UE_LOGF(LogTmvMediaMp4Utils, Error, "FTmvMediaMp4Muxer::AddTrack failed: %ls", *LastError);
	}
	else if (InTrackConfig.TrackType == ETmvMediaTrackType::Video)
	{
		VideoTrackIndices.Add(TrackIndex);
	}

	return TrackIndex;
}

bool FTmvMediaMp4Muxer::SupportsStartTimecode() const
{
	return true;
}

bool FTmvMediaMp4Muxer::SetStartTimecode(const FTimecode& InTimecode, const FFrameRate& InFrameRate)
{
	if (bStarted)
	{
		UE_LOGF(LogTmvMediaMp4Utils, Error, "FTmvMediaMp4Muxer::SetStartTimecode: Cannot set timecode after Start() has been called.");
		return false;
	}

	StoredTimecode = InTimecode;
	StoredFrameRate = InFrameRate;
	bHasTimecode = true;
	return true;
}

ETmvMediaContainerResult FTmvMediaMp4Muxer::Start(FSampleRequestDelegate InSampleRequestDelegate, FStatusDelegate InStatusDelegate)
{
	if (!Mp4Muxer.IsValid())
	{
		LastError = TEXT("Muxer not configured. Call Configure() first.");
		return ETmvMediaContainerResult::Fail;
	}

	// Create an internal timecode track if SetTimecode() was called.
	if (bHasTimecode)
	{
		IMP4RawMuxer::FTrackSpec TcSpec;
		TcSpec.SampleEntryFormat = MP4Utilities::MakeBoxAtom('t','m','c','d');
		TcSpec.bIsAllKeyframes = true;
		TcSpec.Timescale = static_cast<uint32>(StoredFrameRate.Numerator);
		TcSpec.ConstantSampleDuration = static_cast<uint32>(StoredFrameRate.Denominator);

		IMP4RawMuxer::FTrackSpec::FTimecode TcProps;
		TcProps.bDropFrame = StoredTimecode.bDropFrameFormat;
		TcProps.bMax24Hours = true;
		TcProps.Timescale = StoredFrameRate.Numerator;
		TcProps.FrameDuration = StoredFrameRate.Denominator;
		TcProps.FramesPerSecond = static_cast<int8>(FMath::CeilToInt(static_cast<float>(StoredFrameRate.AsDecimal())));
		TcSpec.Properties.Emplace<IMP4RawMuxer::FTrackSpec::FTimecode>(MoveTemp(TcProps));

		InternalTimecodeTrackIndex = Mp4Muxer->AddTrack(TcSpec);
		if (InternalTimecodeTrackIndex >= 0)
		{
			// Link all video tracks to the timecode track.
			for (int32 VideoIdx : VideoTrackIndices)
			{
				Mp4Muxer->AddTrackReference(VideoIdx, InternalTimecodeTrackIndex, MP4Utilities::MakeBoxAtom('t','m','c','d'));
			}

			// Pre-build the timecode sample: 4-byte big-endian frame counter.
			const uint32 FrameCounter = static_cast<uint32>(StoredTimecode.ToFrameNumber(StoredFrameRate).Value);
			TimecodeSampleData.SetNumUninitialized(4);
			TimecodeSampleData[0] = static_cast<uint8>((FrameCounter >> 24) & 0xFF);
			TimecodeSampleData[1] = static_cast<uint8>((FrameCounter >> 16) & 0xFF);
			TimecodeSampleData[2] = static_cast<uint8>((FrameCounter >> 8) & 0xFF);
			TimecodeSampleData[3] = static_cast<uint8>(FrameCounter & 0xFF);
		}
	}

	// Adapt the MP4 muxer's delegate to the TmvMedia interface delegate.
	// If a timecode track was created, its sample requests are handled internally
	// and not forwarded to the caller.
	IMP4RawMuxer::FRequestTrackDataDelegate Mp4RequestDelegate;
	Mp4RequestDelegate.BindSPLambda(this, [this, InSampleRequestDelegate](const TArray<IMP4RawMuxer::FSampleRequest>& InMp4Requests)
	{
		// Handle internal timecode track requests.
		if (InternalTimecodeTrackIndex >= 0 && !bTimecodeSampleDelivered)
		{
			for (const IMP4RawMuxer::FSampleRequest& Req : InMp4Requests)
			{
				if (Req.TrackIndex == InternalTimecodeTrackIndex)
				{
					IMP4RawMuxer::FTrackSample TcSample;
					TcSample.Data = MakeArrayView(TimecodeSampleData.GetData(), static_cast<int64>(TimecodeSampleData.Num()));
					TcSample.SampleNumber = 0;
					TcSample.bIsKeyframe = true;
					TcSample.bIsFinalSample = true;
					if (Mp4Muxer->AddTrackSample(InternalTimecodeTrackIndex, TcSample))
					{
						bTimecodeSampleDelivered = true;
					}
					break;
				}
			}
		}

		// Forward non-timecode requests to the caller.
		TArray<ITmvMediaMuxer::FSampleRequest> TmvRequests;
		TmvRequests.Reserve(InMp4Requests.Num());
		for (const IMP4RawMuxer::FSampleRequest& Mp4Req : InMp4Requests)
		{
			if (Mp4Req.TrackIndex != InternalTimecodeTrackIndex)
			{
				ITmvMediaMuxer::FSampleRequest TmvReq;
				TmvReq.TrackIndex = Mp4Req.TrackIndex;
				TmvReq.SampleNumber = Mp4Req.SampleNumber;
				TmvRequests.Add(TmvReq);
			}
		}
		if (TmvRequests.Num() > 0)
		{
			InSampleRequestDelegate.ExecuteIfBound(TmvRequests);
		}
	});

	IMP4RawMuxer::FStatusDelegate Mp4StatusDelegate;
	Mp4StatusDelegate.BindLambda([InStatusDelegate](IMP4RawMuxer::EStatus InMp4Status)
	{
		ITmvMediaMuxer::EStatus TmvStatus = (InMp4Status == IMP4RawMuxer::EStatus::Finished)
			? ITmvMediaMuxer::EStatus::Finished
			: ITmvMediaMuxer::EStatus::Failed;
		InStatusDelegate.ExecuteIfBound(TmvStatus);
	});

	if (!Mp4Muxer->Start(Mp4RequestDelegate, Mp4StatusDelegate))
	{
		LastError = Mp4Muxer->GetLastError();
		UE_LOGF(LogTmvMediaMp4Utils, Error, "FTmvMediaMp4Muxer::Start failed: %ls", *LastError);
		return ETmvMediaContainerResult::Fail;
	}

	bStarted = true;
	return ETmvMediaContainerResult::Success;
}

ETmvMediaContainerResult FTmvMediaMp4Muxer::AddSample(int32 InTrackIndex, const FTmvMediaMuxerSample& InSample)
{
	if (!Mp4Muxer.IsValid())
	{
		LastError = TEXT("Muxer not configured.");
		return ETmvMediaContainerResult::Fail;
	}

	IMP4RawMuxer::FTrackSample Mp4Sample;
	Mp4Sample.Data = InSample.Data;
	Mp4Sample.Duration = InSample.Duration;
	Mp4Sample.DTS = InSample.DTS;
	Mp4Sample.PTS = InSample.PTS;
	Mp4Sample.SampleNumber = InSample.SampleNumber;
	Mp4Sample.bIsKeyframe = InSample.bIsKeyframe;
	Mp4Sample.bIsFinalSample = InSample.bIsFinalSample;

	// Convert subsample info.
	Mp4Sample.SubSamples.Reserve(InSample.SubSamples.Num());
	for (const FTmvMediaMuxerSample::FSubSampleInfo& SubSample : InSample.SubSamples)
	{
		IMP4RawMuxer::FTrackSample::FSubSampleInfo Mp4Sub;
		Mp4Sub.codec_specific_parameters = SubSample.CodecSpecificParameters;
		Mp4Sub.subsample_size = SubSample.SubSampleSize;
		Mp4Sub.subsample_priority = SubSample.SubSamplePriority;
		Mp4Sub.discardable = SubSample.Discardable;
		Mp4Sample.SubSamples.Add(Mp4Sub);
	}

	if (!Mp4Muxer->AddTrackSample(InTrackIndex, Mp4Sample))
	{
		LastError = Mp4Muxer->GetLastError();
		UE_LOGF(LogTmvMediaMp4Utils, Error, "FTmvMediaMp4Muxer::AddSample failed for track %d, sample %u: %ls", InTrackIndex, InSample.SampleNumber, *LastError);
		return ETmvMediaContainerResult::Fail;
	}

	return ETmvMediaContainerResult::Success;
}

ETmvMediaContainerResult FTmvMediaMp4Muxer::Finalize()
{
	if (!Mp4Muxer.IsValid())
	{
		LastError = TEXT("Muxer not configured.");
		return ETmvMediaContainerResult::Fail;
	}
	
	ETmvMediaContainerResult Result = ETmvMediaContainerResult::Success;

	if (!Mp4Muxer->StopAndClose())
	{
		LastError = Mp4Muxer->GetLastError();
		UE_LOGF(LogTmvMediaMp4Utils, Error, "FTmvMediaMp4Muxer::Finalize failed: %ls", *LastError);
		Result = ETmvMediaContainerResult::Fail;
	}

	Mp4Muxer.Reset();
	return Result;
}

FString FTmvMediaMp4Muxer::GetLastError() const
{
	return LastError;
}

} // namespace UE::TmvMedia
