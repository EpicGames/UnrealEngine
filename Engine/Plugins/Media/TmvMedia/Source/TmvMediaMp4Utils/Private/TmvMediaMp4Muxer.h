// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Encoder/ITmvMediaMuxer.h"
#include "MP4Muxer.h"

namespace UE::TmvMedia
{

/**
 * MP4 container muxer implementation.
 * Wraps IMP4RawMuxer from the MP4Muxer module.
 */
class FTmvMediaMp4Muxer : public ITmvMediaMuxer, public TSharedFromThis<FTmvMediaMp4Muxer, ESPMode::ThreadSafe>
{
public:
	FTmvMediaMp4Muxer() = default;
	virtual ~FTmvMediaMp4Muxer() override;

	// ITmvMediaMuxer interface
	virtual ETmvMediaContainerResult Configure(const FTmvMediaMuxerConfig& InConfig) override;
	virtual int32 AddTrack(const FTmvMediaMuxerTrackConfig& InTrackConfig) override;
	virtual bool SupportsStartTimecode() const override;
	virtual bool SetStartTimecode(const FTimecode& InTimecode, const FFrameRate& InFrameRate) override;
	virtual ETmvMediaContainerResult Start(FSampleRequestDelegate InSampleRequestDelegate, FStatusDelegate InStatusDelegate) override;
	virtual ETmvMediaContainerResult AddSample(int32 InTrackIndex, const FTmvMediaMuxerSample& InSample) override;
	virtual ETmvMediaContainerResult Finalize() override;
	virtual FString GetLastError() const override;

private:
	/** The underlying MP4 raw muxer. */
	TSharedPtr<IMP4RawMuxer, ESPMode::ThreadSafe> Mp4Muxer;

	/** Last error message. */
	FString LastError;

	/** Whether Start() has been called on the underlying muxer. */
	bool bStarted = false;

	/** Track indices of video tracks added by the caller (for timecode track references). */
	TArray<int32> VideoTrackIndices;

	/** Stored timecode set via SetTimecode(). */
	FTimecode StoredTimecode;
	FFrameRate StoredFrameRate;
	bool bHasTimecode = false;

	/** Internal timecode track index, or -1 if not created. */
	int32 InternalTimecodeTrackIndex = -1;

	/** Pre-built 4-byte big-endian timecode sample data. */
	TArray<uint8> TimecodeSampleData;

	/** Whether the single timecode sample has been delivered to the underlying muxer. */
	bool bTimecodeSampleDelivered = false;
};

} // namespace UE::TmvMedia
