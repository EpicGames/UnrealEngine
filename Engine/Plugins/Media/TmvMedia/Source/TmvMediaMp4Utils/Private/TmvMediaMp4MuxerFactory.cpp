// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaMp4MuxerFactory.h"

#include "Internationalization/Internationalization.h"
#include "TmvMediaMp4Muxer.h"

FName FTmvMediaMp4MuxerFactory::GetName() const
{
	static const FName Name(TEXT("Tmv"));
	return Name;
}

FText FTmvMediaMp4MuxerFactory::GetDisplayName() const
{
	return NSLOCTEXT("TmvMediaMp4Utils", "MuxerDisplayName", "Tiled-Mipmap Video");
}

TArray<FString> FTmvMediaMp4MuxerFactory::GetSupportedContainerFormats() const
{
	return { TEXT("tmv") };
}

FString FTmvMediaMp4MuxerFactory::GetFileExtension() const
{
	return TEXT("tmv");
}

TSharedPtr<ITmvMediaMuxer, ESPMode::ThreadSafe> FTmvMediaMp4MuxerFactory::CreateMuxer()
{
	return MakeShared<UE::TmvMedia::FTmvMediaMp4Muxer, ESPMode::ThreadSafe>();
}
