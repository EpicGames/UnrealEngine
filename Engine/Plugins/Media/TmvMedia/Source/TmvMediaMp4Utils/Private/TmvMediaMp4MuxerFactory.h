// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Encoder/ITmvMediaMuxerFactory.h"

class FTmvMediaMp4MuxerFactory : public ITmvMediaMuxerFactory
{
public:
	virtual ~FTmvMediaMp4MuxerFactory() override = default;

	// ITmvMediaMuxerFactory interface
	virtual FName GetName() const override;
	virtual FText GetDisplayName() const override;
	virtual TArray<FString> GetSupportedContainerFormats() const override;
	virtual FString GetFileExtension() const override;
	virtual TSharedPtr<ITmvMediaMuxer, ESPMode::ThreadSafe> CreateMuxer() override;
};
