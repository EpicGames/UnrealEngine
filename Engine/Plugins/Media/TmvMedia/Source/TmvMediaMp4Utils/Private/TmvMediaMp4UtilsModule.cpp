// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaMp4UtilsLog.h"
#include "TmvMediaMp4MuxerFactory.h"
#include "TmvMediaMp4DemuxerFactory.h"
#include "ITmvMediaModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogTmvMediaMp4Utils);

/** Implementation of TmvMediaMp4Utils Module. */
class FTmvMediaMp4UtilsModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		MuxerFactory = MakeShared<FTmvMediaMp4MuxerFactory>();
		ITmvMediaModule::GetOrLoad().RegisterMuxerFactory(MuxerFactory);

		DemuxerFactory = MakeShared<FTmvMediaMp4DemuxerFactory>();
		ITmvMediaModule::GetOrLoad().RegisterDemuxerFactory(DemuxerFactory);
	}

	virtual void ShutdownModule() override
	{
		if (ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get())
		{
			TmvMediaModule->UnregisterMuxerFactory(MuxerFactory);
			TmvMediaModule->UnregisterDemuxerFactory(DemuxerFactory);
		}
		MuxerFactory.Reset();
		DemuxerFactory.Reset();
	}
	//~ End IModuleInterface

private:
	TSharedPtr<FTmvMediaMp4MuxerFactory, ESPMode::ThreadSafe> MuxerFactory;
	TSharedPtr<FTmvMediaMp4DemuxerFactory, ESPMode::ThreadSafe> DemuxerFactory;
};

IMPLEMENT_MODULE(FTmvMediaMp4UtilsModule, TmvMediaMp4Utils);
