// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class TmvMediaMp4Utils : ModuleRules
	{
		public TmvMediaMp4Utils(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"TmvMedia",
					"MP4Muxer",
					"MP4Boxes",
					"MP4Utilities",
				});
		}
	}
}
