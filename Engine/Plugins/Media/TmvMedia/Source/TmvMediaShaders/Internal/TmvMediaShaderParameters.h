// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Matrix.h"
#include "ShaderParameterStruct.h"

/**
 * Definition of the common parameters for color management in the sample conversion shaders. 
 */
BEGIN_SHADER_PARAMETER_STRUCT(FTmvMediaShaderColorParameters, )
	SHADER_PARAMETER(uint32, bApplyColorTransform)
	SHADER_PARAMETER(uint32, EOTF)
	SHADER_PARAMETER(FMatrix44f, ColorSpaceMatrix)
	SHADER_PARAMETER(uint32, bConvertYUV)
	SHADER_PARAMETER(FMatrix44f, YUVMatrix)
	SHADER_PARAMETER(float, AlphaScale)
	SHADER_PARAMETER(FVector4f, MipTint)
END_SHADER_PARAMETER_STRUCT()
