// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameterStruct.h"
#include "TmvMediaShaderDefines.h"
#include "TmvMediaShaderParameters.h"

// The vertex shader used by DrawScreenPass to draw a rectangle.
class FTmvSwizzleVS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FTmvSwizzleVS, TMVMEDIASHADERS_API);

	FTmvSwizzleVS() = default;
	FTmvSwizzleVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

/** Define the buffer memory layout (depend on decoder). */
enum class ETmvSwizzleBufferLayouts
{
	/** The buffer contains a subset of tiles. Requires a separate tile mapping buffer. */
	TiledPartial = 0,
	/** The buffer contains a full set of tiles for the frame. */
	TiledFull = 1,
	/** The buffer planar component buffers (not tiled). */
	PlanarFull = 2,

	Count
};

/**
 * Pixel shader swizzle multi-component planar or tiled buffer into a packed RGBA texture.
 */
class FTmvStructuredBufferSwizzlePS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FTmvStructuredBufferSwizzlePS, TMVMEDIASHADERS_API);
	SHADER_USE_PARAMETER_STRUCT(FTmvStructuredBufferSwizzlePS, FGlobalShader);

	/** Defines the number of components in the input buffer */
	class FNumComponents : SHADER_PERMUTATION_INT("PERMUTATION_CHANNELS", 4);

	/** Define the buffer memory layout (depend on decoder). */
	class FBufferLayout : SHADER_PERMUTATION_INT("BUFFER_LAYOUT", static_cast<int32>(ETmvSwizzleBufferLayouts::Count));

	/** Defines the buffer element format (float or int). */
	class FElementFormat : SHADER_PERMUTATION_INT("ELEMENT_FORMAT", 2);

	using FPermutationDomain = TShaderPermutationDomain<FNumComponents, FBufferLayout, FElementFormat>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SRV(StructuredBuffer<uint>, UnswizzledBuffer)
		SHADER_PARAMETER_SRV(StructuredBuffer<FTmvMediaShaderTileDesc>, TileMappingBuffer)
		SHADER_PARAMETER(FIntPoint, DestTextureSize)
		SHADER_PARAMETER(FIntPoint, TileSize)
		SHADER_PARAMETER(FIntPoint, NumTiles)
		SHADER_PARAMETER(int32, TileBufferFullStride)
		SHADER_PARAMETER(FIntVector4, TileBufferOffsets)
		SHADER_PARAMETER(FIntVector4, TileBufferStrides)
		SHADER_PARAMETER(FIntVector4, PlaneBufferOffsets)
		SHADER_PARAMETER(FIntVector4, PlaneBufferStrides)
		SHADER_PARAMETER(FIntVector4, PlaneWidthRatios)
		SHADER_PARAMETER(FIntVector4, PlaneHeightRatios)
		SHADER_PARAMETER_STRUCT_INCLUDE(FTmvMediaShaderColorParameters, ColorParams)
	END_SHADER_PARAMETER_STRUCT()
};