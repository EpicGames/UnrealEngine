// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameterStruct.h"
#include "TmvMediaShaderParameters.h"

/**
 * Pixel shader swizzle multi-component planar texture buffer into a packed RGBA texture.
 */
class FTmvTextureBufferSwizzlePS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FTmvTextureBufferSwizzlePS, TMVMEDIASHADERS_API);
	SHADER_USE_PARAMETER_STRUCT(FTmvTextureBufferSwizzlePS, FGlobalShader);

	/** Defines the number of components in the input buffer */
	class FNumComponents : SHADER_PERMUTATION_INT("PERMUTATION_CHANNELS", 4);
	using FPermutationDomain = TShaderPermutationDomain<FNumComponents>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SRV(Texture2D, InPlaneTexture0)
		SHADER_PARAMETER_SRV(Texture2D, InPlaneTexture1)
		SHADER_PARAMETER_SRV(Texture2D, InPlaneTexture2)
		SHADER_PARAMETER_SRV(Texture2D, InPlaneTexture3)
		SHADER_PARAMETER_SAMPLER(SamplerState, InPlaneTextureSampler)
		SHADER_PARAMETER_STRUCT_INCLUDE(FTmvMediaShaderColorParameters, ColorParams)
	END_SHADER_PARAMETER_STRUCT()
};