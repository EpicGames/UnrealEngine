// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameterStruct.h"
#include "TmvMediaShaderParameters.h"

/**
 * Pixel shader to convert an RGBA (media) texture back to YUVA.
 */
class FTmvTextureYuvConverterPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FTmvTextureYuvConverterPS, TMVMEDIASHADERS_API);
	SHADER_USE_PARAMETER_STRUCT(FTmvTextureYuvConverterPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InTextureSampler)
		SHADER_PARAMETER(FVector4f, UVScaleAndOffset)
		SHADER_PARAMETER(FVector4f, DataScale)
		SHADER_PARAMETER_STRUCT_INCLUDE(FTmvMediaShaderColorParameters, ColorParams)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};