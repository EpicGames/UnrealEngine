// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaShadersModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"

void FTmvMediaShadersModule::StartupModule()
{
	const TSharedPtr<IPlugin> TmvMediaPlugin = IPluginManager::Get().FindPlugin(TEXT("TmvMedia"));
	if (ensure(TmvMediaPlugin))
	{
		const FString PluginShaderDir = FPaths::Combine(TmvMediaPlugin->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/TmvMedia"), PluginShaderDir);
	}
}

IMPLEMENT_MODULE(FTmvMediaShadersModule, TmvMediaShaders);
