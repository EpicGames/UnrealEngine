// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * Module used to define TMV media shaders used for pixel format conversion
 */
class FTmvMediaShadersModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	//~ End IModuleInterface
};
