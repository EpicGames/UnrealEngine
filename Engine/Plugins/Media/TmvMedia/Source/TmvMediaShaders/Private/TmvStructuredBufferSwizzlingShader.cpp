// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvStructuredBufferSwizzlingShader.h"
#include "GlobalShader.h"
#include "ShaderCore.h"
#include "ShaderParameterMacros.h"
#include "CoreMinimal.h"

IMPLEMENT_GLOBAL_SHADER(FTmvStructuredBufferSwizzlePS, "/Plugin/TmvMedia/Private/TmvStructuredBufferSwizzler.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FTmvSwizzleVS, "/Plugin/TmvMedia/Private/TmvStructuredBufferSwizzler.usf", "MainVS", SF_Vertex);
