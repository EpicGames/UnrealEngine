// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"

/**
 * Defines the memory layout of a planar tile buffer.
 * When using tiled memory layout, the size of tiles per plane is defined in
 * FTmvMediaFrameMipInfo::Planes (FTmvMediaFramePlaneInfo).
 * @remark For YUV color components, each plane can have different strides because of chroma subsampling.
 */
struct FTmvMediaShaderTileDesc
{
	/** Offset of each component plane. Includes padding if any. */
	FIntVector4 Offsets;

	/** Stride of each component plane. */
	FIntVector4 Strides;
};
