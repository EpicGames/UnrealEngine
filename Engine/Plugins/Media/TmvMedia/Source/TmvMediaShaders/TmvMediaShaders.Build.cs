// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TmvMediaShaders : ModuleRules
{
	public TmvMediaShaders(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Projects",
				"RenderCore",
				"RHI"
			});
	}
}