{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Tiled Mipmap Video Player",
	"Description" : "Framework for tiled-mipmap video (TMV) playback, includes transcoding tools.\nImplemented using Advanced Professional Video (APV) codec.",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "TmvMedia",
			"Type" : "Runtime",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "TmvMediaEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "TmvMediaShaders",
			"Type" : "Runtime",
			"LoadingPhase" : "PostConfigInit"
		},
		{
			"Name" : "ApvMedia",
			"Type" : "Runtime",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" : [ "Win64" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name" : "TmvMediaMp4Utils",
			"Type" : "Runtime",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" : [ "Win64" ]
		}
	],
	"Plugins" :
	[
		{
			"Name" : "MP4Utilities",
			"Enabled" : true
		}
	]
}
